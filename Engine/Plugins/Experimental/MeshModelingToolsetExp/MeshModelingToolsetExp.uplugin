{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Experimental Mesh Modeling Toolset",
	"Description": "A set of experimental modules implementing 3D mesh creation and editing based on the Interactive Tools Framework",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "MeshModelingToolsExp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshModelingToolsEditorOnlyExp",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryProcessingAdapters",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "ModelingEditorUI",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelingUI",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "ProxyLODPlugin",
			"Enabled": true
		}
	]
}
