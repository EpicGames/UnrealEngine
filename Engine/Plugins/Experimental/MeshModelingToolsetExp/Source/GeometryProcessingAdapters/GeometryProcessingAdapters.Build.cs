// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryProcessingAdapters : ModuleRules
{
	public GeometryProcessingAdapters(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"GeometryProcessingInterfaces",
				
                "InteractiveToolsFramework",
                "MeshDescription",
				"StaticMeshDescription",
				"ModelingOperators",
				"GeometryCore",
				"DynamicMesh",
				"MeshConversion",
				"GeometryAlgorithms", // required for constrained Delaunay triangulation
				"MeshUtilitiesCommon", // required by uvlayoutop
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"ModelingOperators",
				"ModelingOperatorsEditorOnly"
				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",

				"MaterialUtilities",
				"MeshBuilder",
				"MeshReductionInterface", // for UE standard simplification 
				"MeshUtilities",			// for tangents calculation
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);


		AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");

	}
}
