// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryProcessing/ApproximateActorsImpl.h"

#include "GameFramework/Actor.h"
#include "Scene/MeshSceneAdapter.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMesh/ColliderMesh.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "Implicit/Solidify.h"
#include "Implicit/Morphology.h"
#include "Operations/RemoveOccludedTriangles.h"
#include "Operations/MeshPlaneCut.h"
#include "ConstrainedDelaunay2.h"
#include "ParameterizationOps/ParameterizeMeshOp.h"
#include "Parameterization/MeshUVPacking.h"
#include "MeshQueries.h"
#include "ProjectionTargets.h"
#include "Selections/MeshFaceSelection.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#include "AssetUtils/CreateStaticMeshUtil.h"
#include "AssetUtils/CreateTexture2DUtil.h"
#include "AssetUtils/CreateMaterialUtil.h"
#include "AssetUtils/Texture2DUtil.h"
#include "UObject/UObjectGlobals.h"		// for CreatePackage

#include "ImageUtils.h"
#include "Image/ImageInfilling.h"
#include "Sampling/MeshGenericWorldPositionBaker.h"
#include "Scene/SceneCapturePhotoSet.h"
#include "AssetUtils/Texture2DBuilder.h"

#include "Engine/StaticMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialUtilities.h"
#include "ShaderCore.h"

#include "MeshMerge/MeshApproximationSettings.h"

#include "Async/Async.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderCaptureInterface.h"

using namespace UE::Geometry;
using namespace UE::AssetUtils;

DEFINE_LOG_CATEGORY_STATIC(LogApproximateActors, Log, All);

#define LOCTEXT_NAMESPACE "ApproximateActorsImpl"

static TAutoConsoleVariable<int32> CVarApproximateActorsRDOCCapture(
	TEXT("ApproximateActors.RenderCapture"),
	0,
	TEXT("Determines whether or not to trigger a render capture.\n")
	TEXT("0: Turned Off\n")
	TEXT("1: Turned On"),
	ECVF_Default);

struct FGeneratedResultTextures
{
	UTexture2D* BaseColorMap = nullptr;
	UTexture2D* RoughnessMap = nullptr;
	UTexture2D* MetallicMap = nullptr;
	UTexture2D* SpecularMap = nullptr;
	UTexture2D* PackedMRSMap = nullptr;
	UTexture2D* EmissiveMap = nullptr;
	UTexture2D* NormalMap = nullptr;
};


static TUniquePtr<FSceneCapturePhotoSet> CapturePhotoSet(
	const IGeometryProcessing_ApproximateActors::FInput& Input,
	const IGeometryProcessing_ApproximateActors::FOptions& Options
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Captures);

	double FieldOfView = Options.FieldOfViewDegrees;
	double NearPlaneDist = Options.NearPlaneDist;

	FImageDimensions CaptureDimensions(Options.RenderCaptureImageSize, Options.RenderCaptureImageSize);

	TUniquePtr<FSceneCapturePhotoSet> SceneCapture = MakeUnique<FSceneCapturePhotoSet>();


	SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::BaseColor, Options.bBakeBaseColor);
	SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::WorldNormal, Options.bBakeNormalMap);
	SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Emissive, Options.bBakeEmissive);

	bool bMetallic = Options.bBakeMetallic;
	bool bRoughness = Options.bBakeRoughness;
	bool bSpecular  = Options.bBakeSpecular;
	if (Options.bUsePackedMRS && (bMetallic || bRoughness || bSpecular ) )
	{
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::CombinedMRS, true);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Roughness, false);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Metallic, false);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Specular, false);
	}
	else
	{
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::CombinedMRS, false);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Roughness, bRoughness);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Metallic, bMetallic);
		SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Specular, bSpecular);
	}

	// These capture types aren't yet supported by the Approximate Actors interface
	SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::Opacity, false);
	SceneCapture->SetCaptureTypeEnabled(ERenderCaptureType::SubsurfaceColor, false);

	UWorld* World = Input.Actors.IsEmpty() ? (Input.Components.IsEmpty() ? nullptr : Input.Components[0]->GetWorld()) : Input.Actors[0]->GetWorld();

	SceneCapture->SetCaptureSceneActorsAndComponents(World, Input.Actors, Input.Components);

	const TArray<FSpatialPhotoParams> SpatialParams = ComputeStandardExteriorSpatialPhotoParameters(
		World,
		Input.Actors,
		Input.Components,
		CaptureDimensions,
		FieldOfView,
		NearPlaneDist,
		true, true, true, true, true);

	SceneCapture->SetSpatialPhotoParams(SpatialParams);

	SceneCapture->Compute();

	return SceneCapture;
}

static void BakeTexturesFromPhotoCapture(
	TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const IGeometryProcessing_ApproximateActors::FOptions& Options, 
	FGeneratedResultTextures& GeneratedTextures,
	const FDynamicMesh3* WorldTargetMesh,
	const FMeshTangentsd* MeshTangents
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures);

	int32 UVLayer = 0;
	int32 Supersample = FMath::Max(1, Options.AntiAliasMultiSampling);
	if ( (Options.TextureImageSize * Supersample) > 16384)
	{
		UE_LOGF(LogApproximateActors, Warning, "Ignoring requested supersampling rate %d because it would require image buffers with resolution %d, please try lower value.", Supersample, Options.TextureImageSize * Supersample);
		Supersample = 1;
	}

	FImageDimensions OutputDimensions(Options.TextureImageSize*Supersample, Options.TextureImageSize*Supersample);


	FScopedSlowTask Progress(8.f, LOCTEXT("BakingTextures", "Baking Textures..."));
	Progress.MakeDialog(true);

	Progress.EnterProgressFrame(1.f, LOCTEXT("BakingSetup", "Setup..."));

	FDynamicMeshAABBTree3 Spatial(WorldTargetMesh, true);

	FMeshImageBakingCache TempBakeCache;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_MakeCache);
		TempBakeCache.SetDetailMesh(WorldTargetMesh, &Spatial);
		TempBakeCache.SetBakeTargetMesh(WorldTargetMesh);
		TempBakeCache.SetDimensions(OutputDimensions);
		TempBakeCache.SetUVLayer(UVLayer);
		TempBakeCache.SetThickness(0.1);
		TempBakeCache.SetCorrespondenceStrategy(FMeshImageBakingCache::ECorrespondenceStrategy::Identity);
		TempBakeCache.ValidateCache();
	}

	Progress.EnterProgressFrame(1.f, LOCTEXT("BakingBaseColor", "Baking Base Color..."));

	FAxisAlignedBox3d TargetBounds = WorldTargetMesh->GetBounds();
	double RayOffsetHackDist = (double)(100.0f * FMathf::ZeroTolerance * TargetBounds.MinDim() );

	auto VisibilityFunction = [&Spatial, RayOffsetHackDist](const FVector3d& SurfPos, const FVector3d& ImagePosWorld)
	{
		FVector3d RayDir = ImagePosWorld - SurfPos;
		double Dist = Normalize(RayDir);
		FVector3d RayOrigin = SurfPos + RayOffsetHackDist * RayDir;
		int32 HitTID = Spatial.FindNearestHitTriangle(FRay3d(RayOrigin, RayDir), IMeshSpatial::FQueryOptions(Dist));
		return (HitTID == IndexConstants::InvalidID);
	};

	auto GetNumDownsampleSteps = [&](int32 TargetImageSize)
	{
		return (Options.TextureImageSize / TargetImageSize) * Supersample;
	};

	FSceneCapturePhotoSet::FSceneSample DefaultSample;
	FVector4f InvalidColor(0, -1, 0, 1);
	DefaultSample.BaseColor = FVector3f(InvalidColor.X, InvalidColor.Y, InvalidColor.Z);

	FMeshGenericWorldPositionColorBaker BaseColorBaker;
	BaseColorBaker.SetCache(&TempBakeCache);
	BaseColorBaker.ColorSampleFunction = [&](FVector3d Position, FVector3d Normal) {
		FSceneCapturePhotoSet::FSceneSample Sample = DefaultSample;
		SceneCapture->ComputeSample(FRenderCaptureTypeFlags::BaseColor(),
			Position, Normal, VisibilityFunction, Sample);
		return Sample.GetValue4f(ERenderCaptureType::BaseColor);
	};
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_BakeColor);
		BaseColorBaker.Bake();
	}

	// find "hole" pixels
	TArray<FVector2i> MissingPixels;
	TUniquePtr<TImageBuilder<FVector4f>> ColorImage = BaseColorBaker.TakeResult();
	TMarchingPixelInfill<FVector4f> Infill;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_ComputeInfill);
		TempBakeCache.FindSamplingHoles([&](const FVector2i& Coords)
		{
			return ColorImage->GetPixel(Coords) == InvalidColor;
		}, MissingPixels);

		// solve infill for the holes while also caching infill information
		Infill.ComputeInfill(*ColorImage, MissingPixels, InvalidColor,
			[](FVector4f SumValue, int32 Count) {
			float InvSum = (Count == 0) ? 1.0f : (1.0f / Count);
			return FVector4f(SumValue.X * InvSum, SumValue.Y * InvSum, SumValue.Z * InvSum, 1.0f);
		});
	}

	// downsample the image if necessary
	int32 BaseColorDownsampleSteps = GetNumDownsampleSteps(Options.CustomTextureSizeBaseColor);
	if (BaseColorDownsampleSteps > 1)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_Downsample);
		TImageBuilder<FVector4f> Downsampled = ColorImage->FastDownsample(BaseColorDownsampleSteps, FVector4f::Zero(), [](FVector4f V, int N) { return V / (float)N; });
		*ColorImage = MoveTemp(Downsampled);
	}

	// this lambda is used to process the per-channel images. It does the bake, applies infill, and downsamples if necessary
	auto ProcessChannelFunc = [&](ERenderCaptureType CaptureType, int32 CaptureTextureSize)
	{
		FVector4f DefaultValue(0, 0, 0, 0);
		FMeshGenericWorldPositionColorBaker ChannelBaker;
		ChannelBaker.SetCache(&TempBakeCache);
		ChannelBaker.ColorSampleFunction = [&](FVector3d Position, FVector3d Normal) {
			FSceneCapturePhotoSet::FSceneSample Sample = DefaultSample;
			SceneCapture->ComputeSample(FRenderCaptureTypeFlags::Single(CaptureType), Position, Normal, VisibilityFunction, Sample);
			return Sample.GetValue4f(CaptureType);
		};
		ChannelBaker.Bake();
		TUniquePtr<TImageBuilder<FVector4f>> Image = ChannelBaker.TakeResult();

		Infill.ApplyInfill(*Image,
			[](FVector4f SumValue, int32 Count) {
			float InvSum = (Count == 0) ? 1.0f : (1.0f / Count);
			return FVector4f(SumValue.X * InvSum, SumValue.Y * InvSum, SumValue.Z * InvSum, 1.0f);
		});

		int32 DownsampleSteps = GetNumDownsampleSteps(CaptureTextureSize);
		if (DownsampleSteps > 1)
		{
			TImageBuilder<FVector4f> Downsampled = Image->FastDownsample(DownsampleSteps, FVector4f::Zero(), [](FVector4f V, int N) { return V / (float)N; });
			*Image = MoveTemp(Downsampled);
		}

		return MoveTemp(Image);
	};

	bool bMetallic = Options.bBakeMetallic;
	bool bRoughness = Options.bBakeRoughness;
	bool bSpecular = Options.bBakeSpecular;
	TUniquePtr<TImageBuilder<FVector4f>> RoughnessImage, MetallicImage, SpecularImage, PackedMRSImage, EmissiveImage;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_OtherChannels);

		if (Options.bUsePackedMRS && (bMetallic || bRoughness || bSpecular))
		{
			int32 MRSTextureSize = FMath::Max(Options.CustomTextureSizeRoughness, FMath::Max(Options.CustomTextureSizeMetallic, Options.CustomTextureSizeSpecular));
			PackedMRSImage = ProcessChannelFunc(ERenderCaptureType::CombinedMRS, MRSTextureSize);
		}
		else
		{
			if (bRoughness)
			{
				RoughnessImage = ProcessChannelFunc(ERenderCaptureType::Roughness, Options.CustomTextureSizeRoughness);
			}
			if (bMetallic)
			{
				MetallicImage = ProcessChannelFunc(ERenderCaptureType::Metallic, Options.CustomTextureSizeMetallic);
			}
			if (bSpecular)
			{
				SpecularImage = ProcessChannelFunc(ERenderCaptureType::Specular, Options.CustomTextureSizeSpecular);
			}
		}

		if (Options.bBakeEmissive)
		{
			EmissiveImage = ProcessChannelFunc(ERenderCaptureType::Emissive, Options.CustomTextureSizeEmissive);
		}
	}



	Progress.EnterProgressFrame(1.f, LOCTEXT("BakingNormals", "Baking Normals..."));

	TUniquePtr<TImageBuilder<FVector3f>> NormalImage;
	if (Options.bBakeNormalMap)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_NormalMapBake);

		// no infill on normal map for now, doesn't make sense to do after mapping to tangent space!
		//  (should we build baked normal map in world space, and then resample to tangent space??)
		FVector4f DefaultNormalValue(0, 0, 1, 1);
		FMeshGenericWorldPositionNormalBaker NormalMapBaker;
		NormalMapBaker.SetCache(&TempBakeCache);
		NormalMapBaker.BaseMeshTangents = MeshTangents;
		NormalMapBaker.NormalSampleFunction = [&](FVector3d Position, FVector3d Normal) {
			FSceneCapturePhotoSet::FSceneSample Sample = DefaultSample;
			SceneCapture->ComputeSample(FRenderCaptureTypeFlags::WorldNormal(),
				Position, Normal, VisibilityFunction, Sample);
			FVector3f NormalColor = Sample.WorldNormal;
			float x = (NormalColor.X - 0.5f) * 2.0f;
			float y = (NormalColor.Y - 0.5f) * 2.0f;
			float z = (NormalColor.Z - 0.5f) * 2.0f;
			return FVector3f(x, y, z);
		};

		NormalMapBaker.Bake();
		NormalImage = NormalMapBaker.TakeResult();

		int32 NormalMapDownsampleSteps = GetNumDownsampleSteps(Options.CustomTextureSizeNormalMap);
		if (NormalMapDownsampleSteps > 1)
		{
			TImageBuilder<FVector3f> Downsampled = NormalImage->FastDownsample(NormalMapDownsampleSteps, FVector3f::Zero(), [](FVector3f V, int N) { return V / (float)N; });
			*NormalImage = MoveTemp(Downsampled);
		}
	}

	// build textures
	Progress.EnterProgressFrame(1.f, LOCTEXT("BuildingTextures", "Building Textures..."));
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Textures_BuildTextures);

		FScopedSlowTask BuildTexProgress(6.f, LOCTEXT("BuildingTextures", "Building Textures..."));
		BuildTexProgress.MakeDialog(true);
		if (Options.bBakeBaseColor && ColorImage.IsValid())
		{
			BuildTexProgress.EnterProgressFrame(1.f);
			GeneratedTextures.BaseColorMap = FTexture2DBuilder::BuildTextureFromImage(*ColorImage, FTexture2DBuilder::ETextureType::Color, true, false);
		}
		if (Options.bBakeEmissive && EmissiveImage.IsValid())
		{
			BuildTexProgress.EnterProgressFrame(1.f);
			GeneratedTextures.EmissiveMap = FTexture2DBuilder::BuildTextureFromImage(*EmissiveImage, FTexture2DBuilder::ETextureType::EmissiveHDR, false, false);
			GeneratedTextures.EmissiveMap->CompressionSettings = TC_HDR_Compressed;
		}
		if (Options.bBakeNormalMap && NormalImage.IsValid())
		{
			BuildTexProgress.EnterProgressFrame(1.f);
			GeneratedTextures.NormalMap = FTexture2DBuilder::BuildTextureFromImage(*NormalImage, FTexture2DBuilder::ETextureType::NormalMap, false, false);
		}

		if ( (bRoughness || bMetallic || bSpecular) && PackedMRSImage.IsValid())
		{
			BuildTexProgress.EnterProgressFrame(1.f);
			GeneratedTextures.PackedMRSMap = FTexture2DBuilder::BuildTextureFromImage(*PackedMRSImage, FTexture2DBuilder::ETextureType::ColorLinear, false, false);

		}
		else
		{ 
			if (bRoughness && RoughnessImage.IsValid())
			{
				BuildTexProgress.EnterProgressFrame(1.f);
				GeneratedTextures.RoughnessMap = FTexture2DBuilder::BuildTextureFromImage(*RoughnessImage, FTexture2DBuilder::ETextureType::Roughness, false, false);
			}
			if (bMetallic && MetallicImage.IsValid())
			{
				BuildTexProgress.EnterProgressFrame(1.f);
				GeneratedTextures.MetallicMap = FTexture2DBuilder::BuildTextureFromImage(*MetallicImage, FTexture2DBuilder::ETextureType::Metallic, false, false);
			}
			if (bSpecular && SpecularImage.IsValid())
			{
				BuildTexProgress.EnterProgressFrame(1.f);
				GeneratedTextures.SpecularMap = FTexture2DBuilder::BuildTextureFromImage(*SpecularImage, FTexture2DBuilder::ETextureType::Specular, false, false);
			}
		}
	}
}

struct FApproximationMeshData
{
	IGeometryProcessing_ApproximateActors::EResultCode ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::UnknownError;

	bool bHaveMesh = false;
	FDynamicMesh3 Mesh;

	bool bHaveTangents = false;
	FMeshTangentsd Tangents;
};


static TSharedPtr<FApproximationMeshData> GenerateApproximationMesh(
	TUniquePtr<FMeshSceneAdapter> Scene,
	const IGeometryProcessing_ApproximateActors::FOptions& Options,
	double ApproxAccuracy
)
{
	FScopedSlowTask Progress(8.f, LOCTEXT("Generating Mesh", "Generating Mesh.."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Collect Seed Points"));

	TSharedPtr<FApproximationMeshData> Result = MakeShared<FApproximationMeshData>();

	// collect seed points
	TArray<FVector3d> SeedPoints;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_SeedPoints);
		Scene->CollectMeshSeedPoints(SeedPoints);
	}
	FAxisAlignedBox3d SceneBounds = Scene->GetBoundingBox();

	// calculate a voxel size based on target world-space approximation accuracy
	float WorldBoundsSize = SceneBounds.DiagonalLength();
	int32 VoxelDimTarget = (int)(WorldBoundsSize / ApproxAccuracy) + 1;
	if (VoxelDimTarget < 64)
	{
		VoxelDimTarget = 64;		// use a sane minimum in case the parameter is super-wrong
	}

	// avoid insane memory usage
	if (VoxelDimTarget > Options.ClampVoxelDimension)
	{
		UE_LOGF(LogApproximateActors, Log, "very large voxel size %d clamped to %d", VoxelDimTarget, Options.ClampVoxelDimension);
		VoxelDimTarget = Options.ClampVoxelDimension;
	}

	// make ground plane
	FVector3d GroundPlaneOrigin = FVector3d::ZeroVector;
	FPlane3d GroundClipPlane;
	bool bHaveGroundClipPlane = false;
	if (Options.GroundPlanePolicy == IGeometryProcessing_ApproximateActors::EGroundPlanePolicy::FixedZHeightGroundPlane)
	{
		GroundPlaneOrigin = FVector3d(SceneBounds.Center().X, SceneBounds.Center().Y, Options.GroundPlaneZHeight);
		GroundClipPlane = FPlane3d(FVector3d::UnitZ(), GroundPlaneOrigin);
		bHaveGroundClipPlane = true;
	}

	Progress.EnterProgressFrame(1.f, LOCTEXT("SolidifyMesh", "Approximating Mesh..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Solidify"));

	FDynamicMesh3 SolidMesh;
	{
		// Do in local scope so that memory allocated in Solidify is released after SolidMesh is available
		// TODO: SolidMesh could be replaced with a FColliderMesh if TImplicitMorphology would allow it

		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Solidify);
		FWindingNumberBasedSolidify Solidify(
			[&Scene](const FVector3d& Position) { return Scene->FastWindingNumber(Position, true); },
			SceneBounds, SeedPoints);
		Solidify.SetCellSizeAndExtendBounds(SceneBounds, 2.0 * ApproxAccuracy, VoxelDimTarget);
		Solidify.WindingThreshold = Options.WindingThreshold;

		SolidMesh = FDynamicMesh3(&Solidify.Generate());
		SolidMesh.DiscardAttributes();
	}

	// if solid mesh has no triangles, something has gone wrong
	if (SolidMesh.TriangleCount() == 0)
	{
		UE_LOGF(LogApproximateActors, Warning, "Solidify mesh has no triangles - unable to generate mesh");
		Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::MeshGenerationFailed;
		return Result;
	}

	// CurResultMesh will point to the "current" result and we will update this pointer as we
	// step through various stages of the generation process
	FDynamicMesh3* CurResultMesh = &SolidMesh;		

	if (Options.bVerbose)
	{
		UE_LOGF(LogApproximateActors, Log, "Solidify mesh has %d triangles", CurResultMesh->TriangleCount());
	}

	// we are done w/ the FMeshSceneAdapter now and can free it's memory
	Scene.Reset();
	SeedPoints.Empty();

	Progress.EnterProgressFrame(1.f, LOCTEXT("ClosingMesh", "Topological Operations..."));

	// do topological closure to fix small gaps/etc
	FDynamicMesh3 MorphologyMesh;
	if (Options.bApplyMorphology)
	{
		TRACE_BOOKMARK(TEXT("ApproximateActors-Morphology"));

		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Morphology);
		double MorphologyDistance = Options.MorphologyDistanceMeters * 100.0;		// convert to cm
		FAxisAlignedBox3d MorphologyBounds = CurResultMesh->GetBounds();
		FDynamicMeshAABBTree3 MorphologyBVTree(CurResultMesh);
		TImplicitMorphology<FDynamicMesh3> ImplicitMorphology;
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Close;
		ImplicitMorphology.Source = CurResultMesh;
		ImplicitMorphology.SourceSpatial = &MorphologyBVTree;
		ImplicitMorphology.SetCellSizesAndDistance(MorphologyBounds, MorphologyDistance, VoxelDimTarget, VoxelDimTarget);
		MorphologyMesh = FDynamicMesh3(&ImplicitMorphology.Generate());
		MorphologyMesh.DiscardAttributes();

		CurResultMesh = &MorphologyMesh;
		SolidMesh = FDynamicMesh3();		// we are done with SolidMesh now, release it's memory

		if (Options.bVerbose)
		{
			UE_LOGF(LogApproximateActors, Log, "Morphology mesh has %d triangles", CurResultMesh->TriangleCount());
		}
	}

	// TODO: try doing base clipping here to speed up simplification? slight risk of introducing border issues...

	// if mesh has no triangles, something has gone wrong
	if (CurResultMesh == nullptr || CurResultMesh->TriangleCount() == 0)
	{
		Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::MeshGenerationFailed;
		return Result;
	}

	Progress.EnterProgressFrame(1.f, LOCTEXT("SimplifyingMesh", "Simplifying Mesh..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Simplify"));

	FVolPresMeshSimplification Simplifier(CurResultMesh);
	Simplifier.ProjectionMode = FVolPresMeshSimplification::ETargetProjectionMode::NoProjection;
	Simplifier.DEBUG_CHECK_LEVEL = 0;
	Simplifier.bAllowSeamCollapse = false;

	int32 BaseTargeTriCount = Options.FixedTriangleCount;
	{
		int32 BeforeCount = CurResultMesh->TriangleCount();

		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification);
		if (Options.MeshSimplificationPolicy == IGeometryProcessing_ApproximateActors::ESimplificationPolicy::TrianglesPerUnitSqMeter)
		{
			FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*CurResultMesh);
			double MeshAreaMeterSqr = VolArea.Y * 0.0001;
			int32 AreaBaseTargetTriCount = MeshAreaMeterSqr * Options.SimplificationTargetMetric;

			// do initial fast-collapse pass if enabled
			if (Options.bEnableFastSimplifyPrePass && CurResultMesh->TriangleCount() > 10 * AreaBaseTargetTriCount)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_PrePass);
				Simplifier.FastCollapsePass(ApproxAccuracy, 10, false, 5 * AreaBaseTargetTriCount);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_Pass1);
				Simplifier.SimplifyToTriangleCount(AreaBaseTargetTriCount);
			}
		}
		else if (Options.MeshSimplificationPolicy == IGeometryProcessing_ApproximateActors::ESimplificationPolicy::GeometricTolerance)
		{
			double UseTargetTolerance = Options.SimplificationTargetMetric * 100.0;		// convert to cm (UE Units)

			// do initial fast-collapse pass if enabled
			if (Options.bEnableFastSimplifyPrePass && CurResultMesh->TriangleCount() > 1000000)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_PrePass);
				Simplifier.FastCollapsePass(ApproxAccuracy, 10, false, 1000000);
			}

			// simplify down to a reasonable tri count, as geometric metric is (relatively) expensive
			// (still, this is all incredibly cheap compared to the cost of the rest of this method in practice)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_Pass1);
				Simplifier.SimplifyToTriangleCount(50000);
			}

			// make copy of mesh geometry to use for geometric error measurement
			FColliderMesh Collider;
			Collider.Initialize(*CurResultMesh);
			FColliderMeshProjectionTarget ColliderTarget(&Collider);
			Simplifier.SetProjectionTarget(&ColliderTarget);

			Simplifier.GeometricErrorConstraint = FVolPresMeshSimplification::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = UseTargetTolerance;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_Pass2);
				Simplifier.SimplifyToTriangleCount(8);
			}
		}
		else
		{
			// do initial fast-collapse pass if enabled
			int32 FastCollapseThresh = FMath::Max(1000000, 2 * BaseTargeTriCount);
			if (Options.bEnableFastSimplifyPrePass && CurResultMesh->TriangleCount() > FastCollapseThresh)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Simplification_PrePass);
				Simplifier.FastCollapsePass(ApproxAccuracy, 10, false, FastCollapseThresh);
			}

			Simplifier.SimplifyToTriangleCount(BaseTargeTriCount);
		}

		int32 AfterCount = CurResultMesh->TriangleCount();
		if (Options.bVerbose)
		{
			UE_LOGF(LogApproximateActors, Log, "Simplified mesh from %d to %d triangles", BeforeCount, AfterCount);
		}
	}


	Progress.EnterProgressFrame(1.f, LOCTEXT("RemoveHidden", "Removing Hidden Geometry..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Remove Hidden"));

	if (Options.OcclusionPolicy == IGeometryProcessing_ApproximateActors::EOcclusionPolicy::VisibilityBased)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Occlusion);
		TRemoveOccludedTriangles<FDynamicMesh3> Remover(CurResultMesh);
		Remover.InsideMode = EOcclusionCalculationMode::SimpleOcclusionTest;
		Remover.TriangleSamplingMethod = EOcclusionTriangleSampling::VerticesAndCentroids;
		Remover.AddTriangleSamples = 50;
		Remover.AddRandomRays = 50;
		FDynamicMeshAABBTree3 CurResultMeshSpatial(CurResultMesh, false);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Occlusion_Spatial);
			CurResultMeshSpatial.Build();
		}

		TArray<FTransformSRT3d> NoTransforms;
		NoTransforms.Add(FTransformSRT3d::Identity());
		TArray<FDynamicMeshAABBTree3*> Spatials;
		Spatials.Add(&CurResultMeshSpatial);

		FAxisAlignedBox3d Bounds = CurResultMesh->GetBounds();

		FDynamicMesh3 BasePlaneOccluderMesh;
		FDynamicMeshAABBTree3 BasePlaneOccluderSpatial;
		if (Options.bAddDownwardFacesOccluder)
		{
			FRectangleMeshGenerator RectGen;
			RectGen.Origin = Bounds.Center();
			RectGen.Origin.Z = Bounds.Min.Z - 1.0;
			RectGen.Normal = FVector3f::UnitZ();
			RectGen.Width = RectGen.Height = 10.0 * Bounds.MaxDim();
			BasePlaneOccluderMesh.Copy(&RectGen.Generate());
			BasePlaneOccluderSpatial.SetMesh(&BasePlaneOccluderMesh, true);
			NoTransforms.Add(FTransformSRT3d::Identity());
			Spatials.Add(&BasePlaneOccluderSpatial);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Occlusion_Compute);
			Remover.Select(NoTransforms, Spatials, {}, NoTransforms);
		}
		int32 NumRemoved = 0;
		if (Remover.RemovedT.Num() > 0)
		{
			FMeshFaceSelection Selection(CurResultMesh);
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Occlusion_Clean);
				Selection.Select(Remover.RemovedT);
				Selection.ExpandToOneRingNeighbours(1);
				Selection.ContractBorderByOneRingNeighbours(2);

				// select any tris w/ all verts below clip plane
				if (Options.GroundPlaneClippingPolicy == IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::DiscardFullyHiddenFaces)
				{
					if (bHaveGroundClipPlane)
					{
						for (int32 tid : CurResultMesh->TriangleIndicesItr())
						{
							FVector3d A, B, C;
							CurResultMesh->GetTriVertices(tid, A, B, C);
							if (GroundClipPlane.WhichSide(A) <= 0 && GroundClipPlane.WhichSide(B) <= 0 && GroundClipPlane.WhichSide(C) <= 0)
							{
								Selection.Select(tid);
							}
						}
					}
					else
					{
						UE_LOGF(LogApproximateActors, Warning, "DiscardFullyHiddenFaces Ground Plane Clipping Policy ignored because no Ground Clip Plane is set");
					}
				}
			}
			FDynamicMeshEditor Editor(CurResultMesh);
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Occlusion_Delete);
				TArray SelectionArray(Selection.AsArray());
				NumRemoved = SelectionArray.Num();
				Editor.RemoveTriangles(SelectionArray, true);
			}
		}

		if (Options.bVerbose)
		{
			UE_LOGF(LogApproximateActors, Log, "Occlusion-Filtered mesh has %d triangles (removed %d)", CurResultMesh->TriangleCount(), NumRemoved);
		}
	}

	// if occlusion removed all the triangles, fail
	if (CurResultMesh->TriangleCount() == 0)
	{
		Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::MeshGenerationFailed;
		return Result;
	}

	TRACE_BOOKMARK(TEXT("ApproximateActors-Clip Ground"));

	if (Options.GroundPlaneClippingPolicy == IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::CutFaces ||
		Options.GroundPlaneClippingPolicy == IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::CutFacesAndFill)
	{
		if (bHaveGroundClipPlane)
		{
			FMeshPlaneCut PlaneCut(CurResultMesh, GroundPlaneOrigin, -GroundClipPlane.Normal);
			PlaneCut.Cut();
			if (Options.GroundPlaneClippingPolicy == IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::CutFacesAndFill)
			{
				PlaneCut.HoleFill(ConstrainedDelaunayTriangulate<double>, true);
			}
		}
		else
		{
			UE_LOGF(LogApproximateActors, Warning, "Ground Plane Cut/Fill Policy ignored because no Ground Clip Plane is set");
		}
	}

	// if clipping removed all the triangles, fail
	if (CurResultMesh->TriangleCount() == 0)
	{
		Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::MeshGenerationFailed;
		return Result;
	}

	TRACE_BOOKMARK(TEXT("ApproximateActors-Normals and UVs"));

	// re-enable attributes
	CurResultMesh->EnableAttributes();

	//  TODO: clip hidden triangles against occluder geo like landscape

	// compute normals
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Normals);
		if (Options.bCalculateHardNormals)
		{
			FMeshNormals::InitializeOverlayTopologyFromOpeningAngle(CurResultMesh, CurResultMesh->Attributes()->PrimaryNormals(), Options.HardNormalsAngleDeg);
			FMeshNormals::QuickRecomputeOverlayNormals(*CurResultMesh);
		}
		else
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(CurResultMesh->Attributes()->PrimaryNormals());
		}
	}

	// exit here if we are just generating a merged collision mesh
	if (Options.BasePolicy == IGeometryProcessing_ApproximateActors::EApproximationPolicy::CollisionMesh)
	{
		Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::Success;
		Result->bHaveMesh = true;
		Result->Mesh = MoveTemp(*CurResultMesh);
		return Result;
	}

	Progress.EnterProgressFrame(1.f, LOCTEXT("ComputingUVs", "Computing UVs..."));

	// compute UVs
	bool bHaveValidUVs = true;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UVInputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	// PatchBuilder AutoUV currently requires compact input mesh. TODO: fix that, then we can just re-use CurResultMesh here.
	UVInputMesh->CompactCopy(*CurResultMesh);
	//*UVInputMesh = MoveTemp(*CurResultMesh);
	FParameterizeMeshOp ParameterizeMeshOp;
	ParameterizeMeshOp.Stretch = Options.UVAtlasStretchTarget;
	
	// UVAtlas parameters
	ParameterizeMeshOp.NumCharts = 0;
	
	// PatchBuilder generation parameters
	ParameterizeMeshOp.InitialPatchCount = Options.PatchBuilderInitialPatchCount;
	ParameterizeMeshOp.bRespectInputGroups = false;
	ParameterizeMeshOp.PatchCurvatureAlignmentWeight = Options.PatchBuilderCurvatureAlignment;
	ParameterizeMeshOp.PatchMergingMetricThresh = Options.PatchBuilderMergingThreshold;
	ParameterizeMeshOp.PatchMergingAngleThresh = Options.PatchBuilderMaxNormalDeviationDeg;
	ParameterizeMeshOp.ExpMapNormalSmoothingSteps = 5;
	ParameterizeMeshOp.ExpMapNormalSmoothingAlpha = 0.25;

	ParameterizeMeshOp.InputMesh = UVInputMesh;
	ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::XAtlas;
	if (Options.UVPolicy == IGeometryProcessing_ApproximateActors::EUVGenerationPolicy::PreferUVAtlas)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::UVAtlas;
	}
	else if (Options.UVPolicy == IGeometryProcessing_ApproximateActors::EUVGenerationPolicy::PreferPatchBuilder)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::PatchBuilder;
	}


	FProgressCancel UVProgressCancel;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_GenerateUVs);
		ParameterizeMeshOp.CalculateResult(&UVProgressCancel);
	}

	TUniquePtr<FDynamicMesh3> FinalMesh;

	FGeometryResult UVResultInfo = ParameterizeMeshOp.GetResultInfo();
	if (! UVResultInfo.HasResult())
	{
		UE_LOGF(LogApproximateActors, Warning, "UV Auto-Generation Failed for target path %ls", *Options.BasePackagePath);
		bHaveValidUVs = false;
		FinalMesh = MakeUnique<FDynamicMesh3>(MoveTemp(*UVInputMesh));
	}
	else
	{
		FinalMesh = ParameterizeMeshOp.ExtractResult();
	}

	// if UVs failed, fall back to box projection
	if (!bHaveValidUVs)
	{
		FDynamicMeshUVEditor UVEditor(FinalMesh.Get(), 0, true);
		TArray<int32> AllTriangles;
		for (int32 tid : FinalMesh->TriangleIndicesItr())
		{
			AllTriangles.Add(tid);
		}
		UVEditor.SetTriangleUVsFromBoxProjection(AllTriangles, [&](const FVector3d& P) { return P; },
												 FFrame3d(FinalMesh->GetBounds().Center()), FVector3d::One());
		bHaveValidUVs = true;
	}


	Progress.EnterProgressFrame(1.f, LOCTEXT("PackingUVs", "Packing UVs..."));

	// repack UVs
	if (bHaveValidUVs)
	{
		FDynamicMeshUVOverlay* RepackUVLayer = FinalMesh->Attributes()->PrimaryUV();
		RepackUVLayer->SplitBowties();
		FDynamicMeshUVPacker Packer(RepackUVLayer);
		Packer.TextureResolution = Options.TextureImageSize / 4;		// maybe too conservative? We don't have gutter control currently.
		Packer.GutterSize = 1.0;		// not clear this works
		Packer.bAllowFlips = false;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_PackUVs);
			bool bPackingOK = Packer.StandardPack();
			if (!bPackingOK)
			{
				UE_LOGF(LogApproximateActors, Warning, "UV Packing Failed for target path %ls", *Options.BasePackagePath);
			}
		}
	}

	Progress.EnterProgressFrame(1.f, LOCTEXT("ComputingTangents", "Computing Tangents..."));

	Result->ResultCode = IGeometryProcessing_ApproximateActors::EResultCode::Success;
	Result->bHaveMesh = true;
	Result->Mesh = MoveTemp(*FinalMesh);

	// compute tangents
	Result->bHaveTangents = true;
	Result->Tangents.SetMesh(&Result->Mesh);
	FComputeTangentsOptions TangentsOptions;
	TangentsOptions.bAveraged = true;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Tangents);
		Result->Tangents.ComputeTriVertexTangents(
			Result->Mesh.Attributes()->PrimaryNormals(),
			Result->Mesh.Attributes()->PrimaryUV(),
			TangentsOptions);
	}

	return Result;
}

IGeometryProcessing_ApproximateActors::FOptions FApproximateActorsImpl::ConstructOptions(const FMeshApproximationSettings& UseSettings)
{
	//
	// Construct options for ApproximateActors operation
	//
	FOptions Options;

	Options.BasePolicy = (UseSettings.OutputType == EMeshApproximationType::MeshShapeOnly) ?
		IGeometryProcessing_ApproximateActors::EApproximationPolicy::CollisionMesh :
		IGeometryProcessing_ApproximateActors::EApproximationPolicy::MeshAndGeneratedMaterial;
	
	Options.MeshDataLODPolicy = (UseSettings.bUseRenderLODMeshes) ?
		EMeshDataSourceLODPolicy::LOD0RenderMeshes : EMeshDataSourceLODPolicy::LOD0SourceMeshes;
	
	Options.WorldSpaceApproximationAccuracyMeters = UseSettings.ApproximationAccuracy;

	Options.bAutoThickenThinParts = UseSettings.bAttemptAutoThickening;
	Options.AutoThickenThicknessMeters = UseSettings.TargetMinThicknessMultiplier * UseSettings.ApproximationAccuracy;
	Options.bIgnoreTinyParts = UseSettings.bIgnoreTinyParts;
	Options.TinyPartMaxDimensionMeters = UseSettings.TinyPartSizeMultiplier * UseSettings.ApproximationAccuracy;

	Options.BaseCappingPolicy = IGeometryProcessing_ApproximateActors::EBaseCappingPolicy::NoBaseCapping;
	if (UseSettings.BaseCapping == EMeshApproximationBaseCappingType::ConvexPolygon)
	{
		Options.BaseCappingPolicy = IGeometryProcessing_ApproximateActors::EBaseCappingPolicy::ConvexPolygon;
	}
	else if (UseSettings.BaseCapping == EMeshApproximationBaseCappingType::ConvexSolid)
	{
		Options.BaseCappingPolicy = IGeometryProcessing_ApproximateActors::EBaseCappingPolicy::ConvexSolid;
	}

	Options.ClampVoxelDimension = UseSettings.ClampVoxelDimension;
	Options.WindingThreshold = UseSettings.WindingThreshold;
	Options.bApplyMorphology = UseSettings.bFillGaps;
	Options.MorphologyDistanceMeters = UseSettings.GapDistance;

	if (UseSettings.GroundClipping == EMeshApproximationGroundPlaneClippingPolicy::NoGroundClipping)
	{
		Options.GroundPlanePolicy = EGroundPlanePolicy::NoGroundPlane;
		Options.GroundPlaneClippingPolicy = IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::NoClipping;
	}
	else if (UseSettings.GroundClipping == EMeshApproximationGroundPlaneClippingPolicy::DiscardWithZPlane)
	{
		Options.GroundPlanePolicy = EGroundPlanePolicy::FixedZHeightGroundPlane;
		Options.GroundPlaneZHeight = UseSettings.GroundClippingZHeight;
		Options.GroundPlaneClippingPolicy = IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::DiscardFullyHiddenFaces;
	}
	else if (UseSettings.GroundClipping == EMeshApproximationGroundPlaneClippingPolicy::CutWithZPlane)
	{
		Options.GroundPlanePolicy = EGroundPlanePolicy::FixedZHeightGroundPlane;
		Options.GroundPlaneZHeight = UseSettings.GroundClippingZHeight;
		Options.GroundPlaneClippingPolicy = IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::CutFaces;
	}
	else if (UseSettings.GroundClipping == EMeshApproximationGroundPlaneClippingPolicy::CutAndFillWithZPlane)
	{
		Options.GroundPlanePolicy = EGroundPlanePolicy::FixedZHeightGroundPlane;
		Options.GroundPlaneZHeight = UseSettings.GroundClippingZHeight;
		Options.GroundPlaneClippingPolicy = IGeometryProcessing_ApproximateActors::EGroundPlaneClippingPolicy::CutFacesAndFill;
	}



	Options.OcclusionPolicy = (UseSettings.OcclusionMethod == EOccludedGeometryFilteringPolicy::VisibilityBasedFiltering) ?
		IGeometryProcessing_ApproximateActors::EOcclusionPolicy::VisibilityBased : IGeometryProcessing_ApproximateActors::EOcclusionPolicy::None;
	Options.bAddDownwardFacesOccluder = UseSettings.bOccludeFromBottom;

	Options.FixedTriangleCount = UseSettings.TargetTriCount;
	if (UseSettings.SimplifyMethod == EMeshApproximationSimplificationPolicy::TrianglesPerArea)
	{
		Options.MeshSimplificationPolicy = IGeometryProcessing_ApproximateActors::ESimplificationPolicy::TrianglesPerUnitSqMeter;
		Options.SimplificationTargetMetric = UseSettings.TrianglesPerM;
	}
	else if (UseSettings.SimplifyMethod == EMeshApproximationSimplificationPolicy::GeometricTolerance)
	{
		Options.MeshSimplificationPolicy = IGeometryProcessing_ApproximateActors::ESimplificationPolicy::GeometricTolerance;
		Options.SimplificationTargetMetric = UseSettings.GeometricDeviation;
	}
	else
	{
		Options.MeshSimplificationPolicy = IGeometryProcessing_ApproximateActors::ESimplificationPolicy::FixedTriangleCount;
	}
	Options.bEnableFastSimplifyPrePass = UseSettings.bEnableSimplifyPrePass;

	Options.UVPolicy = IGeometryProcessing_ApproximateActors::EUVGenerationPolicy::PreferXAtlas;
	if (UseSettings.UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferUVAtlas)
	{
		Options.UVPolicy = IGeometryProcessing_ApproximateActors::EUVGenerationPolicy::PreferUVAtlas;
	}
	else if (UseSettings.UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferPatchBuilder)
	{
		Options.UVPolicy = IGeometryProcessing_ApproximateActors::EUVGenerationPolicy::PreferPatchBuilder;
	}
	Options.PatchBuilderInitialPatchCount =  FMath::Clamp(UseSettings.InitialPatchCount, 1, 99999);
	Options.PatchBuilderCurvatureAlignment = FMath::Clamp(UseSettings.CurvatureAlignment, 0.001, 1000.0);
	Options.PatchBuilderMergingThreshold = FMath::Clamp(UseSettings.MergingThreshold, 1.0, 9999.0);
	Options.PatchBuilderMaxNormalDeviationDeg = FMath::Clamp(UseSettings.MaxAngleDeviation, 0.0, 180);

	Options.bCalculateHardNormals = UseSettings.bEstimateHardNormals;
	Options.HardNormalsAngleDeg = FMath::Clamp(UseSettings.HardNormalAngle, 0.001f, 89.99f);

	Options.TextureImageSize = UseSettings.MaterialSettings.TextureSize.X;
	
	if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_UseSingleTextureSize)
	{
		Options.TextureSizePolicy = ETextureSizePolicy::TextureSize;
	}
	else if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_UseAutomaticBiasedSizes)
	{
		Options.TextureSizePolicy = ETextureSizePolicy::CustomTextureSize;

		int32 NormalSize = Options.TextureImageSize;
		int32 BaseColorSize = FMath::Max(Options.TextureImageSize >> 1, 32);
		int32 PropertiesSize = FMath::Max(Options.TextureImageSize >> 2, 16);

		Options.CustomTextureSizeBaseColor = BaseColorSize;
		Options.CustomTextureSizeRoughness = PropertiesSize;
		Options.CustomTextureSizeMetallic = PropertiesSize;
		Options.CustomTextureSizeSpecular = PropertiesSize;
		Options.CustomTextureSizeEmissive = PropertiesSize;
		Options.CustomTextureSizeNormalMap = NormalSize;
	}
	else if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_UseManualOverrideTextureSize)
	{
		Options.TextureSizePolicy = ETextureSizePolicy::CustomTextureSize;

		Options.CustomTextureSizeBaseColor = UseSettings.MaterialSettings.DiffuseTextureSize.X;
		Options.CustomTextureSizeRoughness = UseSettings.MaterialSettings.RoughnessTextureSize.X;
		Options.CustomTextureSizeMetallic = UseSettings.MaterialSettings.MetallicTextureSize.X;
		Options.CustomTextureSizeSpecular = UseSettings.MaterialSettings.SpecularTextureSize.X;
		Options.CustomTextureSizeEmissive = UseSettings.MaterialSettings.EmissiveTextureSize.X;
		Options.CustomTextureSizeNormalMap = UseSettings.MaterialSettings.NormalTextureSize.X;
	}

	Options.AntiAliasMultiSampling = FMath::Max(1, UseSettings.MultiSamplingAA);

	Options.RenderCaptureImageSize = (UseSettings.RenderCaptureResolution == 0) ?
		Options.TextureImageSize : UseSettings.RenderCaptureResolution;
	Options.FieldOfViewDegrees = UseSettings.CaptureFieldOfView;
	Options.NearPlaneDist = UseSettings.NearPlaneDist;

	Options.bMaximizeBakeParallelism = UseSettings.bEnableParallelBaking;

	Options.bVerbose = UseSettings.bPrintDebugMessages;
	Options.bWriteDebugMesh = UseSettings.bEmitFullDebugMesh;

	// Nanite settings
	Options.bGenerateNaniteEnabledMesh = UseSettings.bGenerateNaniteEnabledMesh;
	Options.NaniteFallbackTarget = UseSettings.NaniteFallbackTarget == ::ENaniteFallbackTarget::Auto ? IGeometryProcessing_ApproximateActors::ENaniteFallbackTarget::Auto :
								  (UseSettings.NaniteFallbackTarget == ::ENaniteFallbackTarget::PercentTriangles ? IGeometryProcessing_ApproximateActors::ENaniteFallbackTarget::PercentTriangles : IGeometryProcessing_ApproximateActors::ENaniteFallbackTarget::RelativeError);
	Options.NaniteFallbackPercentTriangles = UseSettings.NaniteFallbackPercentTriangles;
	Options.NaniteFallbackRelativeError = UseSettings.NaniteFallbackRelativeError;

	// Distance field
	Options.bAllowDistanceField = UseSettings.bAllowDistanceField;

	// Ray tracing
	Options.bSupportRayTracing = UseSettings.bSupportRayTracing;

	// Material properties baking
	Options.bBakeBaseColor = true;
	Options.bBakeRoughness = UseSettings.MaterialSettings.bRoughnessMap;
	Options.bBakeMetallic = UseSettings.MaterialSettings.bMetallicMap;
	Options.bBakeSpecular = UseSettings.MaterialSettings.bSpecularMap;
	Options.bBakeEmissive = UseSettings.MaterialSettings.bEmissiveMap;
	Options.bBakeNormalMap = UseSettings.MaterialSettings.bNormalMap;

	return Options;
}


void FApproximateActorsImpl::ApproximateActors(const FInput& Input, const FOptions& Options, FResults& ResultsOut)
{
	int32 ActorClusters = 1;
	FScopedSlowTask Progress(1.f, LOCTEXT("ApproximatingActors", "Generating Actor Approximation..."));
	Progress.MakeDialog(true);
	Progress.EnterProgressFrame(1.f);
	GenerateApproximationForActorSet(Input, Options, ResultsOut);
}




void FApproximateActorsImpl::GenerateApproximationForActorSet(const FInput& Input, const FOptions& Options, FResults& ResultsOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate);

	RenderCaptureInterface::FScopedCapture RenderCapture(CVarApproximateActorsRDOCCapture.GetValueOnAnyThread() == 1, TEXT("ApproximateActors"));

	if (Options.BasePolicy == IGeometryProcessing_ApproximateActors::EApproximationPolicy::MeshAndGeneratedMaterial)
	{
		// The scene capture photoset part of this process relies on debug view modes being available.
		// If it ain't the case, fail immediatelly
		if (!AllowDebugViewmodes())
		{
			UE_LOGF(LogApproximateActors, Error, "Debug view modes not are available - unable to generate material");
			ResultsOut.ResultCode = EResultCode::MaterialGenerationFailed;
			return;
		}
	}

	//
	// extract all visible meshes from the list of Actors and build up a "mesh scene" that represents the
	// assembly of geometry, taking advantage of instancing where possible, but also pulling those meshes
	// apart and processing them if necessary to (eg) thicken them, etc, so that later processing works better.
	// FMeshSceneAdapter does all the heavy lifting, here it is just being configured and built
	//
	FScopedSlowTask Progress(11.f, LOCTEXT("ApproximatingActors", "Generating Actor Approximation..."));
	Progress.EnterProgressFrame(1.f, LOCTEXT("BuildingScene", "Building Scene..."));

	float ApproxAccuracy = Options.WorldSpaceApproximationAccuracyMeters * 100.0;		// convert to cm (UE Units)

	TUniquePtr<FMeshSceneAdapter> Scene = MakeUnique<FMeshSceneAdapter>();
	FMeshSceneAdapterBuildOptions SceneBuildOptions;
	SceneBuildOptions.bIgnoreStaticMeshSourceData = (Options.MeshDataLODPolicy == EMeshDataSourceLODPolicy::LOD0RenderMeshes);		// default is false
	SceneBuildOptions.bThickenThinMeshes = Options.bAutoThickenThinParts;
	SceneBuildOptions.DesiredMinThickness = Options.AutoThickenThicknessMeters * 100.0;		// convert to cm (UE Units)
	// filter out objects smaller than 10% of voxel size
	SceneBuildOptions.bFilterTinyObjects = Options.bIgnoreTinyParts;
	SceneBuildOptions.TinyObjectBoxMaxDimension = Options.TinyPartMaxDimensionMeters;
	SceneBuildOptions.bOnlySurfaceMaterials = true;				// don't include decal geometry in 3D mesh scene (will be included in renderings)
	SceneBuildOptions.bEnableUVQueries = SceneBuildOptions.bEnableNormalsQueries = false;		// not required in this context, will reduce memory usage
	SceneBuildOptions.bPrintDebugMessages = Options.bVerbose;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_BuildScene);
		TRACE_BOOKMARK(TEXT("ApproximateActors-Adding Actors"));
		Scene->AddActors(Input.Actors);
		TRACE_BOOKMARK(TEXT("ApproximateActors-Adding Components"));
		Scene->AddComponents(Input.Components);
	}

	if (!Scene->IsValid())
	{
		UE_LOGF(LogApproximateActors, Warning, "No valid input actors/components - unable to generate mesh");
		ResultsOut.ResultCode = EResultCode::MeshGenerationFailed;
		return;
	}

	{
		TRACE_BOOKMARK(TEXT("ApproximateActors-Building Scene"));
		Scene->Build(SceneBuildOptions);
	}

	if (Options.bVerbose)
	{
		FMeshSceneAdapter::FStatistics Stats;
		Scene->GetGeometryStatistics(Stats);
		UE_LOGF(LogApproximateActors, Log, "%lld triangles in %lld unique meshes, total %lld triangles in %lld instances",
			Stats.UniqueMeshTriangleCount, Stats.UniqueMeshCount, Stats.InstanceMeshTriangleCount, Stats.InstanceMeshCount);
	}

	// add a "base cap" if desired, this helps with (eg) large meshes with no base like a 3D scan of a mountain, etc
	if (Options.BaseCappingPolicy != EBaseCappingPolicy::NoBaseCapping)
	{
		TRACE_BOOKMARK(TEXT("ApproximateActors-Capping"));

		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_Capping);

		double UseThickness = 0.0;
		if (Options.BaseCappingPolicy == EBaseCappingPolicy::ConvexSolid)
		{
			UseThickness = (Options.BaseThicknessOverrideMeters != 0) ? (Options.BaseThicknessOverrideMeters * 100.0) :
						   (Options.bAutoThickenThinParts ? SceneBuildOptions.DesiredMinThickness : 1.25 * ApproxAccuracy);
		}

		double UseHeight = (Options.BaseHeightOverrideMeters != 0) ? (Options.BaseHeightOverrideMeters * 100.0) : (2.0 * ApproxAccuracy);
		Scene->GenerateBaseClosingMesh(UseHeight, UseThickness);
	}

	FDynamicMesh3 DebugMesh;
	FDynamicMesh3* WriteDebugMesh = nullptr;
	if (Options.bWriteDebugMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ApproximateActorsImpl_Generate_DebugMesh);
		DebugMesh.EnableAttributes();
		Scene->GetAccumulatedMesh(DebugMesh);
		FMeshNormals::InitializeMeshToPerTriangleNormals(&DebugMesh);
		WriteDebugMesh = &DebugMesh;
	}

	// build spatial evaluation cache in the FMeshSceneAdapter, necessary for the mesh build below
	TRACE_BOOKMARK(TEXT("ApproximateActors-Building Cache"));
	Scene->BuildSpatialEvaluationCache();

	//
	// Generate a new mesh that approximates the entire scene represented by FMeshSceneAdapter.
	//
	TRACE_BOOKMARK(TEXT("ApproximateActors-Building Approx Mesh"));

	// Pass ownership of the Scene to GenerateApproximationMesh() so that it can delete it as soon
	// as possible (because it is often very large, memory-wise)
	TFuture<TSharedPtr<FApproximationMeshData>> GenerateMeshFuture = Async(EAsyncExecution::ThreadPool, [&Scene, Options, ApproxAccuracy]()
	{
		return GenerateApproximationMesh( MoveTemp(Scene), Options, ApproxAccuracy);
	});
	
	FDynamicMesh3 FinalMesh;
	FMeshTangentsd FinalMeshTangents;
	auto WaitForMeshAvailable = [&]()
	{
		TSharedPtr<FApproximationMeshData> ApproximationMeshData = GenerateMeshFuture.Get();
		ResultsOut.ResultCode = ApproximationMeshData->ResultCode;
		FinalMesh = MoveTemp(ApproximationMeshData->Mesh);
		FinalMeshTangents = MoveTemp(ApproximationMeshData->Tangents);
	};

	// if we are only generating collision mesh, we are done now
	if (Options.BasePolicy == IGeometryProcessing_ApproximateActors::EApproximationPolicy::CollisionMesh)
	{
		WaitForMeshAvailable();
		if (ResultsOut.ResultCode == EResultCode::Success)
		{
			EmitGeneratedMeshAsset(Options, ResultsOut, &FinalMesh, nullptr, WriteDebugMesh);
		}
		return;
	}

	// if parallel capture is not allowed, force mesh computation to finish now
	if (Options.bMaximizeBakeParallelism == false)
	{
		WaitForMeshAvailable();
		if (ResultsOut.ResultCode != EResultCode::Success)
		{
			return;
		}
	}

	//
	// create a set of spatially located render captures of the scene ("photo set").
	//
	Progress.EnterProgressFrame(1.f, LOCTEXT("CapturingScene", "Capturing Scene..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Capture Photos"));

	TUniquePtr<FSceneCapturePhotoSet> SceneCapture = CapturePhotoSet(Input, Options);

	// if parallel capture was allowed, need to force the mesh compute to finish now to be able to proceed
	if (Options.bMaximizeBakeParallelism == true)
	{
		WaitForMeshAvailable();
		if (ResultsOut.ResultCode != EResultCode::Success)
		{
			return;
		}
	}

	//
	// bake textures onto the generated approximation mesh by projecting/sampling
	// the set of captured photos
	//
	Progress.EnterProgressFrame(1.f, LOCTEXT("BakingTextures", "Baking Textures..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Bake Textures"));

	FOptions OverridenOptions = Options;

	// evaluate required texture size if needed
	if (OverridenOptions.TextureSizePolicy == ETextureSizePolicy::TexelDensity)
	{
		OverridenOptions.TextureSizePolicy = ETextureSizePolicy::TextureSize;
		OverridenOptions.TextureImageSize = FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(FinalMesh, OverridenOptions.MeshTexelDensity);
	}

	if (OverridenOptions.TextureSizePolicy == ETextureSizePolicy::TextureSize)
	{
		// Assign the same texture size to all properties
		OverridenOptions.TextureImageSize = FMath::RoundUpToPowerOfTwo(OverridenOptions.TextureImageSize);
		OverridenOptions.CustomTextureSizeBaseColor = OverridenOptions.TextureImageSize;
		OverridenOptions.CustomTextureSizeRoughness = OverridenOptions.TextureImageSize;
		OverridenOptions.CustomTextureSizeMetallic = OverridenOptions.TextureImageSize;
		OverridenOptions.CustomTextureSizeSpecular = OverridenOptions.TextureImageSize;
		OverridenOptions.CustomTextureSizeEmissive = OverridenOptions.TextureImageSize;
		OverridenOptions.CustomTextureSizeNormalMap = OverridenOptions.TextureImageSize;
	}
	else
	{
		OverridenOptions.CustomTextureSizeBaseColor = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeBaseColor);
		OverridenOptions.CustomTextureSizeRoughness = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeRoughness);
		OverridenOptions.CustomTextureSizeMetallic = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeMetallic);
		OverridenOptions.CustomTextureSizeSpecular = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeSpecular);
		OverridenOptions.CustomTextureSizeEmissive = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeEmissive);
		OverridenOptions.CustomTextureSizeNormalMap = FMath::RoundUpToPowerOfTwo(OverridenOptions.CustomTextureSizeNormalMap);

		// Store the maximum texture size in TextureImageSize. This will serve as the capture size for all properties in the baking cache.
		// Images will then be downscaled as needed for each property.
		OverridenOptions.TextureImageSize = OverridenOptions.CustomTextureSizeBaseColor;
		OverridenOptions.TextureImageSize = FMath::Max(OverridenOptions.TextureImageSize, OverridenOptions.CustomTextureSizeRoughness);
		OverridenOptions.TextureImageSize = FMath::Max(OverridenOptions.TextureImageSize, OverridenOptions.CustomTextureSizeMetallic);
		OverridenOptions.TextureImageSize = FMath::Max(OverridenOptions.TextureImageSize, OverridenOptions.CustomTextureSizeSpecular);
		OverridenOptions.TextureImageSize = FMath::Max(OverridenOptions.TextureImageSize, OverridenOptions.CustomTextureSizeEmissive);
		OverridenOptions.TextureImageSize = FMath::Max(OverridenOptions.TextureImageSize, OverridenOptions.CustomTextureSizeNormalMap);
	}

	// bake textures for Actor
	FGeneratedResultTextures GeneratedTextures;
	BakeTexturesFromPhotoCapture(SceneCapture, OverridenOptions,
		GeneratedTextures,
		&FinalMesh, &FinalMeshTangents);

	Progress.EnterProgressFrame(1.f, LOCTEXT("Writing Assets", "Writing Assets..."));
	TRACE_BOOKMARK(TEXT("ApproximateActors-Create Material"));

	// Make material for textures by creating MIC of input material, or fall back to known material
	UMaterialInterface* UseBaseMaterial = (Options.BakeMaterial != nullptr) ? 
		Options.BakeMaterial : 
		Options.bUsePackedMRS ?
			LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/FullMaterialBakeOutputMaterial_PackedMRS")) :
			LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/FullMaterialBakeOutputMaterial"));
	FMaterialAssetOptions MatOptions;
	MatOptions.NewAssetPath = Options.BasePackagePath + TEXT("_Material");
	FMaterialAssetResults MatResults;
	ECreateMaterialResult MatResult = UE::AssetUtils::CreateDerivedMaterialInstance(UseBaseMaterial, MatOptions, MatResults);
	UMaterialInstanceConstant* NewMaterial = nullptr;
	if (ensure(MatResult == ECreateMaterialResult::Ok))
	{
		NewMaterial = MatResults.NewMaterialInstance;
		ResultsOut.NewMaterials.Add(NewMaterial);
	}

	// this lambda converts a generated texture to an Asset, and then assigns it to a parameter of the Material
	FString BaseTexturePath = MatOptions.NewAssetPath;
	auto WriteTextureLambda = [BaseTexturePath, NewMaterial, &ResultsOut](
		UTexture2D* Texture, 
		FTexture2DBuilder::ETextureType Type,
		FName MaterialParamName,
		FName SwitchName)
	{
		if (Texture)
		{
			FTexture2DBuilder::CopyPlatformDataToSourceData(Texture, Type);

			if (Type == FTexture2DBuilder::ETextureType::Roughness
				|| Type == FTexture2DBuilder::ETextureType::Metallic
				|| Type == FTexture2DBuilder::ETextureType::Specular)
			{
				UE::AssetUtils::ConvertToSingleChannel(Texture);
			}

			// Make sure the texture is a VT if required by the material sampler
			if (NewMaterial != nullptr)
			{
				UTexture* DefaultTexture = nullptr;
				NewMaterial->GetTextureParameterValue(MaterialParamName, DefaultTexture);
				if (ensure(DefaultTexture))
				{
					Texture->VirtualTextureStreaming = DefaultTexture->VirtualTextureStreaming;
				}
			}

			FTexture2DAssetOptions TexOptions;
			TexOptions.NewAssetPath = BaseTexturePath + TEXT("_") + MaterialParamName.ToString();
			FTexture2DAssetResults Results;
			ECreateTexture2DResult TexResult = UE::AssetUtils::SaveGeneratedTexture2DAsset(Texture, TexOptions, Results);
			if (ensure(TexResult == ECreateTexture2DResult::Ok))
			{
				ResultsOut.NewTextures.Add(Texture);
				if (NewMaterial != nullptr)
				{
					NewMaterial->SetTextureParameterValueEditorOnly(MaterialParamName, Texture);
				}
			}
		}

		if (NewMaterial && SwitchName != NAME_None)
		{
			FGuid SwitchGuid;
			bool bUnusedValue = false;
			if (NewMaterial->GetStaticSwitchParameterValue(SwitchName, bUnusedValue, SwitchGuid))
			{
				const bool bUseSwitchValue = true;
				NewMaterial->SetStaticSwitchParameterValueEditorOnly(SwitchName, bUseSwitchValue);
			}
		}
	};

	TRACE_BOOKMARK(TEXT("ApproximateActors-Write Textures"));

	// process the generated textures
	if (Options.bBakeBaseColor)
	{
		WriteTextureLambda(GeneratedTextures.BaseColorMap, FTexture2DBuilder::ETextureType::Color, Options.BaseColorTexParamName, Options.BaseColorSwitchName);
	}
	if (Options.bBakeEmissive)
	{
		WriteTextureLambda(GeneratedTextures.EmissiveMap, FTexture2DBuilder::ETextureType::EmissiveHDR, Options.EmissiveTexParamName, Options.EmissiveSwitchName);
	}
	if (Options.bBakeNormalMap)
	{
		WriteTextureLambda(GeneratedTextures.NormalMap, FTexture2DBuilder::ETextureType::NormalMap, Options.NormalTexParamName, Options.NormalSwitchName);
	}

	if ((Options.bBakeRoughness || Options.bBakeMetallic || Options.bBakeSpecular) && Options.bUsePackedMRS)
	{
		WriteTextureLambda(GeneratedTextures.PackedMRSMap, FTexture2DBuilder::ETextureType::ColorLinear, Options.PackedMRSTexParamName, Options.PackedMRSSwitchName);
	}
	if (Options.bBakeRoughness)
	{
		WriteTextureLambda(GeneratedTextures.RoughnessMap, FTexture2DBuilder::ETextureType::Roughness, Options.RoughnessTexParamName, Options.RoughnessSwitchName);
	}
	if (Options.bBakeMetallic)
	{
		WriteTextureLambda(GeneratedTextures.MetallicMap, FTexture2DBuilder::ETextureType::Metallic, Options.MetallicTexParamName, Options.MetallicSwitchName);
	}
	if (Options.bBakeSpecular)
	{
		WriteTextureLambda(GeneratedTextures.SpecularMap, FTexture2DBuilder::ETextureType::Specular, Options.SpecularTexParamName, Options.SpecularSwitchName);
	}


	// force material update now that we have updated texture parameters
	// (does this do that? Let calling code do it?)
	if (NewMaterial != nullptr)
	{
		NewMaterial->PostEditChange();
	}

	EmitGeneratedMeshAsset(Options, ResultsOut, &FinalMesh, NewMaterial, WriteDebugMesh);
	ResultsOut.ResultCode = EResultCode::Success;
}


UStaticMesh* FApproximateActorsImpl::EmitGeneratedMeshAsset(
	const FOptions& Options, 
	FResults& ResultsOut,
	FDynamicMesh3* FinalMesh,
	UMaterialInterface* Material,
	FDynamicMesh3* DebugMesh)
{
	FStaticMeshAssetOptions MeshAssetOptions;

	MeshAssetOptions.CollisionType = ECollisionTraceFlag::CTF_UseSimpleAsComplex;
	MeshAssetOptions.bEnableRecomputeTangents = false;

	MeshAssetOptions.NewAssetPath = Options.BasePackagePath;
	MeshAssetOptions.SourceMeshes.DynamicMeshes.Add(FinalMesh);

	MeshAssetOptions.bGenerateNaniteEnabledMesh = Options.bGenerateNaniteEnabledMesh;
	MeshAssetOptions.NaniteSettings.bEnabled = Options.bGenerateNaniteEnabledMesh;
	MeshAssetOptions.NaniteSettings.FallbackTarget = Options.NaniteFallbackTarget == IGeometryProcessing_ApproximateActors::ENaniteFallbackTarget::Auto ? ::ENaniteFallbackTarget::Auto : 
													 (Options.NaniteFallbackTarget == IGeometryProcessing_ApproximateActors::ENaniteFallbackTarget::PercentTriangles ? ::ENaniteFallbackTarget::PercentTriangles : ::ENaniteFallbackTarget::RelativeError);
	MeshAssetOptions.NaniteSettings.FallbackPercentTriangles = Options.NaniteFallbackPercentTriangles;
	MeshAssetOptions.NaniteSettings.FallbackRelativeError = Options.NaniteFallbackRelativeError;

	MeshAssetOptions.bSupportRayTracing = Options.bSupportRayTracing;
	MeshAssetOptions.bAllowDistanceField = Options.bAllowDistanceField;
	MeshAssetOptions.bGenerateLightmapUVs = Options.bGenerateLightmapUVs;
	MeshAssetOptions.bCreatePhysicsBody = Options.bCreatePhysicsBody;

	MeshAssetOptions.bBuildReversedIndexBuffer = Options.bBuildReversedIndexBuffer;

	if (Material)
	{
		MeshAssetOptions.AssetMaterials.Add(Material);
	}
	else
	{
		MeshAssetOptions.AssetMaterials.Add(UMaterial::GetDefaultMaterial(MD_Surface));
	}

	MeshAssetOptions.bDeferPostEditChange = true; // Defer static mesh build to control options

	FStaticMeshResults MeshAssetOutputs;
	ECreateStaticMeshResult ResultCode = UE::AssetUtils::CreateStaticMeshAsset(MeshAssetOptions, MeshAssetOutputs);
	ensure(ResultCode == ECreateStaticMeshResult::Ok);

	
	UStaticMesh::FBuildParameters BuildParameters;
	BuildParameters.bInSilent = true;
	BuildParameters.bInRebuildUVChannelData = true;
	BuildParameters.bInEnforceLightmapRestrictions = true;

	MeshAssetOutputs.StaticMesh->Build(BuildParameters);

	ResultsOut.NewMeshAssets.Add(MeshAssetOutputs.StaticMesh);

	if (DebugMesh != nullptr)
	{
		FStaticMeshAssetOptions DebugMeshAssetOptions = MeshAssetOptions;
		DebugMeshAssetOptions.NewAssetPath = Options.BasePackagePath + TEXT("_DEBUG");
		DebugMeshAssetOptions.SourceMeshes.DynamicMeshes.Reset(1);
		DebugMeshAssetOptions.SourceMeshes.DynamicMeshes.Add(DebugMesh);

		FStaticMeshResults DebugMeshAssetOutputs;
		UE::AssetUtils::CreateStaticMeshAsset(DebugMeshAssetOptions, DebugMeshAssetOutputs);
	}

	return MeshAssetOutputs.StaticMesh;
}


#undef LOCTEXT_NAMESPACE
