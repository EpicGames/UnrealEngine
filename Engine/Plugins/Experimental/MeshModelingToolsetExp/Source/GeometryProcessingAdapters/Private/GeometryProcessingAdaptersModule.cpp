// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryProcessingAdaptersModule.h"
#include "Features/IModularFeatures.h"

#include "GeometryProcessing/ApproximateActorsImpl.h"
#include "GeometryProcessing/MeshAutoUVImpl.h"

#define LOCTEXT_NAMESPACE "FGeometryProcessingAdaptersModule"

using namespace UE::Geometry;

void FGeometryProcessingAdaptersModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
	ApproximateActors = MakeShared<FApproximateActorsImpl>();
	if (ApproximateActors.IsValid())
	{
		IModularFeatures::Get().RegisterModularFeature(IGeometryProcessing_ApproximateActors::GetModularFeatureName(), ApproximateActors.Get());
	}

	MeshAutoUV = MakeShared<FMeshAutoUVImpl>();
	if (MeshAutoUV.IsValid())
	{
		IModularFeatures::Get().RegisterModularFeature(IGeometryProcessing_MeshAutoUV::GetModularFeatureName(), MeshAutoUV.Get());
	}
}

void FGeometryProcessingAdaptersModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	
	if ( ApproximateActors.IsValid() )
	{
		IModularFeatures::Get().UnregisterModularFeature(IGeometryProcessing_ApproximateActors::GetModularFeatureName(), ApproximateActors.Get());
		ApproximateActors = nullptr;
	}

	if (MeshAutoUV.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IGeometryProcessing_MeshAutoUV::GetModularFeatureName(), MeshAutoUV.Get());
		MeshAutoUV = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryProcessingAdaptersModule, GeometryProcessingAdapters)
