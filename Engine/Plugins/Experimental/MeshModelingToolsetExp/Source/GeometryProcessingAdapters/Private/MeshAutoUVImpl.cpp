// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryProcessing/MeshAutoUVImpl.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "ParameterizationOps/ParameterizeMeshOp.h"


using namespace UE::Geometry;


IGeometryProcessing_MeshAutoUV::FOptions FMeshAutoUVImpl::ConstructDefaultOptions()
{
	return IGeometryProcessing_MeshAutoUV::FOptions();
}

void FMeshAutoUVImpl::GenerateUVs(FMeshDescription& InOutMesh, const IGeometryProcessing_MeshAutoUV::FOptions& Options, IGeometryProcessing_MeshAutoUV::FResults& ResultsOut)
{
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> DynamicMesh = MakeShared<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe>();
	FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
	MeshDescriptionToDynamicMesh.Convert(&InOutMesh, *DynamicMesh);

	UE::Geometry::FParameterizeMeshOp ParameterizeMeshOp;
	ParameterizeMeshOp.InputMesh = DynamicMesh;
	ParameterizeMeshOp.Stretch = Options.UVAtlasStretch;
	ParameterizeMeshOp.NumCharts = Options.UVAtlasNumCharts;
	ParameterizeMeshOp.XAtlasMaxIterations = Options.XAtlasMaxIterations;

	ParameterizeMeshOp.InitialPatchCount = Options.NumInitialPatches;
	ParameterizeMeshOp.PatchCurvatureAlignmentWeight = Options.CurvatureAlignment;
	ParameterizeMeshOp.PatchMergingMetricThresh = Options.MergingThreshold;
	ParameterizeMeshOp.PatchMergingAngleThresh = Options.MaxAngleDeviationDeg;
	ParameterizeMeshOp.ExpMapNormalSmoothingSteps = Options.SmoothingSteps;
	ParameterizeMeshOp.ExpMapNormalSmoothingAlpha = Options.SmoothingAlpha;

	ParameterizeMeshOp.bEnablePacking = Options.bAutoPack;
	ParameterizeMeshOp.Width = ParameterizeMeshOp.Height = Options.PackingTargetWidth;

	if (Options.Method == IGeometryProcessing_MeshAutoUV::EAutoUVMethod::UVAtlas)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::UVAtlas;
	}
	else if (Options.Method == IGeometryProcessing_MeshAutoUV::EAutoUVMethod::XAtlas)
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::XAtlas;
	}
	else
	{
		ParameterizeMeshOp.Method = UE::Geometry::EParamOpBackend::PatchBuilder;
	}

	ParameterizeMeshOp.CalculateResult(nullptr);
	
	TUniquePtr<UE::Geometry::FDynamicMesh3> DynamicMeshWithUVs = ParameterizeMeshOp.ExtractResult();
	if (DynamicMeshWithUVs)
	{
		// The DynamicMesh has now been populated with valid UVs for each vertex instance.
		// Rather than performing a full conversion of the dynamic mesh back to a mesh description, 
		// we can use the mapping in TriIDMap which was generated during the MeshDescription -> DynamicMesh conversion
		// This is actually required as the conversion to a DynamicMesh may have removed duplicate triangles.
		FDynamicMeshUVOverlay* DynamicMeshUVs = DynamicMeshWithUVs->Attributes()->PrimaryUV();
		TVertexInstanceAttributesRef<FVector2f> MeshDescriptionUVs = InOutMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

		// For each triangle of the dynamic mesh.
		for (const int DynamicMeshTID : DynamicMeshWithUVs->TriangleIndicesItr())
		{
			// Map triangle from the DynamicMesh to it's equivalent in the MeshDescription.
			int32 MeshDescriptionTID = MeshDescriptionToDynamicMesh.TriIDMap[DynamicMeshTID];

			// Grab the vertex instances (3) used by that triangle.
			const FIndex3i DynamicMeshTriVIDs = DynamicMeshWithUVs->GetTriangle(DynamicMeshTID);

			// For each triangle corner index...
			for (int Index = 0; Index < 3; ++Index)
			{
				// Grab the computed UV in the dynamic mesh for that vertex instance.
				const int DynamicMeshVID = DynamicMeshTriVIDs[Index];
				FVector2f UV = DynamicMeshUVs->GetElementAtVertex(DynamicMeshTID, DynamicMeshVID);

				// Assign the computed UV to the mesh description vertex instance.
				FVertexInstanceID MeshDescriptionVID = InOutMesh.GetTriangleVertexInstance(MeshDescriptionTID, Index);
				MeshDescriptionUVs.Set(MeshDescriptionVID, UV);
			}
		}

		ResultsOut.ResultCode = IGeometryProcessing_MeshAutoUV::EResultCode::Success;
	}
	else
	{
		ResultsOut.ResultCode = IGeometryProcessing_MeshAutoUV::EResultCode::UnknownError;
	}
}