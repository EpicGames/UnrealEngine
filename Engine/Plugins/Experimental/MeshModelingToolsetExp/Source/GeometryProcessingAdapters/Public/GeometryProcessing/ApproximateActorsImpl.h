// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryProcessingInterfaces/ApproximateActors.h"

#define UE_API GEOMETRYPROCESSINGADAPTERS_API

class UMaterialInterface;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Implementation of IGeometryProcessing_ApproximateActors
 */
class FApproximateActorsImpl : public IGeometryProcessing_ApproximateActors
{
public:
	UE_API virtual FOptions ConstructOptions(const FMeshApproximationSettings& MeshApproximationSettings) override;

	UE_API virtual void ApproximateActors(const FInput& Input, const FOptions& Options, FResults& ResultsOut) override;
	


protected:

	UE_API virtual void GenerateApproximationForActorSet(const FInput& Input, const FOptions& Options, FResults& ResultsOut);

	UE_API virtual UStaticMesh* EmitGeneratedMeshAsset(
		const FOptions& Options,
		FResults& ResultsOut,
		FDynamicMesh3* FinalMesh,
		UMaterialInterface* Material = nullptr,
		FDynamicMesh3* DebugMesh = nullptr);
};


}
}

#undef UE_API
