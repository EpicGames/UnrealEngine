// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryProcessingInterfaces/MeshAutoUV.h"

#define UE_API GEOMETRYPROCESSINGADAPTERS_API


namespace UE
{
namespace Geometry
{

/**
 * Implementation of IGeometryProcessing_MeshAutoUV
 */
class FMeshAutoUVImpl : public IGeometryProcessing_MeshAutoUV
{
public:
	UE_API virtual FOptions ConstructDefaultOptions() override;

	UE_API virtual void GenerateUVs(FMeshDescription& InOutMesh, const FOptions& Options, FResults& ResultsOut) override;
};


}
}

#undef UE_API
