// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

namespace UE::Geometry
{
	class FApproximateActorsImpl;
	class FMeshAutoUVImpl;
}

class FGeometryProcessingAdaptersModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	
protected:
	TSharedPtr<UE::Geometry::FApproximateActorsImpl> ApproximateActors;
	TSharedPtr<UE::Geometry::FMeshAutoUVImpl> MeshAutoUV;
};
