// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshModelingToolsEditorOnlyExp : ModuleRules
{
	public MeshModelingToolsEditorOnlyExp(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[]
			{
				// ... add public include paths required here ...
			}
		);

		PrivateIncludePaths.AddRange(
			new string[]
			{
				// ... add other private include paths required here ...
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore",
				"GeometryFramework",
				"DynamicMesh",
				"MeshConversion",
				"MeshModelingTools",
				"MeshModelingToolsExp",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"ModelingOperators",
				"ModelingOperatorsEditorOnly",
				"MeshModelingToolsEditorOnly",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BSPUtils",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"InputCore",
				"UnrealEd",
				"Persona",
				"AdvancedPreviewScene",
				"SceneOutliner",
				"Slate",
				"Json",
				"ApplicationCore"
				// ... add private dependencies that you statically link with here ...	
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}