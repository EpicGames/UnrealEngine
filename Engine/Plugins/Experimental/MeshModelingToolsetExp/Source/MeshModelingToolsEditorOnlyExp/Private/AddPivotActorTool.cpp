// Copyright Epic Games, Inc. All Rights Reserved.

#include "AddPivotActorTool.h"

#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Components/BillboardComponent.h"
#include "Editor.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "ModelingToolTargetUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddPivotActorTool)

#define LOCTEXT_NAMESPACE "UAddPivotActorTool"

using namespace UE::Geometry;

namespace AddPivotActorToolLocals
{
	UBillboardComponent* GetPivotBillboard(AActor* PivotActor)
	{
		for (const TObjectPtr<USceneComponent>& Child : PivotActor->GetRootComponent()->GetAttachChildren())
		{
			if (UBillboardComponent* Billboard = Cast<UBillboardComponent>(Child))
			{
				return Billboard;
			}
		}
		return nullptr;
	}
}

const FToolTargetTypeRequirements& UAddPivotActorToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(
		USceneComponentBackedTarget::StaticClass()
		);
	return TypeRequirements;
}

bool UAddPivotActorToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	using namespace AddPivotActorToolLocals;

	// There are some limitations for when we can use this tool. 
	// 1. We operate on the actor, not component level.
	//   TODO: Is there a good way to operate on a sub-actor level? Or should we be checking that
	//   we've selected all the components of each actor?
	// 2. If there are multiple actors selected, they need to have a common parent (or no parent),
	//   because otherwise we will be breaking up the user's hierarchy when we nest everything under
	//   the empty actor.

	if (SceneState.SelectedActors.Num() == 1 && ExactCast<AActor>(SceneState.SelectedActors[0]) 
		&& GetPivotBillboard(SceneState.SelectedActors[0]))
	{
		return true;
	}

	TSet<AActor*> ParentActors;

	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), 
		[&ParentActors](UActorComponent* Component) {
			AActor* Actor = Component->GetOwner();
			ParentActors.Add(Actor->GetAttachParentActor());
		});

	return ParentActors.Num() == 1;
}

UMultiSelectionMeshEditingTool* UAddPivotActorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UAddPivotActorTool* NewTool = NewObject<UAddPivotActorTool>(SceneState.ToolManager);
	if (SceneState.SelectedActors.Num() == 1 && ExactCast<AActor>(SceneState.SelectedActors[0]))
	{
		NewTool->SetPivotRepositionMode(SceneState.SelectedActors[0]);
	}

	return NewTool;
}

void UAddPivotActorToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	const FToolTargetTypeRequirements& Requirements = GetTargetRequirements();
	TArray<TObjectPtr<UToolTarget>> Targets;
	if (SceneState.SelectedComponents.Num() > 0)
	{
		Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, Requirements);
	}
	else
	{
		// When dealing with actor selection, only grab one target per owning actor.
		TSet<AActor*> OwningActors;
		SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, Requirements, [this, &SceneState, &Targets, &Requirements, &OwningActors](UActorComponent* Component)
		{
			if (AActor* OwnerActor = Component->GetOwner())
			{
				bool bAlreadyInSet = false;
				OwningActors.Add(OwnerActor, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					Targets.Add(SceneState.TargetManager->BuildTarget(Component, Requirements));
				}
			}
		});
	}
	
	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);
}

void UAddPivotActorTool::Setup()
{
	using namespace AddPivotActorToolLocals;

	FTransform StartTransform = FTransform::Identity;
	if (ExistingPivotActor.IsValid())
	{
		SetToolDisplayName(LOCTEXT("ModifyPivotActorToolName", "Modify Pivot Actor"));

		GetToolManager()->DisplayMessage(LOCTEXT("OnStartToolEdit",
			"Modifies the position of the pivot actor while keeping children in place. "
			"Hold Ctrl to snap to items in scene."),
			EToolMessageLevel::UserNotification);

		StartTransform = ExistingPivotActor->GetTransform();
		ExistingPivotOriginalTransform = StartTransform;

		// Hide the pivot sprite.
		UBillboardComponent* Billboard = GetPivotBillboard(ExistingPivotActor.Get());
		if (ensure(Billboard))
		{
			Billboard->SetVisibility(false);
		}
	}
	else
	{
		SetToolDisplayName(LOCTEXT("AddPivotActorToolName", "Add Pivot Actor"));

		GetToolManager()->DisplayMessage(LOCTEXT("OnStartToolAdd",
			"Adds an empty actor as the parent of the selected actors. Use gizmo to choose where/how "
			"the empty actor is placed. Hold Ctrl to snap to items in scene."),
			EToolMessageLevel::UserNotification);

		// Figure out where to start the gizmo. The location will be the average,
		// and the rotation will either be identity, or the target's if there is
		// only one.
		FVector3d StartTranslation = FVector3d::Zero();
		for (const TObjectPtr<UToolTarget>& Target : Targets)
		{
			StartTranslation += UE::ToolTarget::GetLocalToWorldTransform(Target).GetTranslation();
		}
		StartTranslation /= Targets.Num();

		StartTransform.SetTranslation(StartTranslation);

		if (Targets.Num() == 1)
		{
			StartTransform.SetRotation(FQuat(
				UE::ToolTarget::GetLocalToWorldTransform(Targets[0]).GetRotation()));
		}
	}

	TransformProperties = NewObject<UPivotActorTransformProperties>();
	AddToolPropertySource(TransformProperties);
	TransformProperties->Position = StartTransform.GetTranslation();
	TransformProperties->Rotation = StartTransform.GetRotation();

	// Set up the gizmo.
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform(StartTransform);
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(
		GetToolManager()->GetPairedGizmoManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this);
	TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());
	TransformProxy->OnTransformChanged.AddUObject(this, &UAddPivotActorTool::GizmoTransformChanged);

	GizmoPositionWatcher.Initialize(
		[this]() { return TransformProperties->Position; },
		[this](FVector NewPosition)
		{ 
			UpdateGizmoFromProperties();
		}, TransformProperties->Position);
	GizmoRotationWatcher.Initialize(
		[this]() { return TransformProperties->Rotation; },
		[this](FQuat NewRotation)
		{
			UpdateGizmoFromProperties();
		}, TransformProperties->Rotation);

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(TransformGizmo);
}

void UAddPivotActorTool::OnTick(float DeltaTime)
{
	GizmoPositionWatcher.CheckAndUpdate();
	GizmoRotationWatcher.CheckAndUpdate();
}

void UAddPivotActorTool::UpdateGizmoFromProperties()
{
	if (TransformGizmo)
	{
		TransformGizmo->SetNewGizmoTransform(FTransform(TransformProperties->Rotation, TransformProperties->Position));
	}
}

void UAddPivotActorTool::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	TransformProperties->Position = Transform.GetTranslation();
	TransformProperties->Rotation = Transform.GetRotation();
	NotifyOfPropertyChangeByTool(TransformProperties);
	GizmoPositionWatcher.SilentUpdate();
	GizmoRotationWatcher.SilentUpdate();
}

void UAddPivotActorTool::OnShutdown(EToolShutdownType ShutdownType)
{
	using namespace AddPivotActorToolLocals;

	if (ExistingPivotActor.IsValid())
	{
		// Make the sprite visible again.
		UBillboardComponent* Billboard = GetPivotBillboard(ExistingPivotActor.Get());
		if (Billboard)
		{
			Billboard->SetVisibility(true);
		}
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		if (ExistingPivotActor.IsValid())
		{
			USceneComponent* PivotComponent = ExistingPivotActor->GetRootComponent();
			FTransform GizmoTransform = TransformProxy->GetTransform();

			GetToolManager()->BeginUndoTransaction(LOCTEXT("MovePivotTransactionName", "Move Pivot Actor"));

			// Move the billboard (the rendered pivot) to the new location (i.e. don't adjust it's relative transform),
			// but keep all the other children in their current location.
			TArray<USceneComponent*> Children;
			PivotComponent->GetChildrenComponents(false, Children);
			UBillboardComponent* Billboard = GetPivotBillboard(ExistingPivotActor.Get());
			for (USceneComponent* Child : Children)
			{
				if (Child == Billboard)
				{
					continue;
				}
				Child->Modify();

				FTransform NewTransform = Child->GetComponentToWorld();
				NewTransform.SetToRelativeTransform(GizmoTransform);
				Child->SetRelativeTransform(NewTransform);
			}

			// Move the pivot itself. Note that this needs to happen after moving the children above, because
			// otherwise the above GetComponentToWorld call won't get us the original location.
			PivotComponent->Modify();
			PivotComponent->SetWorldTransform(GizmoTransform);

			GetToolManager()->EndUndoTransaction();
		}
		// If in "add" mode
		else if (ensure(Targets.Num() > 0)) 
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("AddPivotActorTransactionName", "Add Pivot Actor"));

			// Create an empty actor at the location of the gizmo. The way we do it here, using this factory, is
			// editor-only.
			UActorFactoryEmptyActor* EmptyActorFactory = NewObject<UActorFactoryEmptyActor>();
			FAssetData AssetData(EmptyActorFactory->GetDefaultActorClass(FAssetData()));
			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = TEXT("PivotActor");
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
			AActor* NewActor = EmptyActorFactory->CreateActor(AssetData.GetAsset(),
				TargetWorld->GetCurrentLevel(),
				TransformProxy->GetTransform(),
				SpawnParams);

			// Set the mobility of the pivot actor. It's usually Movable unless one of the actors is Static,
			// because you can only nest Static under Static.
			EComponentMobility::Type MobilityToUse = EComponentMobility::Movable;
			for (const TObjectPtr<UToolTarget>& Target : Targets)
			{
				AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
				if (TargetActor->GetRootComponent()->Mobility == EComponentMobility::Static)
				{
					MobilityToUse = EComponentMobility::Static;
					break;
				}
			}
			NewActor->GetRootComponent()->SetMobility(MobilityToUse);

			// Grab the first selected target. It will have the same parent as the other ones, so
			// we'll use it to figure out the empty actor's parent.
			AActor* TargetActor = UE::ToolTarget::GetTargetActor(Targets[0]);

			// This is also editor-only: it's the label that shows up in the hierarchy
			NewActor->SetActorLabel(Targets.Num() == 1 ? TargetActor->GetActorLabel() + TEXT("_Pivot")
				: TEXT("Pivot"));

			// Attach the empty actor in the correct place in the hierarchy. This can be done in a non-editor-only
			// way, but it's easier to use the editor's function to do it so that undo/redo and level saving are 
			// properly done.
			if (AActor* ParentActor = TargetActor->GetAttachParentActor())
			{
				GEditor->ParentActors(ParentActor, NewActor, NAME_None);
			}
			for (const TObjectPtr<UToolTarget>& Target : Targets)
			{
				TargetActor = UE::ToolTarget::GetTargetActor(Target);
				GEditor->ParentActors(NewActor, TargetActor, NAME_None);
			}
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewActor);

			GetToolManager()->EndUndoTransaction();
		}
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	TransformProxy = nullptr;
	TransformGizmo = nullptr;
	DragAlignmentMechanic->Shutdown();
	ExistingPivotActor.Reset();
}

void UAddPivotActorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
}

#undef LOCTEXT_NAMESPACE

