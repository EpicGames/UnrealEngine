// Copyright Epic Games, Inc. All Rights Reserved.

#include "BspConversionTool.h"

#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "AssetSelection.h"
#include "ComponentSourceInterfaces.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/StaticMeshActor.h"
#include "InteractiveToolManager.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Model.h"
#include "Operations/MeshBoolean.h"
#include "StaticMeshAttributes.h"
#include "ToolBuilderUtil.h"
#include "Tools/EditorComponentSourceFactory.h"
#include "ToolSetupUtil.h"
#include "Engine/Selection.h"
#include "ScopedTransaction.h"

#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Logging/TokenizedMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BspConversionTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBspConversionTool"

// Forward declarations of local functions
void ConvertBrushesToDynamicMesh(TArray<ABrush*>& BrushesToConvert, FDynamicMesh3& OutputMesh, 
	TArray<UMaterialInterface*>& OutputMaterials);
bool ApplyDynamicMeshBooleanOperation(
	FDynamicMesh3& MeshA, const FTransformSRT3d& TransformA, const TArray<UMaterialInterface*>& MaterialsA,
	FDynamicMesh3& MeshB, const FTransformSRT3d& TransformB, const TArray<UMaterialInterface*>& MaterialsB,
	FDynamicMesh3& OutputMesh, FTransformSRT3d& OutputTransform, TArray<UMaterialInterface*>& OutputMaterials,
	FMeshBoolean::EBooleanOp Operation);
FText GetBrushGeometryErrorMessage(ABrush* Brush);

// Element stored in CachedBrushes: the resulting dynamic mesh and the materials array
typedef TPair<TSharedPtr<const FDynamicMesh3>, TSharedPtr<const TArray<UMaterialInterface*>>> FCachedResult;


// Tool builder functions

bool UBspConversionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// We allow the tool to be built even if nothing is selected because the tool has a "select all" option.
	// We just need to be able to get the place to save assets.
	return true;
}

UInteractiveTool* UBspConversionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UBspConversionTool* NewTool = NewObject<UBspConversionTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}


// Tool property functions

void UBspConversionToolActionPropertySet::PostAction(EBspConversionToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


// Tool itself

UBspConversionTool::UBspConversionTool()
{
}

void UBspConversionTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
}

void UBspConversionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

bool UBspConversionTool::CanAccept() const
{
	// We precompute this value and update it at various editor events
	return bCanAccept;
}

void UBspConversionTool::Setup()
{
	UInteractiveTool::Setup();

	// Link in the tool properties and actions that will be displayed in the side panel
	Settings = NewObject<UBspConversionToolProperties>();
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	ToolActions = NewObject<UBspConversionToolActionPropertySet>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	SetToolDisplayName(LOCTEXT("ToolName", "Convert BSP"));
	// Give a description to put in the side panel
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Convert geometry brushes (also known as BSP brushes) into a single static mesh."),
		EToolMessageLevel::UserNotification);

	// We write out an empty warning message to make the sidebar look unchanged if we write out a warning message and then
	// change it to be an empty string to clear it.
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

	// See if we have valid targets selected
	bCanAccept = AtLeastOneValidConversionTarget();

	// Set up the preview mesh
	PreviewMesh = NewObject<UPreviewMesh>(this, "Preview Mesh");
	PreviewMesh->CreateInWorld(TargetWorld, FTransform::Identity);

	// The material is set to have opacity 0, so that only the wireframe shows.
	PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetSimpleCustomMaterial(GetToolManager(), FLinearColor(), 0.0f));
	PreviewMesh->EnableWireframe(Settings->bShowPreview);

	// Register with any editor events that will require us to recompute the preview or update bCanAccept
	USelection::SelectionChangedEvent.AddUObject(this, &UBspConversionTool::OnEditorSelectionChanged);
	GEngine->OnLevelActorListChanged().AddUObject(this, &UBspConversionTool::OnEditorLevelActorListChanged);
	GEngine->OnActorMoved().AddUObject(this, &UBspConversionTool::OnEditorActorMoved);

	// Do some precomputing if we show a preview.
	if (bCanAccept && Settings->bShowPreview)
	{
		CompareAndUpdateConversionTargets();

		FText ErrorMessage;
		// When we do the actual computation, we may find new errors that prevent us from accepting
		bCanAccept = ComputeAndUpdatePreviewMesh(&ErrorMessage);

		if (!ErrorMessage.IsEmpty())
		{
			GetToolManager()->DisplayMessage(ErrorMessage, EToolMessageLevel::UserWarning);
		}
	}
}

/**
 * Returns true if there is at least one valid conversion actor selected (an explicitly selected brush, or a volume if
 * those are included).
 *
 * Notes:
 * - An explictly selected subtractive brush is valid, as users may want to delete subtractive brushes through the tool.
 * - Even with an additive brush, a resulting mesh could still end up empty, for instance if a brush was fully inside a subtractive one.
 * - Does not check if targets have valid geometry for the "convert then combine" path (for mesh boolean operations),
 *   as that currently requires doing an actual conversion.
 */
bool UBspConversionTool::AtLeastOneValidConversionTarget() const
{
	for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
	{
		ABrush* Brush = Cast<ABrush>(*Iter);
		if (IsValidConversionTarget(Brush))
		{
			return true;
		}
	}

	return false;
}

/**
 * Determines whether the passed in pointer is a valid convertible brush. Safe to call with nullptr.
 * Considers Settings->bIncludeVolumes.
 */
bool UBspConversionTool::IsValidConversionTarget(const ABrush* Brush) const
{
	return IsValid(Brush)
		&& (!Brush->IsVolumeBrush() || Settings->bIncludeVolumes)
		&& (Brush->BrushType == EBrushType::Brush_Add || Brush->BrushType == EBrushType::Brush_Subtract);
}

void UBspConversionTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("BspConversionToolTransactionName", "BSP Conversion"));

		// If settings had been set to show a preview, then we would have already generated a preview mesh
		// that we could copy the result from. Thus, we only need to generate here if the settings were set
		// to not use a preview.
		if (!Settings->bShowPreview)
		{
			CompareAndUpdateConversionTargets();
			FText ErrorMessage;
			if (!ComputeAndUpdatePreviewMesh(&ErrorMessage))
			{
				// We're closing the tool, so it's too late to just display a side panel warning. Give a pop up.
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
				
				// Don't delete any brushes if we failed
				BrushesToDelete.Empty();
			}
		}

		// We only need to output something if the result wasn't empty.
		if (PreviewMesh->GetMesh()->VertexCount() > 0)
		{
			TArray<UMaterialInterface*> Materials;
			PreviewMesh->GetMaterials(Materials);

			FCreateMeshObjectParams NewMeshObjectParams;
			NewMeshObjectParams.TargetWorld = TargetWorld;
			NewMeshObjectParams.Transform = PreviewMesh->GetTransform();
			NewMeshObjectParams.BaseName = TEXT("BspMesh");
			NewMeshObjectParams.Materials = Materials;
			NewMeshObjectParams.SetMesh(PreviewMesh->GetMesh());
			FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
			if (Result.IsOK() && Result.NewActor != nullptr)
			{
				ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
			}
		}

		// Delete brushes that we marked for deletion. This may need to happen even if the resulting mesh
		// was empty, for instance if everything was inside a subtractive brush.
		for (ABrush* Brush : BrushesToDelete)
		{
			TargetWorld->EditorDestroyActor(Brush, true);
		}
		GEditor->RebuildAlteredBSP();

		GetToolManager()->EndUndoTransaction();
	}

	// Remove the preview mesh
	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	// Deregister all the callbacks we registered
	USelection::SelectionChangedEvent.RemoveAll(this);
	GEngine->OnLevelActorListChanged().RemoveAll(this);
	GEngine->OnActorMoved().RemoveAll(this);

	// Empty cached data
	CachedBrushes.Empty();

	Settings->SaveProperties(this);
}

// Conversion functions

/**
 * Updates the targets that the conversion functions operate on (BrushesToConvert, BrushForPivot),
*  based on level composition, current selection, and settings. Also checks if this is a change 
*  from the previous value of BrushesToConvert.
 *
 * @return true if BrushesToConvert changed.
 */
bool UBspConversionTool::CompareAndUpdateConversionTargets()
{
	TArray<ABrush*> PreviousBrushesToConvert = BrushesToConvert;
	BrushesToConvert.Empty();
	BrushesToDelete.Empty();

	// One thing we need to check is whether we have just a single additive brush,
	// in which case we want to keep the pivot the same (and we will set BrushForPivot)
	int NumAdditiveBrushes = 0;

	// The order of brush composition is determined by their order in the ULevel, and may be different from 
	// their order of selection. We make BrushesToConvert contain selected brushes in the proper order.
	auto& LevelActorsInCompositionOrder = TargetWorld->GetCurrentLevel()->Actors;
	for (int i = 0; i < LevelActorsInCompositionOrder.Num(); ++i)
	{
		ABrush* Brush = Cast<ABrush>(LevelActorsInCompositionOrder[i]);
		if (IsValidConversionTarget(Brush)
			&& (Brush->IsSelected() || (!Settings->bExplicitSubtractiveBrushSelection && Brush->BrushType == EBrushType::Brush_Subtract)))
		{
			BrushesToConvert.Add(Brush);

			if (Brush->BrushType == EBrushType::Brush_Add)
			{
				++NumAdditiveBrushes;
				BrushForPivot = Brush;

				// Additive volume brushes may not be deleted, depending on settings
				if (!Brush->IsVolumeBrush() || Settings->bRemoveConvertedVolumes)
				{
					BrushesToDelete.Add(Brush);
				}
			}
			// Subtractive brushes only get deleted if they were explicitly selected and the settings allow it
			else if (Settings->bRemoveConvertedSubtractiveBrushes && Brush->IsSelected())
			{
				BrushesToDelete.Add(Brush);
			}
		}
	}

	if (NumAdditiveBrushes != 1)
	{
		BrushForPivot = nullptr;
	}

	return BrushesToConvert != PreviousBrushesToConvert;
}

/**
 * Performs conversion of BrushesToConvert and stores the result in the preview mesh.
 * 
 * @param OutErrorMessage Error message to fill if there is a problem.
 * @return true if successful.
 */
bool UBspConversionTool::ComputeAndUpdatePreviewMesh(FText* OutErrorMessage)
{
	if (Settings->ConversionMode == EBspConversionMode::CombineFirst)
	{
		return CombineThenConvert(OutErrorMessage);
	}
	else
	{
		return ConvertThenCombine(OutErrorMessage);
	}
}

/**
 * The easy conversion path, where we just use the existing bsp conversion function to
 * convert everything that was selected. Operates on non-manifold geometry. Doesn't write
 * out an error message. Returns true unless BrushesToConvert was empty.
 *
 * BrushesToConvert must be initialized.
 */
bool UBspConversionTool::CombineThenConvert(FText*)
{
	if (BrushesToConvert.Num() == 0)
	{
		PreviewMesh->ClearPreview();
		return false;
	}

	FDynamicMesh3 OutputMesh;
	TArray<UMaterialInterface*> OutputMaterials;
	ConvertBrushesToDynamicMesh(BrushesToConvert, OutputMesh, OutputMaterials);

	// The created mesh is built with its pivot at the origin, so we need to reset it. If there was
	// just one additive brush, we keep the pivot the same, otherwise we center it.
	FVector3d NewPivot;
	if (BrushForPivot)
	{
		NewPivot = (FVector3d)BrushForPivot->GetTransform().GetLocation();
	}
	else
	{
		NewPivot = OutputMesh.GetBounds().Center();
	}

	MeshTransforms::Translate(OutputMesh, -NewPivot);
	FTransformSRT3d Transform = FTransformSRT3d::Identity();
	Transform.SetTranslation(NewPivot);

	PreviewMesh->UpdatePreview(&OutputMesh);
	PreviewMesh->SetTransform((FTransform)Transform);
	PreviewMesh->SetMaterials(OutputMaterials);

	return true;
}

/**
 * The more complicated conversion path, where we convert brushes individually and use static mesh boolean
 * operations to combine them. Boolean operations fail when geometry is non-manifold, which happens in
 * the case of stair brushes, mainly, due to them not being properly closed.
 * The preview gets cleared in the case of an error.
 * 
 * @param ErrorMessage Place to write out error message if a conversion results in invalid geometry.
 * @return false if BrushesToConvert is empty or a conversion gives invalid geometry, true otherwise.
 */
bool UBspConversionTool::ConvertThenCombine(FText *ErrorMessage)
{
	if (BrushesToConvert.Num() == 0)
	{
		PreviewMesh->ClearPreview();
		return false;
	}

	// We'll need temporary space to store meshes as we build them up. The input and output meshes,
	// transforms, and material arrays will swap after we merge in each new mesh.
	FDynamicMesh3 MeshStorage[2];
	FTransformSRT3d TransformStorage[2];
	TArray<UMaterialInterface*> MaterialArraysStorage[2];

	FDynamicMesh3* InputMesh = &MeshStorage[0];
	FDynamicMesh3* OutputMesh = &MeshStorage[1];
	
	FTransformSRT3d* InputTransform = &TransformStorage[0];
	FTransformSRT3d* OutputTransform = &TransformStorage[1];

	TArray<UMaterialInterface*>* InputMaterials = &MaterialArraysStorage[0];
	TArray<UMaterialInterface*>* OutputMaterials = &MaterialArraysStorage[1];

	// The individual mesh to merge in is dependent only on that brush, and therefore can be
	// cached. Unfortunately, we need to be able to modify the mesh while doing boolean operations
	// to update the material references of the triangles to a common material set, so we need to
	// create copies of the cached mesh.
	FDynamicMesh3 NextMesh;

	// The transform when a brush is first converted is always the same
	FTransformSRT3d NextTransform = FTransformSRT3d::Identity();

	// We could actually point to cached materials since we don't change these, but for consistency,
	// we'll have a local copy.
	TArray<UMaterialInterface*> NextMaterials;

	// This will point to the next bsp brush that we're considering
	ABrush* NextBrush = BrushesToConvert[0];

	// When converting individual brushes, we pass them to ConvertBrushesToDynamicMesh as a
	// one-element array.
	TArray<ABrush*> WrappedBrush = { NextBrush };

	// Convert the first brush unless it is subtractive, in which case the mesh will stay empty.
	// This isn't just an optimization- we also need to avoid caching this brush, or trying to use a cached
	// additive version.
	if (NextBrush->BrushType != EBrushType::Brush_Subtract)
	{
		if (Settings->bCacheBrushes && CachedBrushes.Contains(NextBrush))
		{
			OutputMesh->Copy(*CachedBrushes[NextBrush].Key);
			*OutputMaterials = *CachedBrushes[NextBrush].Value;
		}
		else
		{
			ConvertBrushesToDynamicMesh(WrappedBrush, *OutputMesh, *OutputMaterials);

			if (Settings->bCacheBrushes)
			{
				CachedBrushes.Add(NextBrush, FCachedResult(MakeShared<const FDynamicMesh3>(*OutputMesh),
					MakeShared<const TArray<UMaterialInterface*>>(*OutputMaterials)));
			}
		}
		*OutputTransform = FTransformSRT3d::Identity(); // Always identity when first converted
	}

	// Now convert the other meshes one at a time and apply them using boolean mesh operations
	for (int i = 1; i < BrushesToConvert.Num(); ++i)
	{
		NextBrush = BrushesToConvert[i];

		Swap(OutputMesh, InputMesh);
		Swap(OutputTransform, InputTransform);
		Swap(OutputMaterials, InputMaterials);
		OutputMesh->Clear();

		FMeshBoolean::EBooleanOp Operation = FMeshBoolean::EBooleanOp::Union;
		bool IsSubtractiveBrush = NextBrush->BrushType == EBrushType::Brush_Subtract;
		if (IsSubtractiveBrush)
		{
			Operation = FMeshBoolean::EBooleanOp::Difference;

			// We need a solid mesh to subtract, so the conversion code needs to operate on an additive brush.
			// However, we're going to undo this once we're done because the brush may persist after the conversion,
			// depending on the settings.
			// Despite the undoing, we need to call Modify now because it gets called in the conversion code, which
			// would cause the changed property to be saved.
			NextBrush->Modify(false);
			NextBrush->BrushType = EBrushType::Brush_Add;
		}

		// Convert the next brush
		if (Settings->bCacheBrushes && CachedBrushes.Contains(NextBrush))
		{
			NextMesh.Copy(*CachedBrushes[NextBrush].Key);
			NextMaterials = *CachedBrushes[NextBrush].Value;
		}
		else
		{
			WrappedBrush[0] = NextBrush;

			// Get the result
			ConvertBrushesToDynamicMesh(WrappedBrush, NextMesh, NextMaterials);

			if (Settings->bCacheBrushes)
			{
				CachedBrushes.Add(NextBrush, FCachedResult(MakeShared<const FDynamicMesh3>(NextMesh),
					MakeShared<const TArray<UMaterialInterface*>>(NextMaterials)));
			}
		}

		// Apply the boolean operation
		bool bSuccess = true;
		if (IsSubtractiveBrush)
		{
			bSuccess = ApplyDynamicMeshBooleanOperation(
				*InputMesh, *InputTransform, *InputMaterials,
				NextMesh, NextTransform, NextMaterials,
				*OutputMesh, *OutputTransform, *OutputMaterials, Operation);
		
			// Undo the change in brush type that we did before.
			NextBrush->BrushType = EBrushType::Brush_Subtract;
		}
		else
		{
			// For union, we actually swap the order of meshes in hopes of better lining up
			// with BSP brush priority in coplanar places (brushes added later have priority,
			// whereas for our boolean operations, first mesh has priority)
			bSuccess = ApplyDynamicMeshBooleanOperation(
				NextMesh, NextTransform, NextMaterials,
				*InputMesh, *InputTransform, *InputMaterials,
				*OutputMesh, *OutputTransform, *OutputMaterials, Operation);
		}

		if (!bSuccess)
		{
			PreviewMesh->ClearPreview();
			if (ErrorMessage)
			{
				*ErrorMessage = GetBrushGeometryErrorMessage(NextBrush);
			}
			return false;
		}
	}//end converting other brushes

	// If there was only a single additive brush, we should keep its pivot in the original location.
	if (BrushForPivot)
	{
		FVector3d BrushTranslation = (FVector3d)BrushForPivot->GetTransform().GetLocation();
		MeshTransforms::Translate(*OutputMesh, OutputTransform->GetTranslation() - BrushTranslation);
		OutputTransform->SetTranslation(BrushTranslation);
	}
	// Otherwise, the pivot set by boolean operations is appropriate.

	PreviewMesh->UpdatePreview(OutputMesh);
	PreviewMesh->SetTransform((FTransform)(*OutputTransform));
	PreviewMesh->SetMaterials(*OutputMaterials);

	return true;
}

FText GetBrushGeometryErrorMessage(ABrush* Brush)
{
	return FText::Format(LOCTEXT("ConvertThenCombineInvalidGeometryError", 
		"Failed attempting the \"Convert, then Combine\" path while trying to compose brush \"{0}\" with "
		"previous results. Try using \"Combine, then Convert\" to convert, then use MeshInspector to look "
		"for problematic areas around that brush."),
		// Brush->GetActorLabel is an editor-only call
		FText::FromString(Brush->GetActorLabel()));
}

/** Uses our existing conversion functions to convert brushes to a single DynamicMesh. */
void ConvertBrushesToDynamicMesh(TArray<ABrush*>& BrushesToConvert, FDynamicMesh3& OutputMesh, TArray<UMaterialInterface*>& OutputMaterials)
{
	// Have the editor rebuild a model composed only of selected brushes into a temporary UModel object, and make
	// sure its polygons get built.
	// Even though it's temporary, we're not allowed to make a UModel on the stack
	UModel* TempModel = NewObject<UModel>();
	TempModel->Initialize(nullptr);
	GEditor->RebuildModelFromBrushes(BrushesToConvert, TempModel);
	GEditor->bspBuildFPolys(TempModel, true, 0); // SurfLinks parameter doesn't matter

	// Prep some output variables
	FMeshDescription MeshDescription;
	FStaticMeshAttributes StaticMeshAttributes(MeshDescription);
	StaticMeshAttributes.Register();

	TArray<FStaticMaterial> Materials;

	// Do the actual conversion using our old conversion function
	GetBrushMesh(nullptr, TempModel, MeshDescription, Materials);

	// Get a list of material interfaces out of the list of materials that we got
	OutputMaterials.Empty();
	for (FStaticMaterial Material : Materials)
	{
		OutputMaterials.Add(Material.MaterialInterface);
	}

	// Turn the mesh description into a DynamicMesh
	OutputMesh.Clear();
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(&MeshDescription, OutputMesh);
}

bool ApplyDynamicMeshBooleanOperation(
	FDynamicMesh3& MeshA, const FTransformSRT3d& TransformA, const TArray<UMaterialInterface*>& MaterialsA,
	FDynamicMesh3& MeshB, const FTransformSRT3d& TransformB, const TArray<UMaterialInterface*>& MaterialsB,
	FDynamicMesh3& OutputMesh, FTransformSRT3d& OutputTransform, TArray<UMaterialInterface*>& OutputMaterials,
	FMeshBoolean::EBooleanOp Operation)
{
	// These need to be enabled on both meshes to deal with materials properly. This is relevant, for
	// instance, if the first brush in the composition list was empty.
	MeshA.EnableAttributes();
	MeshA.Attributes()->EnableMaterialID();
	MeshB.EnableAttributes();
	MeshB.Attributes()->EnableMaterialID();


	// We'll need to combine all the materials into the output, but not duplicate them across the two
	// meshes. This will keep track of the materials that we have seen.
	TMap<UMaterialInterface*, int> SeenMaterials;

	// Start with the materials from the first mesh.
	OutputMaterials = MaterialsA;
	for (int i = 0; i < OutputMaterials.Num(); ++i)
	{
		SeenMaterials.Add(OutputMaterials[i], i);
	}

	// Add any new materials from mesh B, and remap any materials that we already have in that mesh. MaterialBRemap
	// maps old material indices to new ones.
	TArray<int> MaterialBRemap;
	for (UMaterialInterface* Mat : MaterialsB)
	{
		int NewMatIndex;
		int* FoundMatIdx = SeenMaterials.Find(Mat);
		if (FoundMatIdx)
		{
			NewMatIndex = *FoundMatIdx;
		}
		else
		{
			NewMatIndex = OutputMaterials.Add(Mat);
			SeenMaterials.Add(Mat, NewMatIndex);
		}
		MaterialBRemap.Add(NewMatIndex);
	}

	// Apply the remapping of material indices to mesh B.
	FDynamicMeshMaterialAttribute* MaterialIDs = MeshB.Attributes()->GetMaterialID();
	for (int TID : MeshB.TriangleIndicesItr())
	{
		MaterialIDs->SetValue(TID, MaterialBRemap[MaterialIDs->GetValue(TID)]);
	}

	// Perform the actual boolean operation.
	FMeshBoolean BooleanOperation(&MeshA, TransformA, &MeshB, TransformB, &OutputMesh, Operation);
	BooleanOperation.bSimplifyAlongNewEdges = true;
	bool bSuccess = BooleanOperation.Compute();
	OutputTransform = BooleanOperation.ResultTransform;

	return bSuccess;
}


// Button support

void UBspConversionTool::RequestAction(EBspConversionToolAction ActionType)
{
	if (PendingAction == EBspConversionToolAction::NoAction)
	{
		PendingAction = ActionType;
	}
}

void UBspConversionTool::OnTick(float DeltaTime)
{
	if (PendingAction != EBspConversionToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = EBspConversionToolAction::NoAction;
	}
}

void UBspConversionTool::ApplyAction(EBspConversionToolAction ActionType)
{
	switch (ActionType)
	{
	case EBspConversionToolAction::SelectAllValidBrushes:
	{
		FScopedTransaction Transaction(LOCTEXT("SelectAllValidBrushes", "Select All Valid Brushes"));

		FSelectedObjectsChangeList NewSelection;
		NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;

		// Accumulate all actors in the current level that are valid brushes
		for (auto Iter(TargetWorld->GetCurrentLevel()->Actors.CreateConstIterator()); Iter; ++Iter)
		{
			ABrush* Brush = Cast<ABrush>(*Iter);
			if (IsValidConversionTarget(Brush))
			{
				NewSelection.Actors.Add(Brush);
			}
		}

		GetToolManager()->RequestSelectionChange(NewSelection);
	}
	break;
	case EBspConversionToolAction::DeselectVolumes:
	{
		FScopedTransaction Transaction(LOCTEXT("DeselectVolumes", "Deselect Volumes"));

		FSelectedObjectsChangeList NewSelection;
		NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;

		// Out of the current selection, keep anything except volume brushes
		for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			if (!(Cast<ABrush>(Actor) && Cast<ABrush>(Actor)->IsVolumeBrush()))
			{
				NewSelection.Actors.Add(Actor);
			}
		}

		GetToolManager()->RequestSelectionChange(NewSelection);
	}
	break;
	case EBspConversionToolAction::DeselectNonValid:
	{
		FScopedTransaction Transaction(LOCTEXT("DeselectNonValid", "Deselect Non-Valid"));

		// Normally, "GEditor->SelectNone(true, false, false)" would deselect all brushes, but
		// it does not deselect volumes, which we want to do depending on the settings.

		FSelectedObjectsChangeList NewSelection;
		NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;

		// From current selection, select everything that is valid given current settings
		for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
		{
			ABrush* Brush = Cast<ABrush>(*Iter);
			if (IsValidConversionTarget(Brush))
			{
				NewSelection.Actors.Add(Brush);
			}
		}

		GetToolManager()->RequestSelectionChange(NewSelection);
	}
	break;
	}
}


// Callback functions

// We need to keep track of the following:
// - BrushesToConvert needs to contain relevant brushes given settings and selection
// - PreviewMesh may need updating
// - CachedBrushes must not contain any outdated conversions.

/**
 * This is the primary event we'll be responding to. It affects whether we can accept
 * or not, and the preview, if it is being shown. Does not affect cached brushes.
 */
void UBspConversionTool::OnEditorSelectionChanged(UObject* NewSelection)
{
	USelection* Selection = Cast<USelection>(NewSelection);
	if (!Selection)
	{
		// Clear things
		bCanAccept = false;
		PreviewMesh->ClearPreview();
		BrushesToConvert.Empty();
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning); // This "clears" it
		return;
	}

	// Update whether the tool can accept safely
	bCanAccept = AtLeastOneValidConversionTarget();

	// Update preview. This may change a true bCanAccept to false if there are errors
	if (!bCanAccept)
	{
		// If there isn't anything selected yet, clear things
		PreviewMesh->ClearPreview();
		BrushesToConvert.Empty();
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
	}
	else if (Settings->bShowPreview && CompareAndUpdateConversionTargets())
	{
		FText ErrorMessage;
		bCanAccept = ComputeAndUpdatePreviewMesh(&ErrorMessage);

		// We do this regardless of success because it resets the error message if ErrorMessage is empty.
		GetToolManager()->DisplayMessage(ErrorMessage, EToolMessageLevel::UserWarning);
	}
}

/**
 * Changes to the actor list may change the order of brush composition, or add or remove implicitly included
 * subtractive brushes. Should not affect cached brushes.
 */
void UBspConversionTool::OnEditorLevelActorListChanged()
{
	// Safest thing to do is to do the same things as OnEditorSelectionChanged, even though bCanAccept should
	// only change in very weird cases (such as removing an implicit subtractive stair brush).

	// Update whether the tool can accept safely
	bCanAccept = AtLeastOneValidConversionTarget();

	// Update preview. This may change a true bCanAccept to false if there are errors
	if (!bCanAccept)
	{
		// If there isn't anything selected yet, clear things
		PreviewMesh->ClearPreview();
		BrushesToConvert.Empty();
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
	}
	else if (Settings->bShowPreview && CompareAndUpdateConversionTargets())
	{
		FText ErrorMessage;
		bCanAccept = ComputeAndUpdatePreviewMesh(&ErrorMessage);

		// We do this regardless of success because it resets the error message if ErrorMessage is empty.
		GetToolManager()->DisplayMessage(ErrorMessage, EToolMessageLevel::UserWarning);
	}
}

/**
 * Actor movements include any transform changes, and change both the preview and the
 * cached brushes. It shouldn't affect BrushesToConvert or bCanAccept.
 */
void UBspConversionTool::OnEditorActorMoved(AActor* Actor)
{
	// Make sure it was a brush that got moved
	ABrush* Brush = Cast<ABrush>(Actor);
	if (!Brush)
	{
		return;
	}

	// Remove cached version of brush
	if (CachedBrushes.Contains(Brush))
	{
		CachedBrushes.Remove(Brush);
	}

	// Update preview if this is a relevant brush
	if (bCanAccept && Settings->bShowPreview && BrushesToConvert.Contains(Brush))
	{
		FText ErrorMessage;
		bCanAccept = ComputeAndUpdatePreviewMesh();
		GetToolManager()->DisplayMessage(ErrorMessage, EToolMessageLevel::UserWarning);
	}
}

void UBspConversionTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// There are a few properties that are only relevant on accept, and don't change anything now
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBspConversionToolProperties, bRemoveConvertedSubtractiveBrushes)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBspConversionToolProperties, bRemoveConvertedVolumes)))
	{
		return;
	}

	// For the most part, it ends up being too messy to consider all properties individually. It
	// is safer to reset the state (though only clear cached brushes if we have to).

	if (!Settings->bCacheBrushes)
	{
		CachedBrushes.Empty();
	}

	bCanAccept = AtLeastOneValidConversionTarget();

	// Clear preview
	PreviewMesh->ClearPreview();
	BrushesToConvert.Empty();
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning); // This "clears" it

	if (Settings->bShowPreview && bCanAccept)
	{
		CompareAndUpdateConversionTargets();
		FText ErrorMessage;
		bCanAccept = ComputeAndUpdatePreviewMesh(&ErrorMessage);
		GetToolManager()->DisplayMessage(ErrorMessage, EToolMessageLevel::UserWarning);
	}
}

#undef LOCTEXT_NAMESPACE
