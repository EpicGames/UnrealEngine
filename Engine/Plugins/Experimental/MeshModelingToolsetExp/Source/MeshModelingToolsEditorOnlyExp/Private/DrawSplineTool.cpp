// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawSplineTool.h"

#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "AssetSelection.h" // FActorFactoryAssetProxy
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "Components/SplineComponent.h"
#include "CoreGlobals.h" // GUndo
#include "Drawing/PreviewGeometryActor.h"
#include "Editor/UnrealEdEngine.h" // DuplicateActors
#include "Engine/Blueprint.h"
#include "Engine/World.h" 
#include "GameFramework/Actor.h"
#include "ILevelEditor.h"
#include "InputState.h" // FInputDeviceRay
#include "InteractiveToolManager.h"
#include "ISceneOutliner.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Kismet2/ComponentEditorUtils.h" // GenerateValidVariableName
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "Selection/ToolSelectionUtil.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SplineUtil.h"
#include "ToolBuilderUtil.h"
#include "UnrealEdGlobals.h" // GUnrealEd
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "UDynamicMesh.h"
#include "ToolTargetManager.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawSplineTool)

#define LOCTEXT_NAMESPACE "UDrawSplineTool"

using namespace UE::Geometry;

/**
 * Helper to hide actors from the outliner if it's not an actor that
 *  we defined to be automatically hidden (e.g. APreviewGeometryActor),
 *  that works by being friended to FSetActorHiddenInSceneOutliner.
 * This is a temporary measure until we have a cleaner way to hide
 *  ourselves from the outliner through TEDS.
 * Note that just creating this class doesn't actually refresh the outliner
 *  unless you happen to take an action that does (such as reparenting things),
 *  so you would need to call RefreshOutliner().
 */
class FModelingToolsSetActorHiddenInSceneOutliner
{
public:
	FModelingToolsSetActorHiddenInSceneOutliner(AActor* InActor, bool bHidden)
	{
		FSetActorHiddenInSceneOutliner Setter(InActor, bHidden);
	}

	/**
	 * Does a full refresh of the outliner. Note that this can be comparatively
	 *  slow, so it should happen rarely.
	 */
	void RefreshOutliner()
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
		if (!LevelEditorModule)
		{
			return;
		}

		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();
		if (!LevelEditor.IsValid())
		{
			return;
		}

		for (TWeakPtr<ISceneOutliner> OutlinerWeak : LevelEditor->GetAllSceneOutliners())
		{
			if (TSharedPtr<ISceneOutliner> Outliner = OutlinerWeak.Pin())
			{
				Outliner->FullRefresh();
			}
		}
	}
};

namespace DrawSplineToolLocals
{
	FText AddPointTransactionName = LOCTEXT("AddPointTransactionName", "Add Point");

	USplineComponent* CreateNewSplineInActor(AActor* Actor, bool bTransact = false, bool bSetAsRoot = false)
	{
		if (!ensure(Actor))
		{
			return nullptr;
		}

		if (bTransact)
		{
			Actor->Modify();
		}

		FName NewComponentName = *FComponentEditorUtils::GenerateValidVariableName(USplineComponent::StaticClass(), Actor);
		// Note that the RF_Transactional is important here for the spline to undo/redo properly in the future
		USplineComponent* Spline = NewObject<USplineComponent>(Actor, USplineComponent::StaticClass(), NewComponentName, 
			bTransact ? RF_Transactional : RF_NoFlags);

		if (bSetAsRoot)
		{
			Actor->SetRootComponent(Spline);
		}
		else
		{
			Spline->SetupAttachment(Actor->GetRootComponent());
		}

		Spline->OnComponentCreated();
		Actor->AddInstanceComponent(Spline);
		Spline->RegisterComponent();
		Spline->ResetRelativeTransform();
		Actor->PostEditChange();

		return Spline;
	};

	USplineComponent* GetOrCreateTargetSpline(AActor* Actor, int32 TargetIndex, bool bTransact = false)
	{
		if (TargetIndex >= 0)
		{
			TInlineComponentArray<USplineComponent*> SplineComponents;
			Actor->GetComponents<USplineComponent>(SplineComponents);
			if (TargetIndex < SplineComponents.Num())
			{
				return SplineComponents[TargetIndex];
			}
		}

		// If we got to here, we didn't have an existing target at that index. Create one.
		return CreateNewSplineInActor(Actor, bTransact);
	}

	void CopySplineToSpline(const USplineComponent& Source, USplineComponent& Destination, bool bTransact = false)
	{
		if (bTransact)
		{
			Destination.Modify();
		}

		Destination.ClearSplinePoints();
		Destination.bSplineHasBeenEdited = true;

		// We iterate here (rather than just copying over the SplineCurves data) so that we can transform
		// the data properly into the coordinate space of the target component.
		int32 NumSplinePoints = Source.GetNumberOfSplinePoints();
		for (int32 i = 0; i < NumSplinePoints; ++i)
		{
			Destination.AddSplinePoint(Source.GetLocationAtSplinePoint(i, ESplineCoordinateSpace::World),
				ESplineCoordinateSpace::World, false);
			Destination.SetUpVectorAtSplinePoint(i, Source.GetUpVectorAtSplinePoint(i, ESplineCoordinateSpace::World),
				ESplineCoordinateSpace::World, false);
			Destination.SetTangentsAtSplinePoint(i,
				Source.GetArriveTangentAtSplinePoint(i, ESplineCoordinateSpace::World),
				Source.GetLeaveTangentAtSplinePoint(i, ESplineCoordinateSpace::World),
				ESplineCoordinateSpace::World, false);
			Destination.SetSplinePointType(i, Source.GetSplinePointType(i), false);
		}

		Destination.SetClosedLoop(Source.IsClosedLoop());

		Destination.UpdateSpline();
	};

	// Gives the scale used for tangent visualization (and which therefore needs to be used in raycasting the handles)
	float GetNonzeroTangentScale()
	{
		// We'll use the setting that gets used when editing splines in the viewport, but if it is set to 0 (because
		//  the user presumably doesn't want to see tangent handles in viewport), we'll ignore it and use the default
		//  value of 0.5, because we need to be able to draw a tangent for our "tangent drag" approach.
		// The abs here isn't actually necessary since the setting is clamped in editor, but it looks more correct.
		float Scale = FMath::Abs(GetDefault<ULevelEditorViewportSettings>()->SplineTangentScale);
		return Scale > 0.0f ? Scale : 0.5f;
	}

	// Might be useful to have in SplineUtil, but uncertain what the API should be (should it be part of
	// DrawSpline? Should there be options for selection color?). Also potentially messier to match the tangent
	// scale with the UI interaction..
	void DrawTangent(const USplineComponent& SplineComp, int32 PointIndex, IToolsContextRenderAPI& RenderAPI)
	{
		if (!ensure(PointIndex >= 0 && PointIndex < SplineComp.GetNumberOfSplinePoints()))
		{
			return;
		}

		FPrimitiveDrawInterface* PDI = RenderAPI.GetPrimitiveDrawInterface();

		const float TangentScale = GetNonzeroTangentScale();
		const float TangentHandleSize = 8.0f + GetDefault<ULevelEditorViewportSettings>()->SplineTangentHandleSizeAdjustment;

		const FVector Location = SplineComp.GetLocationAtSplinePoint(PointIndex, ESplineCoordinateSpace::World);
		const FVector LeaveTangent = SplineComp.GetLeaveTangentAtSplinePoint(PointIndex, ESplineCoordinateSpace::World) * TangentScale;
		const FVector ArriveTangent = SplineComp.bAllowDiscontinuousSpline ?
			SplineComp.GetArriveTangentAtSplinePoint(PointIndex, ESplineCoordinateSpace::World) * TangentScale : LeaveTangent;

		FColor Color = FColor::White;

		PDI->DrawLine(Location, Location - ArriveTangent, Color, SDPG_Foreground);
		PDI->DrawLine(Location, Location + LeaveTangent, Color, SDPG_Foreground);

		PDI->DrawPoint(Location + LeaveTangent, Color, TangentHandleSize, SDPG_Foreground);
		PDI->DrawPoint(Location - ArriveTangent, Color, TangentHandleSize, SDPG_Foreground);
	}

	// Undoes a point addition with an auto tangent
	class FSimplePointInsertionChange : public UDrawSplineTool::FSplineChange
	{
	public:
		FSimplePointInsertionChange(const FVector3d& HitLocationIn, const FVector3d& UpVectorIn)
			: HitLocation(HitLocationIn)
			, UpVector(UpVectorIn)
		{
		}

		virtual void Apply(USplineComponent& Spline) override
		{
			Spline.AddSplinePoint(HitLocation, ESplineCoordinateSpace::World, false);
			int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
			Spline.SetUpVectorAtSplinePoint(PointIndex, UpVector, ESplineCoordinateSpace::World, true);
		}
		virtual void Revert(USplineComponent& Spline) override
		{
			if (ensure(Spline.GetNumberOfSplinePoints() > 0))
			{
				Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, true);
			}
		}
		virtual FString ToString() const override
		{
			return TEXT("FSimplePointInsertionChange");
		}

	protected:
		FVector3d HitLocation;
		FVector3d UpVector;
	};

	// Undoes a point addition with an explicit tangent
	class FTangentPointInsertionChange : public UDrawSplineTool::FSplineChange
	{
	public:
		FTangentPointInsertionChange(const FVector3d& HitLocationIn, const FVector3d& UpVectorIn, const FVector3d& TangentIn)
			: HitLocation(HitLocationIn)
			, UpVector(UpVectorIn)
			, Tangent(TangentIn)
		{
		}

		virtual void Apply(USplineComponent& Spline) override
		{
			Spline.AddSplinePoint(HitLocation, ESplineCoordinateSpace::World, false);
			int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
			Spline.SetUpVectorAtSplinePoint(PointIndex, UpVector, ESplineCoordinateSpace::World, false);
			Spline.SetTangentAtSplinePoint(PointIndex, Tangent, ESplineCoordinateSpace::World, true);
		}
		virtual void Revert(USplineComponent& Spline) override
		{
			if (ensure(Spline.GetNumberOfSplinePoints() > 0))
			{
				Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, true);
			}
		}
		virtual FString ToString() const override
		{
			return TEXT("FTangentPointInsertionChange");
		}

	protected:
		FVector3d HitLocation;
		FVector3d UpVector;
		FVector3d Tangent;
	};

	// Undoes a free draw stroke (multiple points at once)
	class FStrokeInsertionChange : public UDrawSplineTool::FSplineChange
	{
	public:
		FStrokeInsertionChange(const TArray<FVector3d>& HitLocationsIn, const TArray<FVector3d>& UpVectorsIn)
			: HitLocations(HitLocationsIn)
			, UpVectors(UpVectorsIn)
		{
			if (!ensure(HitLocations.Num() == UpVectors.Num()))
			{
				int32 Num = FMath::Min(HitLocations.Num(), UpVectors.Num());
				HitLocations.SetNum(Num);
				UpVectors.SetNum(Num);
			}
		}

		virtual void Apply(USplineComponent& Spline) override
		{
			for (int32 i = 0; i < HitLocations.Num(); ++i)
			{
				Spline.AddSplinePoint(HitLocations[i], ESplineCoordinateSpace::World, false);
				int32 PointIndex = Spline.GetNumberOfSplinePoints() - 1;
				Spline.SetUpVectorAtSplinePoint(PointIndex, UpVectors[i], ESplineCoordinateSpace::World, false);
			}
			Spline.UpdateSpline();
		}
		virtual void Revert(USplineComponent& Spline) override
		{
			for (int32 i = 0; i < HitLocations.Num(); ++i)
			{
				if (!ensure(Spline.GetNumberOfSplinePoints() > 0))
				{
					break;
				}
				Spline.RemoveSplinePoint(Spline.GetNumberOfSplinePoints() - 1, false);
			}
			Spline.UpdateSpline();
		}
		virtual FString ToString() const override
		{
			return TEXT("FStrokeInsertionChange");
		}

	protected:
		TArray<FVector3d> HitLocations;
		TArray<FVector3d> UpVectors;
	};
}

void UDrawSplineTool::Setup()
{
	UInteractiveTool::Setup();

	Settings = NewObject<UDrawSplineToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->TargetActor = StartupSelectedActor;

	SetToolDisplayName(LOCTEXT("DrawSplineToolName", "Draw Spline"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("DrawSplineToolDescription", "Draw a spline to replace an existing one or add it to an actor."),
		EToolMessageLevel::UserNotification);

	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), FFrame3d(FVector3d::Zero(), FVector3d::UnitX()));
	PlaneMechanic->bShowGrid = Settings->bHitCustomPlane;
	PlaneMechanic->CanUpdatePlaneFunc = [this] { return Settings->bHitCustomPlane; };
	Settings->WatchProperty(Settings->bHitCustomPlane, [this](bool) {
		PlaneMechanic->bShowGrid = Settings->bHitCustomPlane;
	});

	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, this);
	AddInputBehavior(ClickOrDragBehavior);

	// Make sure the plane mechanic captures clicks first, to ensure it sees ctrl+clicks to reposition the plane
	PlaneMechanic->UpdateClickPriority(ClickOrDragBehavior->GetPriority().MakeHigher());

	Settings->WatchProperty(Settings->bLoop, [this](bool) {
		if (ensure(WorkingSpline.IsValid()))
		{
			WorkingSpline->SetClosedLoop(Settings->bLoop);
			bNeedToRerunConstructionScript = true;
		}
	});

	TransitionOutputMode();

	Settings->WatchProperty(Settings->OutputMode, [this](EDrawSplineOutputMode) {
		TransitionOutputMode();
	});
	TargetActorWatcherID = Settings->WatchProperty(Settings->TargetActor, [this](TWeakObjectPtr<AActor>) {
		// It's possible for the user to use the actor picker to click on our preview, which we don't
		//  want to be pickable via the actor picker... There doesn't currently seem to be a way prevent
		//  that, so for now we'll just catch this case and keep whatever the previous value was.
		if (Settings->TargetActor == PreviewActor)
		{
			Settings->TargetActor = IsValid(PreviousTargetActor) ? PreviousTargetActor : nullptr;
			Settings->SilentUpdateWatcherAtIndex(TargetActorWatcherID);
		}

		if (PreviousTargetActor != Settings->TargetActor)
		{
			// Don't set PreviousTargetActor here because it needs to be made visible, etc inside
			//  TransitionOutputMode
			
			TransitionOutputMode();
		}
	});
	Settings->WatchProperty(Settings->ExistingSplineIndexToReplace, [this](int32) {
		TransitionOutputMode();
	});
	Settings->WatchProperty(Settings->BlueprintToCreate, [this](TWeakObjectPtr<UBlueprint>) {
		TransitionOutputMode();
		});
	Settings->WatchProperty(Settings->bPreviewUsingActorCopy, [this](bool) {
		TransitionOutputMode();
	});

	Settings->WatchProperty(Settings->bFitToSurface, [this](bool) 
	{
		RefitToSurface();
	});

	Settings->WatchProperty(Settings->MaxNewPoints, [this](int32)
	{
		RefitToSurface();
	});

	Settings->WatchProperty(Settings->DistanceThreshold, [this](double) 
	{
		RefitToSurface();
	});

	Settings->SilentUpdateWatched();

	if (Target)
	{
		if (IPersistentDynamicMeshSource* const DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target))
		{
			UDynamicMesh* const MeshContainer = DynamicMeshSource->GetDynamicMeshContainer();
			ProjectionTarget = MakeUnique<UE::Geometry::FDynamicMeshAABBTree3>(MeshContainer->GetMeshPtr(), true);
		}
		else if (IDynamicMeshProvider* const DynamicMeshProvider = Cast<IDynamicMeshProvider>(Target))
		{
			ProjectionTargetMesh = MakeUnique<UE::Geometry::FDynamicMesh3>();
			ProjectionTargetMesh->Copy(DynamicMeshProvider->GetDynamicMesh());
			ProjectionTarget = MakeUnique<UE::Geometry::FDynamicMeshAABBTree3>(ProjectionTargetMesh.Get(), true);
		}

		if (ISceneComponentBackedTarget* const PrimitiveToolTarget = Cast<ISceneComponentBackedTarget>(Target))
		{
			MeshTargetTransform = PrimitiveToolTarget->GetWorldTransform();
		}
		else
		{
			MeshTargetTransform = FTransform();
		}		

		Settings->bMeshSelected = true;
	}
	else
	{
		Settings->bMeshSelected = false;
	}
}

// Set things up for a new output mode or destination
void UDrawSplineTool::TransitionOutputMode()
{
	using namespace DrawSplineToolLocals;

	// Setting up the previews seems to be the most error prone part of the tool because editor duplicating, hiding
	// from outliner, and avoiding emitting undo/redo transactions seems to be quite finnicky...

	// This function is sometimes called from inside transactions (such as tool start, or dragging the "component to replace"
	// slider). Several calls here would transact in that case (for instance, the Destroy() calls on the previews seem
	// to do it), which we generally don't want to do. So we disable transacting in this function with the hack below.
	// Note that we still have to take care that any editor functions we call don't open their own transactions...
	ITransaction* UndoState = GUndo;
	GUndo = nullptr; // Pretend we're not in a transaction
	ON_SCOPE_EXIT{ GUndo = UndoState; }; // Revert later

	// Restore the visibility of the previous target actor and spline, if needed
	if (PreviousTargetActor)
	{
		PreviousTargetActor->GetRootComponent()->SetVisibility(true, true);
		PreviousTargetActor = nullptr;
	}
	if (HiddenSpline.IsValid())
	{
		HiddenSpline->bDrawDebug = bPreviousSplineVisibility;
		HiddenSpline = nullptr;
	}

	// Keep the previous spline/preview temporarily so we can transfer over spline data
	// when we make new previews
	AActor* PreviousPreview = PreviewActor;
	USplineComponent* PreviousSpline = WorkingSpline.Get();

	PreviewActor = nullptr;
	WorkingSpline = nullptr;
	
	// Used for visualizing the effect of a spline on some special actor
	auto CreateDuplicatePreviewActor = [this](AActor* Actor)
	{
		TArray<AActor*> NewActors;
		GUnrealEd->DuplicateActors({ Actor }, NewActors, GetWorld()->GetCurrentLevel(), FVector3d::Zero());
		if (!ensure(NewActors.Num() > 0))
		{
			return false;
		}

		PreviewActor = NewActors[0];
		PreviewActor->ClearFlags(RF_Transactional);
		PreviewActor->SetFlags(RF_Transient);

		// Make the original invisible
		PreviousTargetActor = Actor;
		PreviousTargetActor->GetRootComponent()->SetVisibility(false, true);

		// Note: unfortunately this won't hide the spline itself as long as the object is selected because that
		//  drawing goes through a different path that doesn't seem to be disableable (unless we deselected
		//  the actor).
		
		// Hide this preview from the outliner
		FModelingToolsSetActorHiddenInSceneOutliner Hider(PreviewActor, true);
		Hider.RefreshOutliner();

		return true;
	};

	auto FallbackSplinePlacement = [this]()
	{
		FRotator Rotation(0.0f, 0.0f, 0.0f);
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.ObjectFlags = RF_Transient;
		PreviewActor = GetTargetWorld()->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);
		USceneComponent* RootComponent = NewObject<USceneComponent>(PreviewActor);
		PreviewActor->AddOwnedComponent(RootComponent);
		PreviewActor->SetRootComponent(RootComponent);
		RootComponent->RegisterComponent();
		
		WorkingSpline = CreateNewSplineInActor(PreviewActor);
	};

	// Set up the new preview
	if (!Settings->bPreviewUsingActorCopy)
	{
		FallbackSplinePlacement();
	}
	else
	{
		switch (Settings->OutputMode)
		{
		case EDrawSplineOutputMode::EmptyActor:
			FallbackSplinePlacement();
			break;
		case EDrawSplineOutputMode::ExistingActor:
		{
			if (!Settings->TargetActor.IsValid()
				|| !CreateDuplicatePreviewActor(Settings->TargetActor.Get()))
			{
				FallbackSplinePlacement();
				break;
			}

			WorkingSpline = GetOrCreateTargetSpline(PreviewActor, Settings->ExistingSplineIndexToReplace);
			bNeedToRerunConstructionScript = true;
			break;
		}
		case EDrawSplineOutputMode::CreateBlueprint:
		{
			bool bCanCreateActor = Settings->BlueprintToCreate.IsValid()
				&& Settings->BlueprintToCreate->GeneratedClass != nullptr
				&& !Settings->BlueprintToCreate->GeneratedClass->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Abstract);

			if (!bCanCreateActor)
			{
				FallbackSplinePlacement();
				break;
			}

			// Instantiate the blueprint
			PreviewActor = FActorFactoryAssetProxy::AddActorForAsset(
				Settings->BlueprintToCreate.Get(),
				/*bSelectActors =*/ false,
				// Important that we don't use the default (RF_Transactional) here, or else we'll end up
				// issuing an undo transaction in this call.
				EObjectFlags::RF_Transient);
			if (!PreviewActor)
			{
				FallbackSplinePlacement();
				break;
			}

			// Hide this preview from outliner
			FModelingToolsSetActorHiddenInSceneOutliner Hider(PreviewActor, true);
			Hider.RefreshOutliner();

			WorkingSpline = GetOrCreateTargetSpline(PreviewActor, Settings->ExistingSplineIndexToReplace);
			bNeedToRerunConstructionScript = true;
			break;
		}
		default:
			ensure(false);
		}
	}

	if (ensure(WorkingSpline.IsValid()))
	{
		if (PreviousSpline)
		{
			CopySplineToSpline(*PreviousSpline, *WorkingSpline);
		}
		else
		{
			WorkingSpline->ClearSplinePoints();
		}

		WorkingSpline->SetClosedLoop(Settings->bLoop);

		// This has to be set so that construction script reruns transfer over current spline state.
		WorkingSpline->bSplineHasBeenEdited = true;

		// Get the index of the spline in the components array for recapturing on construction script reruns.
		if (PreviewActor)
		{
			TInlineComponentArray<USplineComponent*> SplineComponents;
			PreviewActor->GetComponents<USplineComponent>(SplineComponents);
			SplineRecaptureIndex = SplineComponents.IndexOfByKey(WorkingSpline.Get());
			ensure(SplineRecaptureIndex >= 0);
		}
	}
	
	// Now that we've copied over previous preview data, destroy the old previews
	if (PreviousPreview)
	{
		PreviousPreview->Destroy();
	}
}

void UDrawSplineTool::Shutdown(EToolShutdownType ShutdownType)
{
	using namespace DrawSplineToolLocals;

	LongTransactions.CloseAll(GetToolManager());

	Settings->SaveProperties(this);

	if (PreviousTargetActor)
	{
		PreviousTargetActor->GetRootComponent()->SetVisibility(true, true);
		PreviousTargetActor = nullptr;
	}
	if (HiddenSpline.IsValid())
	{
		HiddenSpline->bDrawDebug = bPreviousSplineVisibility;
		HiddenSpline = nullptr;
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		if (Settings->bFitToSurface && ProjectionSpline.IsValid())
		{
			WorkingSpline->SetSpline(ProjectionSpline->GetSpline());
		}

		if (WorkingSpline.IsValid() && WorkingSpline->GetNumberOfSplinePoints() > 0)
		{
			GenerateAsset();
		}
	}

	PlaneMechanic->Shutdown();
	
	
	if (WorkingSpline.IsValid())
	{
		WorkingSpline->DestroyComponent();
	}

	if (ProjectionSpline.IsValid())
	{
		ProjectionSpline->DestroyComponent();
	}

	if (PreviewActor)
	{
		PreviewActor->Destroy();
	}

	Super::Shutdown(ShutdownType);
}

void UDrawSplineTool::GenerateAsset()
{
	using namespace DrawSplineToolLocals;
	USplineComponent* OutputSpline = nullptr;
	auto CreateSplineInEmptyActor = [this, &OutputSpline]()
	{
		// Get centroid of spline
		int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
		FVector3d Center = FVector3d::Zero();
		for (int32 i = 0; i < NumSplinePoints; ++i)
		{
			Center += WorkingSpline->GetLocationAtSplinePoint(i, ESplineCoordinateSpace::World);
		}
		Center /= NumSplinePoints;

		// Spawning via a factory is editor-only
		UActorFactoryEmptyActor* EmptyActorFactory = NewObject<UActorFactoryEmptyActor>();
		FAssetData AssetData(EmptyActorFactory->GetDefaultActorClass(FAssetData()));
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = TEXT("Spline");
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		AActor* NewActor = EmptyActorFactory->CreateActor(AssetData.GetAsset(),
			TargetWorld->GetCurrentLevel(),
			FTransform(Center),
			SpawnParams);

		// This is also editor-only: it's the label that shows up in the hierarchy
		FActorLabelUtilities::SetActorLabelUnique(NewActor, TEXT("SplineActor"));

		UActorComponent* OldRoot = NewActor->GetRootComponent();
		OutputSpline = CreateNewSplineInActor(NewActor, true, true);
		OutputSpline->SetWorldTransform(FTransform(Center));
		OldRoot->DestroyComponent();

		CopySplineToSpline(*WorkingSpline, *OutputSpline, true);
	};

	GetToolManager()->BeginUndoTransaction(LOCTEXT("DrawSplineTransactionName", "Draw Spline"));

	switch (Settings->OutputMode)
	{
	case EDrawSplineOutputMode::EmptyActor:
		CreateSplineInEmptyActor();
		break;
	case EDrawSplineOutputMode::ExistingActor:
	{
		if (!Settings->TargetActor.IsValid())
		{
			CreateSplineInEmptyActor();
			break;
		}

		OutputSpline = GetOrCreateTargetSpline(Settings->TargetActor.Get(), Settings->ExistingSplineIndexToReplace, true);
		CopySplineToSpline(*WorkingSpline, *OutputSpline, true);
		Settings->TargetActor->RerunConstructionScripts();
		break;
	}
	case EDrawSplineOutputMode::CreateBlueprint:
	{
		bool bCanCreateActor = Settings->BlueprintToCreate.IsValid()// != nullptr
			&& Settings->BlueprintToCreate->GeneratedClass != nullptr
			&& !Settings->BlueprintToCreate->GeneratedClass->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Abstract);

		if (!bCanCreateActor)
		{
			CreateSplineInEmptyActor();
			break;
		}

		// Instantiate the blueprint
		AActor* NewActor = FActorFactoryAssetProxy::AddActorForAsset(
			Settings->BlueprintToCreate.Get(),
			/*bSelectActors =*/ false);
		if (!NewActor)
		{
			CreateSplineInEmptyActor();
			break;
		}

		OutputSpline = GetOrCreateTargetSpline(NewActor, Settings->ExistingSplineIndexToReplace, true);
		CopySplineToSpline(*WorkingSpline, *OutputSpline, true);
		NewActor->RerunConstructionScripts();
		break;
	}
	default:
		ensure(false);
	}

	// TODO: Someday when we support component selection, we should select OutputSpline directly.
	ToolSelectionUtil::SetNewActorSelection(GetToolManager(), OutputSpline->GetAttachmentRootActor());

	GetToolManager()->EndUndoTransaction();
}

void UDrawSplineTool::RefitToSurface()
{
	if (Settings->bFitToSurface && ProjectionTarget)
	{
		const FInterpCurveVector& Curve = WorkingSpline->GetSplinePointsPosition();
		FInterpCurveVector NewCurve;
		SplineUtil::ProjectSplineToSurface(NewCurve, Curve, *ProjectionTarget, WorkingSpline->GetComponentTransform(), MeshTargetTransform, Settings->DistanceThreshold, Settings->MaxNewPoints);

		if (!ProjectionSpline.IsValid())
		{
			ProjectionSpline = DrawSplineToolLocals::CreateNewSplineInActor(PreviewActor);
		}

		TArray<FVector> ProjectedPoints;
		for (FInterpCurvePointVector NewPoint : NewCurve.Points)
		{
			ProjectedPoints.Add(NewPoint.OutVal);
		}

		ProjectionSpline->SetSplinePoints(ProjectedPoints, ESplineCoordinateSpace::World, /*bUpdateSpline = */ true);
	}
	else if (ProjectionSpline.IsValid())
	{
		ProjectionSpline->DestroyComponent();
		ProjectionSpline.Reset();
	}
}


// Helper to add a point given a hit location and hit normal
void UDrawSplineTool::AddSplinePoint(const FVector3d& HitLocation, const FVector3d& HitNormal)
{
	using namespace DrawSplineToolLocals;
	if (!WorkingSpline.IsValid())
	{
		return;
	}

	int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	FVector3d UpVectorToUse = GetUpVectorToUse(HitLocation, HitNormal, NumSplinePoints);

	WorkingSpline->AddSplinePoint(HitLocation, ESplineCoordinateSpace::World, 
		/*bUpdate =*/ false);
	WorkingSpline->SetUpVectorAtSplinePoint(NumSplinePoints, UpVectorToUse, ESplineCoordinateSpace::World, 
		/*bUpdate =*/ true);

	RefitToSurface();

}

FVector3d UDrawSplineTool::GetUpVectorToUse(const FVector3d& HitLocation, const FVector3d& HitNormal, int32 NumSplinePointsBeforehand)
{
	FVector3d UpVectorToUse = HitNormal;
	switch (Settings->UpVectorMode)
	{
	case EDrawSplineUpVectorMode::AlignToPrevious:
	{
		if (NumSplinePointsBeforehand == 0)
		{
			// TODO: Maybe add some different options of what normal to start with
		}
		else if (NumSplinePointsBeforehand > 1)
		{
			UpVectorToUse = WorkingSpline->GetUpVectorAtSplinePoint(NumSplinePointsBeforehand - 1, ESplineCoordinateSpace::World);
		}
		else // if NumSplinePointsBeforehand == 1
		{
			// If there's only one point, GetUpVectorAtSplinePoint is unreliable because it seeks to build a
			// quaternion from the tangent and the set up vector, and the tangent is zero. We want to use
			// the "stored" up vector directly.
			FVector3d LocalUpVector = WorkingSpline->GetRotationAtSplinePoint(0, ESplineCoordinateSpace::Local).RotateVector(WorkingSpline->DefaultUpVector);
			UpVectorToUse = WorkingSpline->GetComponentTransform().TransformVectorNoScale(LocalUpVector);
		}
		break;
	}
	case EDrawSplineUpVectorMode::UseHitNormal:
		break;
	}

	return UpVectorToUse;
}

bool UDrawSplineTool::Raycast(const FRay& WorldRay, FVector3d& HitLocationOut, FVector3d& HitNormalOut, double& HitTOut)
{
	double BestHitT = TNumericLimits<double>::Max();
	
	if (Settings->bHitCustomPlane)
	{
		FVector IntersectionPoint;
		bool bHitPlane = false;
		GizmoMath::RayPlaneIntersectionPoint(PlaneMechanic->Plane.Origin, PlaneMechanic->Plane.Z(),
			WorldRay.Origin, WorldRay.Direction, bHitPlane, IntersectionPoint);

		if (bHitPlane)
		{
			HitLocationOut = IntersectionPoint;
			HitNormalOut = PlaneMechanic->Plane.Z();
			HitTOut = WorldRay.GetParameter(IntersectionPoint);
			BestHitT = HitTOut;
		}
	}

	if (Settings->bHitWorld)
	{
		FHitResult GeometryHit;
		TArray<const UPrimitiveComponent*> ComponentsToIgnore;
		if (PreviewActor)
		{
			PreviewActor->GetComponents<const UPrimitiveComponent>(ComponentsToIgnore);
		}
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, GeometryHit, WorldRay, &ComponentsToIgnore)
			&& GeometryHit.Distance < BestHitT)
		{
			HitLocationOut = GeometryHit.ImpactPoint;
			HitNormalOut = GeometryHit.ImpactNormal;
			HitTOut = GeometryHit.Distance;
			BestHitT = HitTOut;
		}
	}

	// Only raycast the ground plane / ortho background if we didn't hit anything else
	if (Settings->bHitGroundPlanes && BestHitT == TNumericLimits<double>::Max())
	{
		FVector3d PlaneNormal = CameraState.bIsOrthographic ? -WorldRay.Direction : FVector3d::UnitZ();
		FVector IntersectionPoint;
		bool bHitPlane = false;
		GizmoMath::RayPlaneIntersectionPoint(FVector3d::Zero(), PlaneNormal,
			WorldRay.Origin, WorldRay.Direction, bHitPlane, IntersectionPoint);

		if (bHitPlane)
		{
			HitLocationOut = IntersectionPoint;
			HitNormalOut = PlaneNormal;
			HitTOut = WorldRay.GetParameter(IntersectionPoint);
			BestHitT = HitTOut;
		}
	}

	if (Settings->ClickOffset != 0.0)
	{
		FVector3d OffsetDirection = HitNormalOut;
		if (Settings->OffsetMethod == ESplineOffsetMethod::Custom)
		{
			OffsetDirection = Settings->OffsetDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3d::UnitZ());
		}

		HitLocationOut += OffsetDirection * Settings->ClickOffset;
	}
	
	return BestHitT < TNumericLimits<double>::Max();
}

FInputRayHit UDrawSplineTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FVector3d HitLocation, HitNormal;
	double HitT;
	if (Raycast(ClickPos.WorldRay, HitLocation, HitNormal, HitT))
	{
		return FInputRayHit(HitT);
	}
	return FInputRayHit();
}

void UDrawSplineTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	using namespace DrawSplineToolLocals;

	FVector3d HitLocation, HitNormal;
	double HitT;
	if (Raycast(ClickPos.WorldRay, HitLocation, HitNormal, HitT))
	{
		switch (Settings->DrawMode)
		{
		case EDrawSplineDrawMode::ClickAutoTangent:
		case EDrawSplineDrawMode::FreeDraw:
		{
			AddSplinePoint(HitLocation, HitNormal);

			int32 PointIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;
			GetToolManager()->EmitObjectChange(this,
				MakeUnique<FSimplePointInsertionChange>(
					HitLocation, 
					WorkingSpline->GetUpVectorAtSplinePoint(PointIndex, ESplineCoordinateSpace::World)),
				AddPointTransactionName);
			break;
		}
		case EDrawSplineDrawMode::TangentDrag:
		{
			AddSplinePoint(HitLocation, HitNormal);

			int32 PointIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;
			WorkingSpline->SetTangentAtSplinePoint(PointIndex,
				FVector3d::Zero(),
				ESplineCoordinateSpace::World, true);

			GetToolManager()->EmitObjectChange(this,
				MakeUnique<FTangentPointInsertionChange>( 
					HitLocation, 
					WorkingSpline->GetUpVectorAtSplinePoint(PointIndex, ESplineCoordinateSpace::World), 
					FVector3d::Zero()),
				AddPointTransactionName);
			break;
		}
		}

		bNeedToRerunConstructionScript = true;
	}
}

FInputRayHit UDrawSplineTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FVector3d HitLocation, HitNormal;
	double HitT;
	if (Raycast(PressPos.WorldRay, HitLocation, HitNormal, HitT))
	{
		return FInputRayHit(HitT);
	}
	return FInputRayHit();
}

void UDrawSplineTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	FVector3d HitLocation, HitNormal;
	double HitT;

	LongTransactions.Open(DrawSplineToolLocals::AddPointTransactionName, GetToolManager());

	// Regardless of DrawMode, start by placing a point, though don't emit a transaction until mouse up
	if (ensure(Raycast(PressPos.WorldRay, HitLocation, HitNormal, HitT)))
	{
		AddSplinePoint(HitLocation, HitNormal);

		if (Settings->DrawMode == EDrawSplineDrawMode::FreeDraw)
		{
			// Remember which point started this stroke
			FreeDrawStrokeStartIndex = WorkingSpline->GetNumberOfSplinePoints() - 1;
		}
		
		bNeedToRerunConstructionScript = bNeedToRerunConstructionScript || Settings->bRerunConstructionScriptOnDrag;
	}
}

void UDrawSplineTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	using namespace DrawSplineToolLocals;

	int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();
	if (!ensure(NumSplinePoints > 0))
	{
		return;
	}

	FVector3d HitLocation, HitNormal;
	double HitT;
	if (Raycast(DragPos.WorldRay, HitLocation, HitNormal, HitT))
	{
		switch (Settings->DrawMode)
		{
		case EDrawSplineDrawMode::ClickAutoTangent:
		{
			// Drag the last placed point
			FVector3d UpVector = GetUpVectorToUse(HitLocation, HitNormal, NumSplinePoints);
			WorkingSpline->SetLocationAtSplinePoint(NumSplinePoints - 1, HitLocation, ESplineCoordinateSpace::World, false);
			WorkingSpline->SetUpVectorAtSplinePoint(NumSplinePoints - 1, UpVector, ESplineCoordinateSpace::World, true);

			break;
		}
		case EDrawSplineDrawMode::TangentDrag:
		{
			// Set the tangent
			FVector3d LastPoint = WorkingSpline->GetLocationAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World);
			FVector3d Tangent = (HitLocation - LastPoint) / GetNonzeroTangentScale();
			WorkingSpline->SetTangentAtSplinePoint(NumSplinePoints - 1, Tangent, ESplineCoordinateSpace::World, true);
			bDrawTangentForLastPoint = true;
			break;
		}
			
		case EDrawSplineDrawMode::FreeDraw:
		{
			// Instead of dragging the first placed point (which gets placed in OnClickPress), we drag a second "preview" one
			// until we get far enough from the previous to where we want to place a new control point.
			if (!bFreeDrawPlacedPreviewPoint)
			{
				AddSplinePoint(HitLocation, HitNormal);
				bFreeDrawPlacedPreviewPoint = true;
			}
			else
			{
				FVector3d PreviousPoint = WorkingSpline->GetLocationAtSplinePoint(NumSplinePoints - 2, ESplineCoordinateSpace::World);
				if (FVector3d::DistSquared(HitLocation, PreviousPoint) >= Settings->MinPointSpacing * Settings->MinPointSpacing)
				{
					AddSplinePoint(HitLocation, HitNormal);
				}
				else
				{
					// Drag the preview point
					FVector3d UpVector = GetUpVectorToUse(HitLocation, HitNormal, NumSplinePoints);
					WorkingSpline->SetLocationAtSplinePoint(NumSplinePoints - 1, HitLocation, ESplineCoordinateSpace::World, false);
					WorkingSpline->SetUpVectorAtSplinePoint(NumSplinePoints - 1, UpVector, ESplineCoordinateSpace::World, true);
				}
			}
			break;
		}
		}
	}

	bNeedToRerunConstructionScript = bNeedToRerunConstructionScript || Settings->bRerunConstructionScriptOnDrag;
}

void UDrawSplineTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnClickDrag(ReleasePos);
	OnTerminateDragSequence();
}
void UDrawSplineTool::OnTerminateDragSequence()
{
	using namespace DrawSplineToolLocals;

	bDrawTangentForLastPoint = false;
	bFreeDrawPlacedPreviewPoint = false;
	bNeedToRerunConstructionScript = true;

	int32 NumSplinePoints = WorkingSpline->GetNumberOfSplinePoints();

	// Emit the appropriate undo transaction
	switch (Settings->DrawMode)
	{
	case EDrawSplineDrawMode::ClickAutoTangent:
	{
		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FSimplePointInsertionChange>(
				WorkingSpline->GetLocationAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World),
				WorkingSpline->GetUpVectorAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World)),
			AddPointTransactionName);
		break;
	}
	case EDrawSplineDrawMode::TangentDrag:
	{
		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FTangentPointInsertionChange>( 
				WorkingSpline->GetLocationAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World),
				WorkingSpline->GetUpVectorAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World),
				WorkingSpline->GetTangentAtSplinePoint(NumSplinePoints - 1, ESplineCoordinateSpace::World)),
			AddPointTransactionName);
		break;
	}

	case EDrawSplineDrawMode::FreeDraw:
	{
		TArray<FVector3d> HitLocations;
		TArray<FVector3d> UpVectors;
		for (int32 i = FreeDrawStrokeStartIndex; i < NumSplinePoints; ++i)
		{
			HitLocations.Add(WorkingSpline->GetLocationAtSplinePoint(i, ESplineCoordinateSpace::World));
			UpVectors.Add(WorkingSpline->GetUpVectorAtSplinePoint(i, ESplineCoordinateSpace::World));
		}

		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FStrokeInsertionChange>(HitLocations, UpVectors),
			AddPointTransactionName);
		break;
	}
	}

	LongTransactions.Close(GetToolManager());
}


void UDrawSplineTool::OnTick(float DeltaTime)
{
	if (PlaneMechanic)
	{
		PlaneMechanic->Tick(DeltaTime);
	}

	// check if we've invalidated the WorkingSpline
	if (PreviewActor && !WorkingSpline.IsValid())
	{
		bNeedToRerunConstructionScript = true;
	}

	if (bNeedToRerunConstructionScript)
	{
		bNeedToRerunConstructionScript = false;
		if (PreviewActor)
		{
			PreviewActor->RerunConstructionScripts();

			// Rerunning the construction script can make us lose our reference to the spline, so try to
			// recapture.
			// TODO: This might be avoidable with FComponentReference?
			if (!WorkingSpline.IsValid())
			{
				TInlineComponentArray<USplineComponent*> SplineComponents;
				PreviewActor->GetComponents<USplineComponent>(SplineComponents);

				if (ensure(SplineRecaptureIndex >= 0 && SplineRecaptureIndex < SplineComponents.Num()))
				{
					WorkingSpline = SplineComponents[SplineRecaptureIndex];
				}
				else
				{
					// If we failed to recapture, it's not clear what to do. We can switch to working inside
					// an empty actor, though we'll lose current spline state.
					Settings->OutputMode = EDrawSplineOutputMode::EmptyActor;
					Settings->CheckAndUpdateWatched();
				}
			}
		}
	}

	if (!WorkingSpline.IsValid())
	{
		GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel, true, LOCTEXT("LostWorkingSpline", "The Draw Spline tool must close because the in-progress spline has been unexpectedly deleted."));
	}
}

void UDrawSplineTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace DrawSplineToolLocals;

	Super::Render(RenderAPI);

	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (PlaneMechanic)
	{
		PlaneMechanic->Render(RenderAPI);
	}

	if (!Settings->bFitToSurface && WorkingSpline.IsValid())
	{
		if (bDrawTangentForLastPoint)
		{
			DrawTangent(*WorkingSpline, WorkingSpline->GetNumberOfSplinePoints() - 1, *RenderAPI);
		}

		SplineUtil::FDrawSplineSettings DrawSettings;
		DrawSettings.ScaleVisualizationWidth = Settings->FrameVisualizationWidth;
		SplineUtil::DrawSpline(*WorkingSpline, *RenderAPI, DrawSettings);
	}

	if (Settings->bFitToSurface && ProjectionSpline.IsValid())
	{
		SplineUtil::FDrawSplineSettings DrawSettings;
		DrawSettings.ScaleVisualizationWidth = Settings->FrameVisualizationWidth;
		DrawSettings.RegularColor = FColor::Cyan;
		SplineUtil::DrawSpline(*ProjectionSpline, *RenderAPI, DrawSettings);
	}

}

void UDrawSplineTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
}

bool UDrawSplineTool::CanAccept() const
{
	return WorkingSpline.IsValid() && WorkingSpline->GetNumberOfSplinePoints() > 0;
}

// To be called by builder
void UDrawSplineTool::SetSelectedActor(AActor* Actor)
{
	StartupSelectedActor = Actor;
}
void UDrawSplineTool::SetWorld(UWorld* World)
{
	TargetWorld = World;
}

/// Tool builder:

bool UDrawSplineToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

USingleSelectionMeshEditingTool* UDrawSplineToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UDrawSplineTool>(SceneState.ToolManager);
}

void UDrawSplineToolBuilder::InitializeNewTool(USingleSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const
{
	// Same as USingleSelectionMeshEditingToolBuilder::InitializeNewTool but allow for null Target
	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	Tool->SetTarget(Target);
	Tool->SetWorld(SceneState.World);


	if (UDrawSplineTool* const SplineTool = Cast<UDrawSplineTool>(Tool))
	{
		// May be null
		SplineTool->SetSelectedActor(ToolBuilderUtil::FindFirstActor(SceneState, [](AActor*) { return true; }));
	}
}

void UDrawSplineTool::FSplineChange::Apply(UObject* Object)
{
	UDrawSplineTool* Tool = Cast<UDrawSplineTool>(Object);
	if (!ensure(Tool))
	{
		return;
	}
	TWeakObjectPtr<USplineComponent> Spline = Tool->WorkingSpline;
	if (!ensure(Spline.IsValid()))
	{
		return;
	}

	Apply(*Spline);

	Tool->bNeedToRerunConstructionScript = true;
}

void UDrawSplineTool::FSplineChange::Revert(UObject* Object)
{
	UDrawSplineTool* Tool = Cast<UDrawSplineTool>(Object);
	if (!ensure(Tool))
	{
		return;
	}
	TWeakObjectPtr<USplineComponent> Spline = Tool->WorkingSpline;
	if (!ensure(Spline.IsValid()))
	{
		return;
	}

	Revert(*Spline);

	Tool->bNeedToRerunConstructionScript = true;
}

#undef LOCTEXT_NAMESPACE
