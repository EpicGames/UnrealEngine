// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMeshMaterialsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Changes/ToolCommandChangeSequence.h"
#include "Changes/MeshChange.h"
#include "Util/ColorConstants.h"
#include "Selections/MeshConnectedComponents.h"
#include "MeshRegionBoundaryLoops.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Materials/MaterialInterface.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditMeshMaterialsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UEditMeshMaterialsTool"

void UEditMeshMaterialsEditActions::PostMaterialAction(EEditMeshMaterialsToolActions Action)
{
	if (ParentTool.IsValid() && Cast<UEditMeshMaterialsTool>(ParentTool))
	{
		Cast<UEditMeshMaterialsTool>(ParentTool)->RequestMaterialAction(Action);
	}
}



void UEditMeshMaterialsToolProperties::UpdateFromMaterialsList()
{
	const int32 ActiveMaterialIdx = GetSelectedMaterialIndex();
	
	MaterialNamesList.Reset();
	for ( int32 k = 0; k < Materials.Num(); ++k)
	{
		UMaterialInterface* Mat = Materials[k];
		FString MatName = (Mat != nullptr) ? Mat->GetName() : "(none)";
		FString UseName = FString::Printf(TEXT("[%d] %s"), k, *MatName);
		MaterialNamesList.Add(UseName);
	}

	if (MaterialNamesList.Num() == 0)
	{
		ActiveMaterial = TEXT("(no materials)");
		return;
	}

	// update active material by index
	ActiveMaterial = MaterialNamesList[ActiveMaterialIdx < MaterialNamesList.Num() ? ActiveMaterialIdx : 0];
}

int32 UEditMeshMaterialsToolProperties::GetSelectedMaterialIndex() const
{
	for (int32 k = 0; k < MaterialNamesList.Num(); ++k)
	{
		if (MaterialNamesList[k] == ActiveMaterial)
		{
			return k;
		}
	}
	return 0;
}



/*
 * ToolBuilder
 */

UMeshSurfacePointTool* UEditMeshMaterialsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UEditMeshMaterialsTool* SelectionTool = NewObject<UEditMeshMaterialsTool>(SceneState.ToolManager);
	SelectionTool->SetWorld(SceneState.World);
	return SelectionTool;
}

bool UEditMeshMaterialsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSelectionToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}


void UEditMeshMaterialsTool::Setup()
{
	UMeshSelectionTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Edit Materials"));

	PreviewMesh->ClearOverrideRenderMaterial();

	FComponentMaterialSet AssetMaterials = UE::ToolTarget::GetMaterialSet(Target, true);
	MaterialProps->Materials = AssetMaterials.Materials;
	CurrentMaterials = MaterialProps->Materials;
	InitialMaterialKey = GetMaterialKey();

	MaterialSetWatchIndex = MaterialProps->WatchProperty<FMaterialSetKey>(
		[this](){ return GetMaterialKey(); },
		[this](FMaterialSetKey NewKey) { OnMaterialSetChanged(); });

	FComponentMaterialSet ComponentMaterials = UE::ToolTarget::GetMaterialSet(Target, false);
	if (ComponentMaterials != AssetMaterials)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("MaterialWarning", "The selected Component has a different Material set than the underlying Asset. Asset materials are shown."),
			EToolMessageLevel::UserWarning);
	}

	// Double check that materials are valid after mesh changes
	PreviewMesh->GetOnMeshChanged().AddLambda([this]()
	{
		UpdateMaterialSetErrors();
	});

}


int32 UEditMeshMaterialsTool::FindMaxActiveMaterialID() const
{
	int32 MaxActiveMaterialID = -1;
	PreviewMesh->ProcessMesh([&MaxActiveMaterialID](const FDynamicMesh3& Mesh)
	{
		if (!Mesh.HasAttributes())
		{
			return;
		}
		const FDynamicMeshMaterialAttribute* MaterialAttrib = Mesh.Attributes()->GetMaterialID();
		for (int32 TID : Mesh.TriangleIndicesItr())
		{
			MaxActiveMaterialID = FMath::Max(MaxActiveMaterialID, MaterialAttrib->GetValue(TID));
		}
	});
	return MaxActiveMaterialID;
}


bool UEditMeshMaterialsTool::FixInvalidMaterialIDs()
{
	int32 NumMaterials = MaterialProps->Materials.Num();
	if (!ensure(NumMaterials > 0)) // We must have more than 0 materials or there are no valid material IDs
	{
		return false;
	}
	bool bHasChanged = false;
	PreviewMesh->EditMesh([NumMaterials, &bHasChanged](FDynamicMesh3& Mesh)
	{
		if (!Mesh.HasAttributes())
		{
			return;
		}
		FDynamicMeshMaterialAttribute* MaterialAttrib = Mesh.Attributes()->GetMaterialID();
		for (int32 TID : Mesh.TriangleIndicesItr())
		{
			if (MaterialAttrib->GetValue(TID) >= NumMaterials)
			{
				MaterialAttrib->SetValue(TID, NumMaterials - 1);
				bHasChanged = true;
			}
		}
	});
	return bHasChanged;
}

UMeshSelectionToolActionPropertySet* UEditMeshMaterialsTool::CreateEditActions()
{
	UEditMeshMaterialsEditActions* Actions = NewObject<UEditMeshMaterialsEditActions>(this);
	Actions->Initialize(this);
	return Actions;
}

void UEditMeshMaterialsTool::AddSubclassPropertySets()
{
	MaterialProps = NewObject<UEditMeshMaterialsToolProperties>(this);
	MaterialProps->RestoreProperties(this);
	AddToolPropertySource(MaterialProps);
}


void UEditMeshMaterialsTool::RequestMaterialAction(EEditMeshMaterialsToolActions ActionType)
{
	if (bHavePendingAction)
	{
		return;
	}

	PendingSubAction = ActionType;
	bHavePendingSubAction = true;
}




void UEditMeshMaterialsTool::OnTick(float DeltaTime)
{
	UMeshSelectionTool::OnTick(DeltaTime);

	if (bHavePendingSubAction)
	{
		ApplyMaterialAction(PendingSubAction);
		bHavePendingSubAction = false;
		PendingSubAction = EEditMeshMaterialsToolActions::NoAction;
	}
}


void UEditMeshMaterialsTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UDynamicMeshBrushTool::RegisterActions(ActionSet);

	// There's a bunch of code duplication here from UMeshSelectionTool::RegisterActions.
	// In fact the only difference is currenlty that we don't register the "delete triangles" action.
	// We could just override the function that performs the delete and not bother overriding the
	// RegisterActions method at all, but that seems risky in case the select tool ever adds other
	// things (especially lambdas) that we don't want to support in the edit materials tool...

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 50,
		TEXT("TriSelectIncreaseSize"),
		LOCTEXT("TriSelectIncreaseSize", "Increase Size"),
		LOCTEXT("TriSelectIncreaseSizeTooltip", "Increase Brush Size"),
		EModifierKey::None, EKeys::D,
		[this]() { IncreaseBrushSizeAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 51,
		TEXT("TriSelectDecreaseSize"),
		LOCTEXT("TriSelectDecreaseSize", "Decrease Size"),
		LOCTEXT("TriSelectDecreaseSizeTooltip", "Decrease Brush Size"),
		EModifierKey::None, EKeys::S,
		[this]() { DecreaseBrushSizeAction(); });

#if WITH_EDITOR  	// enum HasMetaData()  is not available at runtime
	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::CycleSelectionMode,
		TEXT("CycleSelectionMode"),
		LOCTEXT("CycleSelectionMode", "Cycle Selection Mode"),
		LOCTEXT("CycleSelectionModeTooltip", "Cycle through selection modes"),
		EModifierKey::None, EKeys::Q,
		[this]() {
			const UEnum* SelectionModeEnum = StaticEnum<EMeshSelectionToolPrimaryMode>();
			check(SelectionModeEnum);
			int32 NumEnum = SelectionModeEnum->NumEnums() - 1;
			do {
				SelectionProps->SelectionMode = (EMeshSelectionToolPrimaryMode)(((int32)SelectionProps->SelectionMode + 1) % NumEnum);
			} while (SelectionModeEnum->HasMetaData(TEXT("Hidden"), (int32)SelectionProps->SelectionMode));
		}
	);

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::CycleViewMode,
		TEXT("CycleViewMode"),
		LOCTEXT("CycleViewMode", "Cycle View Mode"),
		LOCTEXT("CycleViewModeTooltip", "Cycle through face coloring modes"),
		EModifierKey::None, EKeys::A,
		[this]() {
			const UEnum* ViewModeEnum = StaticEnum<EMeshFacesColorMode>();
			check(ViewModeEnum);
			int32 NumEnum = ViewModeEnum->NumEnums() - 1;
			do {
				SelectionProps->FaceColorMode = (EMeshFacesColorMode)(((int32)SelectionProps->FaceColorMode + 1) % NumEnum);
			} while (ViewModeEnum->HasMetaData(TEXT("Hidden"), (int32)SelectionProps->FaceColorMode));
		}
	);
#endif

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::ShrinkSelection,
		TEXT("ShrinkSelection"),
		LOCTEXT("ShrinkSelection", "Shrink Selection"),
		LOCTEXT("ShrinkSelectionTooltip", "Shrink selection"),
		EModifierKey::Shift, EKeys::Comma,
		[this]() { GrowShrinkSelection(false); });

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::GrowSelection,
		TEXT("GrowSelection"),
		LOCTEXT("GrowSelection", "Grow Selection"),
		LOCTEXT("GrowSelectionTooltip", "Grow selection"),
		EModifierKey::Shift, EKeys::Period,
		[this]() { GrowShrinkSelection(true); });

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::OptimizeSelection,
		TEXT("OptimizeSelection"),
		LOCTEXT("OptimizeSelection", "Optimize Selection"),
		LOCTEXT("OptimizeSelectionTooltip", "Optimize selection"),
		EModifierKey::None, EKeys::O,
		[this]() { OptimizeSelection(); });
}


void UEditMeshMaterialsTool::ApplyMaterialAction(EEditMeshMaterialsToolActions ActionType)
{
	switch (ActionType)
	{
	case EEditMeshMaterialsToolActions::AssignMaterial:
		AssignMaterialToSelectedTriangles();
		break;
	}
}


void UEditMeshMaterialsTool::AssignMaterialToSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	// clear current selection
	BeginChange(false);
	for (int tid : SelectedFaces)
	{
		ActiveSelectionChange->Add(tid);
	}
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
	ChangeSeq->AppendChange(Selection, MoveTemp(SelectionChange));

	int32 SetMaterialID = MaterialProps->GetSelectedMaterialIndex();

	// assign new groups to triangles
	// note: using an FMeshChange is kind of overkill here
	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&SelectedFaces, SetMaterialID](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		if (Mesh.Attributes() && Mesh.Attributes()->HasMaterialID())
		{
			FDynamicMeshMaterialAttribute* MaterialIDAttrib = Mesh.Attributes()->GetMaterialID();
			for (int tid : SelectedFaces)
			{
				ChangeTracker.SaveTriangle(tid, true);
				MaterialIDAttrib->SetNewValue(tid, SetMaterialID);
			}
		}
	});
	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolAssignMaterial", "Assign Material"));

	bFullMeshInvalidationPending = true;
	OnExternalSelectionChange();
	bHaveModifiedMesh = true;

	UpdateMaterialSetErrors();
}




void UEditMeshMaterialsTool::OnMaterialSetChanged()
{
	TUniquePtr<FEditMeshMaterials_MaterialSetChange> MaterialChange = MakeUnique<FEditMeshMaterials_MaterialSetChange>();
	MaterialChange->MaterialsBefore = CurrentMaterials;
	MaterialChange->MaterialsAfter = MaterialProps->Materials;

	PreviewMesh->SetMaterials(MaterialProps->Materials);

	CurrentMaterials = MaterialProps->Materials;

	GetToolManager()->EmitObjectChange(this, MoveTemp(MaterialChange), LOCTEXT("MaterialSetChange", "Material Change"));

	MaterialProps->UpdateFromMaterialsList();

	bHaveModifiedMaterials = true;

	UpdateMaterialSetErrors();
}


void UEditMeshMaterialsTool::UpdateMaterialSetErrors()
{
	int32 NumMaterials = MaterialProps->Materials.Num();
	if (NumMaterials == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("NoMaterialsMessage", "Material Set must contain at least one Material"), EToolMessageLevel::UserWarning);
		bShowingMaterialSetError = true;
		bShowingNotEnoughMaterialsError = false;
	}
	else
	{
		int32 MaxActiveMaterialID = FindMaxActiveMaterialID();
		if (MaxActiveMaterialID >= NumMaterials)
		{
			GetToolManager()->DisplayMessage(FText::Format(LOCTEXT("NotEnoughMaterialsMessage", "Material Set only has {0} {0}|plural(one=material,other=materials), but mesh expects at least {1}. Will remap invalid material IDs on 'Accept'"), NumMaterials, MaxActiveMaterialID + 1), EToolMessageLevel::UserWarning);
			bShowingNotEnoughMaterialsError = true;
			bShowingMaterialSetError = false;
		}
		else if (bShowingNotEnoughMaterialsError || bShowingMaterialSetError)
		{
			GetToolManager()->DisplayMessage({}, EToolMessageLevel::UserWarning);
			bShowingNotEnoughMaterialsError = false;
			bShowingMaterialSetError = false;
		}
	}
}



void UEditMeshMaterialsTool::ExternalUpdateMaterialSet(const TArray<UMaterialInterface*>& NewMaterialSet)
{
	// Disable props so they don't update
	SetToolPropertySourceEnabled(MaterialProps, false);
	MaterialProps->Materials = NewMaterialSet;
	SetToolPropertySourceEnabled(MaterialProps, true);
	PreviewMesh->SetMaterials(MaterialProps->Materials);
	CurrentMaterials = MaterialProps->Materials;

	UpdateMaterialSetErrors();
	if (ensure(MaterialSetWatchIndex > -1))
	{
		MaterialProps->SilentUpdateWatcherAtIndex(MaterialSetWatchIndex);
	}
}



bool UEditMeshMaterialsTool::CanAccept() const
{ 
	return  (CurrentMaterials.Num() > 0) &&
		    (UMeshSelectionTool::CanAccept() || bHaveModifiedMaterials); 
}


void UEditMeshMaterialsTool::ApplyShutdownAction(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("EditMeshMaterialsTransactionName", "Edit Materials"));

		if (GetMaterialKey() != InitialMaterialKey)
		{
			FComponentMaterialSet NewMaterialSet;
			NewMaterialSet.Materials = CurrentMaterials;
			UE::ToolTarget::CommitMaterialSetUpdate(Target, NewMaterialSet, true);
		}

		if (bShowingNotEnoughMaterialsError)
		{
			bool bFixedMaterialIDs = FixInvalidMaterialIDs();
			bHaveModifiedMesh = bHaveModifiedMesh || bFixedMaterialIDs;
		}

		if (bHaveModifiedMesh)
		{
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *PreviewMesh->GetMesh(), true);
		}

		GetToolManager()->EndUndoTransaction();
	}
	else
	{
		UMeshSelectionTool::ApplyShutdownAction(ShutdownType);
	}
}




bool UEditMeshMaterialsTool::FMaterialSetKey::operator!=(const FMaterialSetKey& Key2) const
{
	int Num = Values.Num();
	if (Key2.Values.Num() != Num)
	{
		return true;
	}
	for (int j = 0; j < Num; ++j)
	{
		if (Key2.Values[j] != Values[j])
		{
			return true;
		}
	}
	return false;
}

UEditMeshMaterialsTool::FMaterialSetKey UEditMeshMaterialsTool::GetMaterialKey()
{
	FMaterialSetKey Key;
	for (UMaterialInterface* Material : MaterialProps->Materials)
	{
		Key.Values.Add(Material);
	}
	return Key;
}




void FEditMeshMaterials_MaterialSetChange::Apply(UObject* Object)
{
	UEditMeshMaterialsTool* Tool = CastChecked<UEditMeshMaterialsTool>(Object);
	Tool->ExternalUpdateMaterialSet(MaterialsAfter);
}

void FEditMeshMaterials_MaterialSetChange::Revert(UObject* Object)
{
	UEditMeshMaterialsTool* Tool = CastChecked<UEditMeshMaterialsTool>(Object);
	Tool->ExternalUpdateMaterialSet(MaterialsBefore);
}

FString FEditMeshMaterials_MaterialSetChange::ToString() const
{
	return FString(TEXT("MaterialSet Change"));
}

#undef LOCTEXT_NAMESPACE

