// Copyright Epic Games, Inc. All Rights Reserved.

#include "HarvestInstancesTool.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolDataVisualizer.h"
#include "Util/ColorConstants.h"


#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Editor/EditorEngine.h"		// for FActorLabelUtilities

#include UE_INLINE_GENERATED_CPP_BY_NAME(HarvestInstancesTool)


using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UHarvestInstancesTool"


/*
 * ToolBuilder
 */

bool UHarvestInstancesToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int32 ValidTargets = 0;

	// only static mesh components currently supported
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [&ValidTargets](UActorComponent* Component)
	{
		if (Cast<UStaticMeshComponent>(Component))
		{
			ValidTargets++;
		}
	});
	
	return ValidTargets > 0;
}

UMultiSelectionMeshEditingTool* UHarvestInstancesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UHarvestInstancesTool* NewTool = NewObject<UHarvestInstancesTool>(SceneState.ToolManager);
	return NewTool;
}

void UHarvestInstancesToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	TArray<TObjectPtr<UToolTarget>> Targets;
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [this, &Targets, &SceneState](UActorComponent* Component)
	{
		if (Cast<UStaticMeshComponent>(Component))
		{
			Targets.Add(SceneState.TargetManager->BuildTarget(Component, GetTargetRequirements()));
		}
	});

	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);
}

const FToolTargetTypeRequirements& UHarvestInstancesToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UPrimitiveComponentBackedTarget::StaticClass(),
		UAssetBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


/*
 * 
 */

UHarvestInstancesTool::UHarvestInstancesTool()
{
}


void UHarvestInstancesTool::Setup()
{
	UInteractiveTool::Setup();

	OutputSettings = NewObject<UHarvestInstancesTool_OutputSettings>();
	AddToolPropertySource(OutputSettings);
	OutputSettings->RestoreProperties(this);
	OutputSettings->WatchProperty(OutputSettings->bIncludeSingleInstances, [this](bool bNewValue) { UpdateInstanceSets(); });

	Settings = NewObject<UHarvestInstancesToolSettings>();
	AddToolPropertySource(Settings);
	Settings->RestoreProperties(this);

	InitializeInstanceSets();
	UpdateInstanceSets();

	SetToolDisplayName(LOCTEXT("ToolName", "Harvest Instances"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartHarvestInstancesTool", "Harvest repeated StaticMesh Components from selected Actors into new InstancedStaticMeshComponents"),
		EToolMessageLevel::UserNotification);

	UpdateWarningMessages();
}



void UHarvestInstancesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	OutputSettings->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		EmitResults();
	}
}




void UHarvestInstancesTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FToolDataVisualizer LineRenderer;
	LineRenderer.BeginFrame(RenderAPI);

	if (Settings->bDrawBounds)
	{
		int Counter = 0;
		for (const FInstanceSet& InstanceSet : InstanceSets)
		{
			FColor SetColor = LinearColors::SelectFColor(Counter++);
			LineRenderer.SetLineParameters(FLinearColor(0.95f, 0.05f, 0.05f), 4.0);
			FBox Bounds = InstanceSet.Bounds;
			Bounds.Min -= 5.0 * FVector::One();
			Bounds.Max += 5.0 * FVector::One();

			for (const FTransform& Transform : InstanceSet.WorldInstanceTransforms)
			{
				LineRenderer.PushTransform(Transform);
				LineRenderer.DrawWireBox(Bounds, SetColor, 2.0, true);
				LineRenderer.PopTransform();
			}
		}
	}

	LineRenderer.EndFrame();
}



void UHarvestInstancesTool::InitializeInstanceSets()
{
	// collect up source meshes
	this->SourceMeshes.Reset();

	int32 NumTargets = Targets.Num();
	for (int32 TargetIdx = 0; TargetIdx < NumTargets; TargetIdx++)
	{
		UPrimitiveComponent* TargetComponent = UE::ToolTarget::GetTargetComponent(Targets[TargetIdx]);
		if ( UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(TargetComponent) )
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			if (!StaticMesh) continue;
			TArray<UMaterialInterface*> InstanceMaterials = UE::ToolTarget::GetMaterialSet(Targets[TargetIdx], false).Materials;

			FSourceMesh* UseSourceMesh = nullptr;
			for (int32 k = 0; k < SourceMeshes.Num() && UseSourceMesh == nullptr; ++k)
			{
				if (SourceMeshes[k].Mesh == StaticMesh && SourceMeshes[k].InstanceMaterials == InstanceMaterials)
				{
					UseSourceMesh = &SourceMeshes[k];
				}
			}
			if (UseSourceMesh == nullptr)
			{
				FSourceMesh NewSourceMesh;
				NewSourceMesh.Mesh = StaticMesh;
				NewSourceMesh.InstanceMaterials = InstanceMaterials;
				SourceMeshes.Add(NewSourceMesh);
				UseSourceMesh = &SourceMeshes.Last();
			}

			UseSourceMesh->SourceComponents.Add(StaticMeshComponent);

			if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
			{
				int32 NumInstances = ISMComponent->GetInstanceCount();
				for (int32 i = 0; i < NumInstances; ++i)
				{
					if (ISMComponent->IsValidInstance(i))
					{
						FTransform InstanceWorldTransform;
						if (ensure(ISMComponent->GetInstanceTransform(i, InstanceWorldTransform, true)))
						{
							UseSourceMesh->InstanceTransformsWorld.Add(InstanceWorldTransform);
						}
					}
				}
			}
			else
			{
				FTransform WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]);
				UseSourceMesh->InstanceTransformsWorld.Add(WorldTransform);
			}
		}
	}

	// determine if we have 'loner' instances
	for (int32 k = 0; k < SourceMeshes.Num(); ++k)
	{
		if (SourceMeshes[k].InstanceTransformsWorld.Num() == 1)
		{
			OutputSettings->bHaveLonerInstances = true;
		}
	}
}



void UHarvestInstancesTool::UpdateInstanceSets()
{
	TArray<FSourceMesh> FilteredSourceMeshes = SourceMeshes;

	// remove any meshes that only have one instance, if desired
	if (OutputSettings->bIncludeSingleInstances == false)
	{
		for (int32 k = 0; k < FilteredSourceMeshes.Num(); ++k)
		{
			if (FilteredSourceMeshes[k].InstanceTransformsWorld.Num() <= 1)
			{
				FilteredSourceMeshes.RemoveAt(k);
				k--;
			}
		}
	}

	InstanceSets.Reset();
	SourceActors.Reset();
	SourceComponents.Reset();

	int32 NumInstanceSets = FilteredSourceMeshes.Num();
	OutputSettings->bHaveSingleInstanceSet = (NumInstanceSets == 1);

	InstanceSets.SetNum(NumInstanceSets);
	for (int32 Index = 0; Index < NumInstanceSets; Index++)
	{
		FSourceMesh& SourceMesh = FilteredSourceMeshes[Index];

		FInstanceSet& Element = InstanceSets[Index];

		Element.SourceComponents = SourceMesh.SourceComponents;
		for (UStaticMeshComponent* Component : Element.SourceComponents)
		{
			Element.SourceActors.AddUnique(Component->GetOwner());
			SourceActors.AddUnique(Component->GetOwner());
			SourceComponents.Add(Component);
		}

		Element.StaticMesh = SourceMesh.Mesh;
		Element.Bounds = SourceMesh.Mesh->GetBoundingBox();
		Element.InstanceMaterials = SourceMesh.InstanceMaterials;
		Element.WorldInstanceTransforms = SourceMesh.InstanceTransformsWorld;

		// do this?
		//bHaveNonUniformScaleElements = bHaveNonUniformScaleElements || Element.BaseRotateScale.HasNonUniformScale();
	}


	// Check that all SceneComponents of Source Actors are included in instance set. 
	// If not, we cannot delete the Source Actors
	bool bFoundUnhandledComponent = false;
	for (AActor* SourceActor : SourceActors)
	{
		SourceActor->ForEachComponent(true, [&](UActorComponent* Component) 
		{
			if (bFoundUnhandledComponent) return;

			// only going to consider SceneComponents
			if (Cast<USceneComponent>(Component) == nullptr) return;

			if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				if (SourceComponents.Contains(PrimComponent))
				{
					return;
				}
			}

			// for now skip editor-only Components as some editor systems (like navigation) apparently attach additional hidden Components to static meshes...
			if (Component->IsEditorOnly() == false)
			{
				bFoundUnhandledComponent = true;
			}
		});
	}
	OutputSettings->bCanDeleteInputs = (bFoundUnhandledComponent == false);
	UpdateWarningMessages();
}



void UHarvestInstancesTool::UpdateWarningMessages()
{
	FTextBuilder WarningTextBuilder;
	if (OutputSettings->bCanDeleteInputs == false)
	{
		WarningTextBuilder.AppendLine(
			LOCTEXT("CannotDeleteWarning", "Source Actors cannot be deleted because they contain additional Components that cannot be Instanced. "));
	}
	if (bHaveNonUniformScaleElements)
	{
		WarningTextBuilder.AppendLine(
			LOCTEXT("NonUniformScaleWarning", "Source Objects have Non-Uniform Scaling, which may prevent Instance Transforms from working correctly."));
	}

	GetToolManager()->DisplayMessage(WarningTextBuilder.ToText(), EToolMessageLevel::UserWarning);
}


void UHarvestInstancesTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == Settings || PropertySet == OutputSettings)
	{
		return;
	}

	OnParametersUpdated();
}


void UHarvestInstancesTool::OnParametersUpdated()
{
	//MarkPatternDirty();
}




void UHarvestInstancesTool::EmitResults()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("HarvestInstances", "Harvest Instances"));

	bool bCreateHISMs = (OutputSettings->ComponentType == EHarvestInstancesToolOutputType::HISMC);
	int32 NumInstanceSets = InstanceSets.Num();

	TArray<AActor*> NewActors;		// set of new actors created by operation

	// TODO: investigate use of CopyPropertiesForUnrelatedObjects to transfer settings from source to target Components/Actors

	auto ComputeComponentOriginFunc = [](FInstanceSet& Element) -> FVector
	{
		FVector ComponentPosition = FVector::Zero();
		for (const FTransform& WorldTransform : Element.WorldInstanceTransforms)
		{
			ComponentPosition += WorldTransform.GetLocation();
		}
		ComponentPosition /= Element.WorldInstanceTransforms.Num();
		return ComponentPosition;
	};

	auto CreateNewComponentFunc = [bCreateHISMs](FInstanceSet& Element, AActor* ParentActor, FString BaseName) -> UInstancedStaticMeshComponent*
	{
		TSubclassOf<UInstancedStaticMeshComponent> ComponentClass = (bCreateHISMs) ?
			UHierarchicalInstancedStaticMeshComponent::StaticClass() : UInstancedStaticMeshComponent::StaticClass();
		FString Suffix = (bCreateHISMs) ? TEXT("_HISM") : TEXT("_ISM");

		FName UseName = (BaseName.Len() > 0) ? FName(BaseName + Suffix) : FName();

		UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(ParentActor, ComponentClass,
			MakeUniqueObjectName(ParentActor, ComponentClass, UseName));

		ISMComponent->SetFlags(RF_Transactional);
		ISMComponent->bHasPerInstanceHitProxies = true;

		ISMComponent->SetStaticMesh(Element.StaticMesh);
		for (int32 j = 0; j < Element.InstanceMaterials.Num(); ++j)
		{
			ISMComponent->SetMaterial(j, Element.InstanceMaterials[j]);
		}
		return ISMComponent;
	};


	AActor* SingleActor = nullptr;
	if (OutputSettings->bSingleActor || NumInstanceSets == 1)
	{
		FActorSpawnParameters SpawnInfo;
		SingleActor = GetTargetWorld()->SpawnActor<AActor>(SpawnInfo);
		if ( SingleActor == nullptr )
		{
			return;		// ??
		}

		FString UseBaseName = (OutputSettings->ActorName.Len() > 0) ? OutputSettings->ActorName : TEXT("Instances");
		FActorLabelUtilities::SetActorLabelUnique(SingleActor, UseBaseName);

		NewActors.Add(SingleActor);
		bool bFirst = true;

		for (int32 ElemIdx = 0; ElemIdx < NumInstanceSets; ++ElemIdx)
		{
			FInstanceSet& Element = InstanceSets[ElemIdx];
			FString AssetBaseName = FPaths::GetBaseFilename(Element.StaticMesh->GetName());
			AssetBaseName = UE::Modeling::StripGeneratedAssetSuffixFromName(AssetBaseName);

			FVector ComponentPosition = ComputeComponentOriginFunc(Element);
			UInstancedStaticMeshComponent* ISMComponent = CreateNewComponentFunc(Element, SingleActor, AssetBaseName);

			if (bFirst)
			{
				SingleActor->SetRootComponent(ISMComponent);
				ISMComponent->OnComponentCreated();
				SingleActor->AddInstanceComponent(ISMComponent);
				SingleActor->SetActorTransform(FTransform(ComponentPosition));

				bFirst = false;
			}
			else
			{
				ISMComponent->OnComponentCreated();
				ISMComponent->SetWorldTransform(FTransform(ComponentPosition));
				ISMComponent->AttachToComponent(SingleActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
				SingleActor->AddInstanceComponent(ISMComponent);
			}

			for (FTransform WorldTransform : Element.WorldInstanceTransforms)
			{
				ISMComponent->AddInstance(WorldTransform, /*bTransformInWorldSpace=*/true);
			}

			ISMComponent->RegisterComponent();
		}
	}
	else
	{
		for (int32 ElemIdx = 0; ElemIdx < NumInstanceSets; ++ElemIdx)
		{
			FInstanceSet& Element = InstanceSets[ElemIdx];
			FVector ComponentPosition = ComputeComponentOriginFunc(Element);

			FActorSpawnParameters SpawnInfo;
			AActor* NewActor = GetTargetWorld()->SpawnActor<AActor>(SpawnInfo);
			if (NewActor != nullptr)
			{
				FString AssetName = FPaths::GetBaseFilename(Element.StaticMesh->GetName());
				FString UseBaseName = UE::Modeling::StripGeneratedAssetSuffixFromName(AssetName);
				UseBaseName += TEXT("Instances");
				FActorLabelUtilities::SetActorLabelUnique(NewActor, UseBaseName);

				NewActors.Add(NewActor);

				FString AssetBaseName = FPaths::GetBaseFilename(Element.StaticMesh->GetName());
				AssetBaseName = UE::Modeling::StripGeneratedAssetSuffixFromName(AssetBaseName);
				UInstancedStaticMeshComponent* ISMComponent = CreateNewComponentFunc(Element, NewActor, AssetBaseName);

				NewActor->SetRootComponent(ISMComponent);
				ISMComponent->OnComponentCreated();
				NewActor->AddInstanceComponent(ISMComponent);

				NewActor->SetActorTransform(FTransform(ComponentPosition));

				for (FTransform WorldTransform : Element.WorldInstanceTransforms)
				{
					ISMComponent->AddInstance(WorldTransform, /*bTransformInWorldSpace=*/true);
				}

				ISMComponent->RegisterComponent();
			}
		}
	}

	// delete all source actors if desired
	if (OutputSettings->bCanDeleteInputs && OutputSettings->bDeleteInputs)
	{
		// destroy any Actors that won't have any Components left
		for (AActor* Actor : SourceActors)
		{
			Actor->Modify();
			Actor->Destroy();
		}
	}

	if (NewActors.Num() > 0)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewActors);
	}

	GetToolManager()->EndUndoTransaction();
}



#undef LOCTEXT_NAMESPACE
