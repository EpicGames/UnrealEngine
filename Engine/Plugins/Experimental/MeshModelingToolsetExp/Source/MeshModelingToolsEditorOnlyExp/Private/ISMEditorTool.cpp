// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMEditorTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/DoubleClickBehavior.h"
#include "ToolSceneQueriesUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Scene/MeshSceneAdapter.h"
#include "ToolDataVisualizer.h"
#include "Drawing/PreviewGeometryActor.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"

#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMEditorTool)

#define LOCTEXT_NAMESPACE "UISMEditorTool"

namespace ISMEditorToolsLocals
{
	FText NoSelectionError = LOCTEXT("NoSelectionError", "Must select at least one Instance");
}

/*
 * ToolBuilder
 */


template<typename IterFunc>
void ComponentsIteration(const FToolBuilderState& SceneState, IterFunc Func)
{
	if (SceneState.SelectedComponents.Num() > 0)
	{
		for (UActorComponent* Component : SceneState.SelectedComponents)
		{
			if (Func(Component) == false)
			{
				return;
			}
		}
	}
	else if (SceneState.SelectedActors.Num() > 0)
	{
		for (AActor* Actor : SceneState.SelectedActors)
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (Func(Component) == false)
				{
					return;
				}
			}

			// not currently handling child actor components...
		}
	}
}


bool UISMEditorToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	bool bFoundISM = false;
	ComponentsIteration(SceneState, [&bFoundISM](UActorComponent* Component)
	{
		if (Cast<UInstancedStaticMeshComponent>(Component) != nullptr)
		{
			bFoundISM = true;
			return false;
		}
		return true;
	});
	return bFoundISM;
}

UInteractiveTool* UISMEditorToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UISMEditorTool* NewTool = NewObject<UISMEditorTool>(SceneState.ToolManager);

	TArray<UInstancedStaticMeshComponent*> Targets;
	ComponentsIteration(SceneState, [&Targets](UActorComponent* Component)
	{
		if (Cast<UInstancedStaticMeshComponent>(Component) != nullptr)
		{
			Targets.Add(Cast<UInstancedStaticMeshComponent>(Component));
		}
		return true;
	});

	NewTool->SetTargets(MoveTemp(Targets));
	return NewTool;
}



void UISMEditorToolActionPropertySetBase::PostAction(EISMEditorToolActions Action)
{
	ParentTool->RequestAction(Action);
}


/*
 * Tool
 */

void UISMEditorTool::SetTargets(TArray<UInstancedStaticMeshComponent*> Components)
{
	TargetComponents = Components;
}


void UISMEditorTool::Setup()
{
	UInteractiveTool::Setup();

	bMeshSceneDirty = true;

	// set up rectangle marquee mechanic
	RectangleMarqueeMechanic = NewObject<URectangleMarqueeMechanic>(this);
	RectangleMarqueeMechanic->bUseExternalClickDragBehavior = true;
	RectangleMarqueeMechanic->Setup(this);
	RectangleMarqueeMechanic->SetIsEnabled(true);
	RectangleMarqueeMechanic->OnDragRectangleFinished.AddUObject(this, &UISMEditorTool::OnMarqueeRectangleFinished);

	// clickdrag behavior clicks-to-select or does marquee on drag
	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, RectangleMarqueeMechanic);
	ClickOrDragBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	AddInputBehavior(ClickOrDragBehavior);

	DoubleClickBehavior = NewObject<ULocalDoubleClickInputBehavior>();
	DoubleClickBehavior->SetDefaultPriority(ClickOrDragBehavior->GetPriority().MakeHigher());
	DoubleClickBehavior->Initialize();
	DoubleClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& ClickPos) { return IsHitByClick(ClickPos); };
	DoubleClickBehavior->OnUpdateModifierStateFunc = [this](int ModifierID, bool bIsOn) { return OnUpdateModifierState(ModifierID, bIsOn); };
	DoubleClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) { return OnDoubleClicked(ClickPos); };
	AddInputBehavior(DoubleClickBehavior);

	TransformProps = NewObject<UISMEditorToolProperties>();
	AddToolPropertySource(TransformProps);
	TransformProps->RestoreProperties(this);
	TransformProps->WatchProperty(TransformProps->TransformMode, [this](EISMEditorTransformMode NewMode) { UpdateTransformMode(NewMode); });
	TransformProps->WatchProperty(TransformProps->bSetPivotMode, [this](bool bNewValue) { UpdateSetPivotModes(bNewValue); });

	UpdateTransformMode(TransformProps->TransformMode);

	ToolActions = NewObject<UISMEditorToolActionPropertySet>();
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	ExtractActions = NewObject<UISMEditorToolExtractPropertySet>();
	ExtractActions->Initialize(this);
	AddToolPropertySource(ExtractActions);

	TArray<AActor*> UniqueActors;
	for (UInstancedStaticMeshComponent* Component : TargetComponents)
	{
		UniqueActors.AddUnique(Component->GetOwner());
	}
	if (UniqueActors.Num() == 1)
	{
		ReplaceAction = NewObject<UISMEditorToolReplacePropertySet>();
		ReplaceAction->Initialize(this);
		AddToolPropertySource(ReplaceAction);
	}

	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(TargetComponents[0]->GetWorld(), FTransform::Identity);

	SetToolDisplayName(LOCTEXT("ToolName", "Edit ISMs"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartISMEditorTool", "Edit the selected ISMs"),
		EToolMessageLevel::UserNotification);

	// Add space in the warning area that appears when we clear warning messages.
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserError);
}


void UISMEditorTool::Shutdown(EToolShutdownType ShutdownType)
{
	TransformProps->SaveProperties(this);

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	}

	// Release the FMeshSceneAdapter, this will clear out any MeshDescriptions that were loaded unnecessarily.
	// This needs to happen on game thread, if we don't do it here, it may happen during GC later
	MeshScene.Reset();
}


void UISMEditorTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == 1)
	{
		bShiftModifier = bIsOn;
	}
	if (ModifierID == 2)
	{
		bCtrlModifier = bIsOn;
	}
}



void UISMEditorTool::OnMarqueeRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{
	using namespace UE::Geometry;

	if (!bCancelled)
	{
		TArray<const FActorChildMesh*> HitMeshes;
		FCriticalSection HitMeshLock;
		
		MeshScene->ParallelMeshVertexEnumeration(
			[&](int32 NumMeshes) { return true; },
			[&](int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FAxisAlignedBox3d& WorldBounds) { return true; },
			[&](int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FVector3d& WorldPos) 
			{ 
				if ( Rectangle.IsProjectedPointInRectangle( (FVector)WorldPos ) )
				{
					HitMeshLock.Lock();
					HitMeshes.Add(ChildMeshInfo);
					HitMeshLock.Unlock();
					return false;
				}
				return true; 
			});

		TArray<FSelectedInstance> NewSelection;
		if (ModifyingSelection())
		{
			NewSelection = CurrentSelection;
		}

		for (const FActorChildMesh* HitMesh : HitMeshes)
		{
			FSelectedInstance NewItem{ 
				(UInstancedStaticMeshComponent*)HitMesh->SourceComponent, 
				HitMesh->ComponentIndex,
				HitMesh->MeshSpatial->GetWorldBounds(
					[&](const FVector3d& P) { return HitMesh->WorldTransform.TransformPosition(P); })
			};
			
			if (ShouldSubtractSelection())
			{
				NewSelection.RemoveSwap(NewItem);
			}
			else if (ShouldToggleSelection())
			{
				if (NewSelection.RemoveSwap(NewItem) == 0)
				{
					NewSelection.Add(NewItem);
				}
			}
			else // if adding or setting
			{
				NewSelection.AddUnique(NewItem);
			}
		}

		if (NewSelection != CurrentSelection)
		{
			UpdateSelectionInternal(NewSelection, true);
		}
	}
}


void UISMEditorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bInActiveDrag && TransformProps->bHideWhenDragging)
	{
		return;
	}

	if (RectangleMarqueeMechanic)
	{
		RectangleMarqueeMechanic->Render(RenderAPI);
	}

	FToolDataVisualizer LineRenderer;
	LineRenderer.BeginFrame(RenderAPI);

	//if (TransformProps->bShowSelectable)
	//{
	//	LineRenderer.SetLineParameters(FLinearColor(0.2, 0.9, 0.2), 2.0);
	//	for (const FAxisAlignedBox3d& Box : AllMeshBoundingBoxes)
	//	{
	//		LineRenderer.DrawWireBox((FBox)Box);
	//	}
	//}

	if (TransformProps->bShowSelected)
	{
		LineRenderer.SetLineParameters(FLinearColor(0.95f, 0.05f, 0.05f), 4.0);
		for (FSelectedInstance Instance : CurrentSelection)
		{
			if (Instance.Component->IsValidInstance(Instance.Index))
			{
				LineRenderer.DrawWireBox((FBox)Instance.WorldBounds);
			}
		}
	}

	LineRenderer.EndFrame();
}



void UISMEditorTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (RectangleMarqueeMechanic)
	{
		RectangleMarqueeMechanic->DrawHUD(Canvas, RenderAPI);
	}
}



void UISMEditorTool::UpdatePreviewGeometry()
{
	FColor LinesColor(50, 225, 50);
	static constexpr int BoxFaces[6][4] =
	{
		{ 0, 1, 2, 3 },     // back, -z
		{ 5, 4, 7, 6 },     // front, +z
		{ 4, 0, 3, 7 },     // left, -x
		{ 1, 5, 6, 2 },     // right, +x,
		{ 4, 5, 1, 0 },     // bottom, -y
		{ 3, 2, 6, 7 }      // top, +y
	};

	PreviewGeometry->CreateOrUpdateLineSet(TEXT("SelectableBoxes"), AllMeshBoundingBoxes.Num(),
										   [&](int32 Index, TArray<FRenderableLine>& LinesOut)
	{
		FBox Box = (FBox)AllMeshBoundingBoxes[Index];
		FVector Corners[8] =
		{
			Box.Min,
			FVector(Box.Max.X, Box.Min.Y, Box.Min.Z),
			FVector(Box.Max.X, Box.Max.Y, Box.Min.Z),
			FVector(Box.Min.X, Box.Max.Y, Box.Min.Z),
			FVector(Box.Min.X, Box.Min.Y, Box.Max.Z),
			FVector(Box.Max.X, Box.Min.Y, Box.Max.Z),
			Box.Max,
			FVector(Box.Min.X, Box.Max.Y, Box.Max.Z)
		};
		for (int FaceIdx = 0; FaceIdx < 6; FaceIdx++)
		{
			for (int Last = 3, Cur = 0; Cur < 4; Last = Cur++)
			{
				LinesOut.Add(FRenderableLine(Corners[BoxFaces[FaceIdx][Last]], Corners[BoxFaces[FaceIdx][Cur]],
											 LinesColor, 2.0, 0.0));
			}
		}

	}, 12);

}


void UISMEditorTool::OnTick(float DeltaTime)
{
	if (PendingAction != EISMEditorToolActions::NoAction)
	{
		// Clear any messages from last action
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserError);

		switch (PendingAction)
		{
		case EISMEditorToolActions::ClearSelection:
			OnClearSelection();
			break;
		case EISMEditorToolActions::Delete:
			OnDeleteSelection();
			break;
		case EISMEditorToolActions::Duplicate:
			OnDuplicateSelection();
			break;
		case EISMEditorToolActions::DuplicateToNew:
			OnDuplicateToNew();
			break;
		case EISMEditorToolActions::ExtractToNew:
			OnExtractToNew();
			break;
		case EISMEditorToolActions::Replace:
			OnReplaceSelection();
			break;
		case EISMEditorToolActions::ExpandAll:
			OnExpandAllSelection();
			break;
		case EISMEditorToolActions::ExpandLast:
			OnExpandLastSelection();
			break;
		case EISMEditorToolActions::ContractLast:
			OnContractLastSelection();
			break;
		}
		PendingAction = EISMEditorToolActions::NoAction;
	}

	if (bMeshSceneDirty)
	{
		RebuildMeshScene();
		bMeshSceneDirty = false;

		// when we dirty mesh scene on undo/redo, selection may have been invalid
		OnSelectionUpdated();
	}

	if (PreviewGeometry != nullptr)
	{
		PreviewGeometry->SetLineSetVisibility(TEXT("SelectableBoxes"), 
			TransformProps->bShowSelectable && !(bInActiveDrag && TransformProps->bHideWhenDragging));
	}
}


void UISMEditorTool::UpdateSetPivotModes(bool bEnableSetPivot)
{
	for (FISMEditorTarget& Target : ActiveGizmos)
	{
		Target.TransformProxy->bSetPivotMode = bEnableSetPivot;
	}
}



void UISMEditorTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 1,
		TEXT("ToggleSetPivot"),
		LOCTEXT("TransformToggleSetPivot", "Toggle Set Pivot"),
		LOCTEXT("TransformToggleSetPivotTooltip", "Toggle Set Pivot on and off"),
		EModifierKey::None, EKeys::S,
		[this]() { this->TransformProps->bSetPivotMode = !this->TransformProps->bSetPivotMode; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 3,
		TEXT("CycleTransformMode"),
		LOCTEXT("TransformCycleTransformMode", "Next Transform Mode"),
		LOCTEXT("TransformCycleTransformModeTooltip", "Cycle through available Transform Modes"),
		EModifierKey::None, EKeys::A,
		[this]() { this->TransformProps->TransformMode = (EISMEditorTransformMode)(((uint8)TransformProps->TransformMode+1) % (uint8)EISMEditorTransformMode::LastValue); });

}





FInputRayHit UISMEditorTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	using namespace UE::Geometry;

	if (CurrentSelection.Num() > 0)
	{
		return FInputRayHit(0);
	}
	else
	{
		FMeshSceneRayHit HitResult;
		if (MeshScene->FindNearestRayIntersection(ClickPos.WorldRay, HitResult))
		{
			return FInputRayHit(HitResult.RayDistance);
		}
		return FInputRayHit();
	}
}


void UISMEditorTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	using namespace UE::Geometry;

	TArray<FSelectedInstance> NewSelection;

	FMeshSceneRayHit HitResult;
	if (MeshScene->FindNearestRayIntersection(ClickPos.WorldRay, HitResult))
	{
		UE_LOGF(LogTemp, Warning, "Hit Component %ls, Element Index %d, Triangle %d",
			   *HitResult.HitComponent->GetName(), HitResult.HitComponentElementIndex, HitResult.HitMeshTriIndex);

		FSelectedInstance NewItem{ 
			(UInstancedStaticMeshComponent*)HitResult.HitComponent, 
			HitResult.HitComponentElementIndex,
			HitResult.HitMeshSpatialWrapper->GetWorldBounds(
				[&](const FVector3d& P) { return HitResult.LocalToWorld.TransformPosition(P); })
		};

		if (ModifyingSelection())
		{
			NewSelection = CurrentSelection;
		}
		if (ShouldSubtractSelection())
		{
			NewSelection.RemoveSwap(NewItem);
		}
		else if (ShouldToggleSelection())
		{
			if (NewSelection.RemoveSwap(NewItem) == 0)
			{
				NewSelection.Add(NewItem);
			}
		}
		else // if adding or setting
		{
			NewSelection.AddUnique(NewItem);
		}
	}

	if (NewSelection != CurrentSelection)
	{
		UpdateSelectionInternal(NewSelection, true);
	}
}

void UISMEditorTool::OnDoubleClicked(const FInputDeviceRay& ClickPos)
{
	// This is largely the same as OnClicked, except it does selection expansion/contraction.

	using namespace UE::Geometry;

	FText TransactionLabel = LOCTEXT("ExpandByDoubleClick", "Expand Selection By Double Click");
	TArray<FSelectedInstance> NewSelection;

	FMeshSceneRayHit HitResult;
	if (MeshScene->FindNearestRayIntersection(ClickPos.WorldRay, HitResult))
	{
		FSelectedInstance NewItem{
			(UInstancedStaticMeshComponent*)HitResult.HitComponent,
			HitResult.HitComponentElementIndex,
			HitResult.HitMeshSpatialWrapper->GetWorldBounds(
				[&](const FVector3d& P) { return HitResult.LocalToWorld.TransformPosition(P); })
		};

		if (ModifyingSelection())
		{
			NewSelection = CurrentSelection;
		}

		if (ShouldSubtractSelection())
		{
			TransactionLabel = LOCTEXT("ContractByDoubleClick", "Contract Selection By Double Click");
			ContractSelectionBy(NewItem, NewSelection);
		}
		else if (ShouldToggleSelection())
		{
			TransactionLabel = LOCTEXT("ToggleByDoubleClick", "Toggle Selection By Double Click");
			ToggleSelectionBy(NewItem, NewSelection);

			// A double click actually triggers the single click callback first, so we need to toggle
			//  the original item again to undo that toggle. Of course, this would be slightly broken if
			//  the mouse is moving such that the first click clicked something else, but that seems unlikely,
			//  especially since this toggle action is probably uncommon.
			if (NewSelection.Remove(NewItem) == 0)
			{
				NewSelection.AddUnique(NewItem);
			}
		}
		else // if adding or setting
		{
			ExpandSelectionBy(NewItem, NewSelection);
		}
	}

	if (NewSelection != CurrentSelection)
	{
		UpdateSelectionInternal(NewSelection, true, TransactionLabel);
	}
}

void UISMEditorTool::OnSelectionUpdated()
{
	TArray<FString> SelectionStrings;
	for (FSelectedInstance& SelectedItem : CurrentSelection)
	{
		if (TargetComponents.Num() > 1)
		{
			SelectionStrings.Add(FString::Printf(TEXT("%s : %d"), *SelectedItem.Component->GetName(), SelectedItem.Index));
		}
		else
		{
			SelectionStrings.Add(FString::Printf(TEXT("%d"), SelectedItem.Index));
		}
	}
	TransformProps->SelectedInstances = SelectionStrings;

	UpdateTransformMode(TransformProps->TransformMode);
}



void UISMEditorTool::UpdateTransformMode(EISMEditorTransformMode NewMode)
{
	ResetActiveGizmos();

	if (CurrentSelection.Num() == 0)
	{
		return;
	}

	switch (NewMode)
	{
		default:
		case EISMEditorTransformMode::SharedGizmo:
			SetActiveGizmos_Single(false);
			break;

		case EISMEditorTransformMode::SharedGizmoLocal:
			SetActiveGizmos_Single(true);
			break;

		case EISMEditorTransformMode::PerObjectGizmo:
			SetActiveGizmos_PerObject();
			break;
	}

	CurTransformMode = NewMode;
}



namespace UE {
namespace Local {

static void AddInstancedComponentInstance(UInstancedStaticMeshComponent* ISMC, int32 Index, UTransformProxy* TransformProxy, bool bModifyOnTransform)
{
	TransformProxy->AddComponentCustom(ISMC,
		[ISMC, Index]() {
			FTransform Tmp;
			ISMC->GetInstanceTransform(Index, Tmp, true);
			return Tmp;
		},
		[ISMC, Index](const FTransform& NewTransform) {
			ISMC->UpdateInstanceTransform(Index, NewTransform, true, true, true);
		},
		Index, bModifyOnTransform
	);
}

}
}



void UISMEditorTool::SetActiveGizmos_Single(bool bLocalRotations)
{
	check(ActiveGizmos.Num() == 0);

	FISMEditorTarget Transformable;
	Transformable.TransformProxy = NewObject<UTransformProxy>(this);
	Transformable.TransformProxy->bSetPivotMode = TransformProps ? TransformProps->bSetPivotMode : false;
	Transformable.TransformProxy->bRotatePerObject = bLocalRotations;
	Transformable.TransformProxy->OnBeginTransformEdit.AddLambda([this](UTransformProxy*) { bInActiveDrag = true; });
	Transformable.TransformProxy->OnEndTransformEdit.AddLambda([this](UTransformProxy*) { OnTransformCompleted(); bInActiveDrag = false; });
	Transformable.TransformProxy->OnTransformChangedUndoRedo.AddLambda([this](UTransformProxy*, FTransform) { OnTransformCompleted(); });

	for (FSelectedInstance Instance : CurrentSelection)
	{
		if (Instance.Component->IsValidInstance(Instance.Index))
		{
			UE::Local::AddInstancedComponentInstance(Instance.Component, Instance.Index, Transformable.TransformProxy, true);
		}
	}

	// leave out nonuniform scale if we have multiple objects in non-local mode
	bool bCanNonUniformScale = TargetComponents.Num() == 1 || bLocalRotations;
	ETransformGizmoSubElements GizmoElements = (bCanNonUniformScale) ?
		ETransformGizmoSubElements::FullTranslateRotateScale : ETransformGizmoSubElements::TranslateRotateUniformScale;
	Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), GizmoElements, this);
	Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy, GetToolManager());

	ActiveGizmos.Add(Transformable);
}

void UISMEditorTool::SetActiveGizmos_PerObject()
{
	check(ActiveGizmos.Num() == 0);

	for (FSelectedInstance Instance : CurrentSelection)
	{
		if (Instance.Component->IsValidInstance(Instance.Index))
		{
			FISMEditorTarget Transformable;
			Transformable.TransformProxy = NewObject<UTransformProxy>(this);
			Transformable.TransformProxy->bSetPivotMode = TransformProps ? TransformProps->bSetPivotMode : false;
			Transformable.TransformProxy->OnBeginTransformEdit.AddLambda([this](UTransformProxy*) { bInActiveDrag = true; });
			Transformable.TransformProxy->OnEndTransformEdit.AddLambda([this](UTransformProxy*) { OnTransformCompleted(); bInActiveDrag = false; });
			Transformable.TransformProxy->OnTransformChangedUndoRedo.AddLambda([this](UTransformProxy*, FTransform) { OnTransformCompleted(); });

			UE::Local::AddInstancedComponentInstance(Instance.Component, Instance.Index, Transformable.TransformProxy, true);

			ETransformGizmoSubElements GizmoElements = ETransformGizmoSubElements::FullTranslateRotateScale;
			Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), GizmoElements, this);
			Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy, GetToolManager());

			ActiveGizmos.Add(Transformable);
		}
	}
}



void UISMEditorTool::ResetActiveGizmos()
{
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	ActiveGizmos.Reset();
}


void UISMEditorTool::RequestAction(EISMEditorToolActions ActionType)
{
	PendingAction = ActionType;
}


void UISMEditorTool::OnClearSelection()
{
	UpdateSelectionInternal(TArray<FSelectedInstance>(), true);
}

void UISMEditorTool::OnDeleteSelection()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("DeleteSelected", "Delete Selected"));

	TArray<UInstancedStaticMeshComponent*> UniqueInstances;
	for (FSelectedInstance Instance : CurrentSelection)
	{
		if (Instance.Component->IsValidInstance(Instance.Index))
		{
			UniqueInstances.AddUnique(Instance.Component);
		}
	}

	for (UInstancedStaticMeshComponent* DeleteFromComponent : UniqueInstances)
	{
		TArray<int32> InstanceList;
		for (FSelectedInstance Instance : CurrentSelection)
		{
			if (Instance.Component == DeleteFromComponent && Instance.Component->IsValidInstance(Instance.Index))
			{
				InstanceList.Add(Instance.Index);
			}
		}

		DeleteFromComponent->Modify();
		DeleteFromComponent->RemoveInstances(InstanceList);
	}

	bMeshSceneDirty = true;
	GetToolManager()->EmitObjectChange(this, MakeUnique<FISMEditorSceneChange>(), LOCTEXT("SceneChange", "Scene Update"));
	UpdateSelectionInternal(TArray<FSelectedInstance>(), true);

	GetToolManager()->EndUndoTransaction();
}


void UISMEditorTool::OnDuplicateSelection()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("DuplicateSelected", "Duplicate Selected"));

	TArray<FSelectedInstance> NewSelection;
	for (FSelectedInstance Instance : CurrentSelection)
	{
		if (Instance.Component->IsValidInstance(Instance.Index))
		{
			FTransform CurInstanceTransform;
			Instance.Component->GetInstanceTransform(Instance.Index, CurInstanceTransform, true);

			Instance.Component->Modify();
			int32 NewIndex = Instance.Component->AddInstance(CurInstanceTransform, true);

			// copy per-instance custom data if it is defined
			int32 NumCustomDataFloats = Instance.Component->NumCustomDataFloats;
			if (NumCustomDataFloats > 0)
			{
				const TArray<float>& CurCustomData = Instance.Component->PerInstanceSMCustomData;
				int32 BaseIndex = Instance.Index * NumCustomDataFloats;
				if (CurCustomData.Num() >= (BaseIndex + NumCustomDataFloats))
				{
					TArray<float> NewCustomData;
					for (int32 k = 0; k < NumCustomDataFloats; ++k)
					{
						NewCustomData.Add(CurCustomData[BaseIndex + k]);
					}
					Instance.Component->SetCustomData(NewIndex, NewCustomData);
				}
			}

			NewSelection.Add(FSelectedInstance{ Instance.Component, NewIndex, Instance.WorldBounds });
		}
	}

	bMeshSceneDirty = true;
	GetToolManager()->EmitObjectChange(this, MakeUnique<FISMEditorSceneChange>(), LOCTEXT("SceneChange", "Scene Update"));
	UpdateSelectionInternal(NewSelection, true);

	GetToolManager()->EndUndoTransaction();
}

void UISMEditorTool::OnDuplicateToNew()
{
	DuplicateToNew(/*bDeleteFromCurrent =*/ false);
}

void UISMEditorTool::DuplicateToNew(bool bDeleteFromCurrent)
{
	if (CurrentSelection.Num() == 0)
	{
		GetToolManager()->DisplayMessage(ISMEditorToolsLocals::NoSelectionError, EToolMessageLevel::UserError);
		return;
	}

	// Group our selected items by same static mesh and materials
	TArray<TArray<int32>> SelectionIndexBuckets;
	if (TargetComponents.Num() == 1)
	{
		SelectionIndexBuckets.Emplace();
		for (int32 Index = 0; Index < CurrentSelection.Num(); ++Index)
		{
			SelectionIndexBuckets[0].Add(Index);
		}
	}
	else
	{
		TMap<UInstancedStaticMeshComponent*, int32> ComponentToBucket;
		for (int32 SelectionIndex = 0; SelectionIndex < CurrentSelection.Num(); ++SelectionIndex)
		{
			FSelectedInstance& Instance = CurrentSelection[SelectionIndex];
			if (!ensure(Instance.Component))
			{
				continue;
			}

			int32 InstanceBucketIndex = INDEX_NONE;
			if (int32* KnownBucket = ComponentToBucket.Find(Instance.Component))
			{
				InstanceBucketIndex = *KnownBucket;
			}
			else
			{
				TArray<UMaterialInterface*> Materials = Instance.Component->GetMaterials();
				for (int32 BucketIndex = 0; BucketIndex < SelectionIndexBuckets.Num(); ++BucketIndex)
				{
					const FSelectedInstance& RepresentativeItem = CurrentSelection[SelectionIndexBuckets[BucketIndex][0]];
					if (Instance.Component->GetStaticMesh() == RepresentativeItem.Component->GetStaticMesh()
						&& RepresentativeItem.Component->GetMaterials() == Materials)
					{
						InstanceBucketIndex = BucketIndex;
						ComponentToBucket.Add(Instance.Component, InstanceBucketIndex);
						break;
					}
				}
			}
			if (InstanceBucketIndex == INDEX_NONE)
			{
				InstanceBucketIndex = SelectionIndexBuckets.Emplace();
				ComponentToBucket.Add(Instance.Component, InstanceBucketIndex);
			}
			SelectionIndexBuckets[InstanceBucketIndex].Add(SelectionIndex);
		}
	}

	if (!ensure(SelectionIndexBuckets.Num() > 0))
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(bDeleteFromCurrent ? LOCTEXT("ExtractToNew_Transaction", "Extract To New")
		: LOCTEXT("DuplicateToNew_Transaction", "Duplicate To New"));
	ON_SCOPE_EXIT
	{
		GetToolManager()->EndUndoTransaction();
	};

	// Create new actor
	UWorld* TargetWorld = CurrentSelection[SelectionIndexBuckets[0][0]].Component->GetWorld();
	FString UseBaseName = (ExtractActions->ActorName.Len() > 0) ? ExtractActions->ActorName : TEXT("Instances");
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.ObjectFlags = RF_Transactional;
	AActor* NewActor = TargetWorld ? TargetWorld->SpawnActor<AActor>(SpawnInfo) : nullptr;
	if (!NewActor)
	{
		return;
	}

	FActorLabelUtilities::SetActorLabelUnique(NewActor, UseBaseName);

	// Create a component for each bucket. First one as root and others attached.
	bool bFirst = true;
	for (int32 BucketIndex = 0; BucketIndex < SelectionIndexBuckets.Num(); ++BucketIndex)
	{
		TArray<int32>& SelectionIndices = SelectionIndexBuckets[BucketIndex];
		const FSelectedInstance& RepresentativeItem = CurrentSelection[SelectionIndices[0]];
		if (!RepresentativeItem.Component || !RepresentativeItem.Component->GetStaticMesh())
		{
			continue;
		}

		UE::Geometry::FAxisAlignedBox3d Bounds;
		for (int32 Index : SelectionIndices)
		{
			Bounds.Contain(CurrentSelection[Index].WorldBounds);
		}
		FVector ComponentPosition = Bounds.Center();
		
		TSubclassOf<UInstancedStaticMeshComponent> ComponentClass = (ExtractActions->bCreateHISMs) ?
			UHierarchicalInstancedStaticMeshComponent::StaticClass() : UInstancedStaticMeshComponent::StaticClass();
		FString Suffix = (ExtractActions->bCreateHISMs) ? TEXT("_HISM") : TEXT("_ISM");

		FName UseName = (UseBaseName.Len() > 0) ? FName(UseBaseName + Suffix) : FName();

		UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(NewActor, ComponentClass,
			MakeUniqueObjectName(NewActor, ComponentClass, UseName), RF_Transactional);
		ISMComponent->bHasPerInstanceHitProxies = true;
		ISMComponent->SetStaticMesh(RepresentativeItem.Component->GetStaticMesh());
		TArray<UMaterialInterface*> Materials = RepresentativeItem.Component->GetMaterials();
		for (int32 j = 0; j < Materials.Num(); ++j)
		{
			ISMComponent->SetMaterial(j, Materials[j]);
		}
		ISMComponent->OnComponentCreated();

		if (bFirst)
		{
			NewActor->SetRootComponent(ISMComponent);
			NewActor->SetActorTransform(FTransform(ComponentPosition));

			bFirst = false;
		}
		else
		{
			ISMComponent->AttachToComponent(NewActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
			ISMComponent->SetWorldTransform(FTransform(ComponentPosition));
		}
		NewActor->AddInstanceComponent(ISMComponent);

		for (int32 SelectionIndex : SelectionIndices)
		{
			FTransform InstanceTransform;
			if (ensure(CurrentSelection.IsValidIndex(SelectionIndex))
				&& ensure(CurrentSelection[SelectionIndex].Component->GetInstanceTransform(
					CurrentSelection[SelectionIndex].Index, InstanceTransform, /*bWorldSpace=*/ true)))
			{
				ISMComponent->AddInstance(InstanceTransform, /*bTransformInWorldSpace=*/true);
			}
		}

		ISMComponent->RegisterComponent();
	}
 
	if (bDeleteFromCurrent)
	{
		OnDeleteSelection();
	}
}

void UISMEditorTool::OnExtractToNew()
{
	DuplicateToNew(/*bDeleteFromCurrent=*/ true);
}

void UISMEditorTool::OnReplaceSelection()
{
	if (ReplaceAction->ReplaceWith == nullptr)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnReplaceFailed", "Must set valid replacement StaticMesh"), EToolMessageLevel::UserError);
		return;
	}
	if (CurrentSelection.Num() == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnReplaceFailed2", "Must select at least one Instance"), EToolMessageLevel::UserError);
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("ReplaceSelected", "Replace Selected"));

	UInstancedStaticMeshComponent* DupeComponent = CurrentSelection[0].Component;
	FTransform ExistingTransform = DupeComponent->GetComponentToWorld();

	AActor* ParentActor = DupeComponent->GetOwner();
	ParentActor->Modify();

	// duplicate the component
	TArray<UActorComponent*> SourceComponents;
	SourceComponents.Add(DupeComponent);
	TArray<UActorComponent*> NewComponents;
	GUnrealEd->DuplicateComponents(SourceComponents, NewComponents);
	UInstancedStaticMeshComponent* NewComponent = Cast<UInstancedStaticMeshComponent>(NewComponents[0]);

	NewComponent->Modify();
	NewComponent->SetStaticMesh(ReplaceAction->ReplaceWith);
	NewComponent->SetComponentToWorld(ExistingTransform);
	NewComponent->UnregisterComponent();

	NewComponent->ClearInstances();

	TArray<UInstancedStaticMeshComponent*> UniqueInstances;
	for (FSelectedInstance Instance : CurrentSelection)
	{
		if (Instance.Component->IsValidInstance(Instance.Index))
		{
			UniqueInstances.AddUnique(Instance.Component);
		}
	}
	for (UInstancedStaticMeshComponent* DeleteFromComponent : UniqueInstances)
	{
		TArray<int32> InstanceList;
		TArray<FTransform> WorldTransforms;
		for (FSelectedInstance Instance : CurrentSelection)
		{
			if (Instance.Component == DeleteFromComponent && Instance.Component->IsValidInstance(Instance.Index))
			{
				FTransform InstanceTransform;
				Instance.Component->GetInstanceTransform(Instance.Index, InstanceTransform, true);
				WorldTransforms.Add(InstanceTransform);
				InstanceList.Add(Instance.Index);
			}
		}

		DeleteFromComponent->Modify();
		DeleteFromComponent->RemoveInstances(InstanceList);

		NewComponent->AddInstances(WorldTransforms, false, true);
	}

	NewComponent->RegisterComponent();

	TargetComponents.Add(NewComponent);
	bMeshSceneDirty = true;

	TUniquePtr<FISMEditorSceneChange> SceneChange = MakeUnique<FISMEditorSceneChange>();
	SceneChange->Components.Add(NewComponent);
	SceneChange->bAdded = true;
	GetToolManager()->EmitObjectChange(this, MoveTemp(SceneChange), LOCTEXT("SceneChange", "Scene Update"));
	UpdateSelectionInternal(TArray<FSelectedInstance>(), true);

	GetToolManager()->EndUndoTransaction();

	NewComponent->PostEditChange();
	ParentActor->PostEditChange();
}

void UISMEditorTool::OnContractLastSelection()
{
	if (CurrentSelection.IsEmpty())
	{
		return;
	}
	if (!ensure(CurrentSelection.Last().Component))
	{
		return;
	}

	TArray<FSelectedInstance> NewSelection = CurrentSelection;
	ContractSelectionBy(CurrentSelection.Last(), NewSelection);
	if (NewSelection.Num() != CurrentSelection.Num())
	{
		UpdateSelectionInternal(NewSelection, true);
	}
}

void UISMEditorTool::ContractSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify)
{
	// Find all components that have the same static mesh and materials as the last selected item
	UInstancedStaticMeshComponent* SourceComponent = SelectedInstance.Component;
	UStaticMesh* StaticMesh = SelectedInstance.Component->GetStaticMesh();
	TArray<UMaterialInterface*> Materials = SelectedInstance.Component->GetMaterials();

	TArray<UInstancedStaticMeshComponent*> ComponentsToUnselect = TargetComponents.FilterByPredicate(
		[SourceComponent, StaticMesh, &Materials](UInstancedStaticMeshComponent* Component)
		{
			return SourceComponent == Component
				|| (Component && Component->GetStaticMesh() == StaticMesh && Component->GetMaterials() == Materials);
		});

	// Remove these components from the selection
	SelectionToModify.RemoveAll([&ComponentsToUnselect](const FSelectedInstance& Instance)
	{
		return ComponentsToUnselect.Contains(Instance.Component);
	});
}

void UISMEditorTool::ToggleSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify)
{
	// Find all components that have the same static mesh and materials as the last selected item
	UInstancedStaticMeshComponent* SourceComponent = SelectedInstance.Component;
	UStaticMesh* StaticMesh = SelectedInstance.Component->GetStaticMesh();
	TArray<UMaterialInterface*> Materials = SelectedInstance.Component->GetMaterials();

	TArray<UInstancedStaticMeshComponent*> ComponentsToProcess = TargetComponents.FilterByPredicate(
		[SourceComponent, StaticMesh, &Materials](UInstancedStaticMeshComponent* Component)
		{
			return SourceComponent == Component
				|| (Component && Component->GetStaticMesh() == StaticMesh && Component->GetMaterials() == Materials);
		});

	for (UInstancedStaticMeshComponent* Component : ComponentsToProcess)
	{
		int32 NumIndices = Component->GetInstanceCount();
		for (int32 Index = 0; Index < NumIndices; ++Index)
		{
			if (Component->IsValidInstance(Index))
			{
				FSelectedInstance Instance(Component, Index, MeshScene->GetMeshBoundingBox(Component, Index));
				if (SelectionToModify.RemoveSwap(Instance) == 0)
				{
					SelectionToModify.Add(Instance);
				}
			}
		}
	}
}

void UISMEditorTool::OnExpandLastSelection()
{
	if (CurrentSelection.IsEmpty())
	{
		return;
	}
	if (!ensure(CurrentSelection.Last().Component))
	{
		return;
	}
	TArray<FSelectedInstance> NewSelection = CurrentSelection;
	ExpandSelectionBy(CurrentSelection.Last(), NewSelection);
	if (NewSelection.Num() != CurrentSelection.Num())
	{
		UpdateSelectionInternal(NewSelection, true);
	}
}

void UISMEditorTool::ExpandSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify)
{
	// Find all components that have the same static mesh and materials as the last selected item
	UInstancedStaticMeshComponent* SourceComponent = SelectedInstance.Component;
	UStaticMesh* StaticMesh = SelectedInstance.Component->GetStaticMesh();
	TArray<UMaterialInterface*> Materials = SelectedInstance.Component->GetMaterials();

	TArray<UInstancedStaticMeshComponent*> ComponentsToFullySelect = TargetComponents.FilterByPredicate(
		[SourceComponent, StaticMesh, &Materials](UInstancedStaticMeshComponent* Component)
		{
			return SourceComponent == Component 
				|| (Component && Component->GetStaticMesh() == StaticMesh && Component->GetMaterials() == Materials);
		});

	// Select all instances in these components
	for (UInstancedStaticMeshComponent* Component : ComponentsToFullySelect)
	{
		int32 NumIndices = Component->GetInstanceCount();
		for (int32 Index = 0; Index < NumIndices; ++Index)
		{
			if (Component->IsValidInstance(Index))
			{
				SelectionToModify.AddUnique(FSelectedInstance(Component, Index,
					MeshScene->GetMeshBoundingBox(Component, Index)));
			}
		}
	}
}

void UISMEditorTool::OnExpandAllSelection()
{
	if (CurrentSelection.IsEmpty())
	{
		return;
	}

	TArray<UInstancedStaticMeshComponent*> SourceComponents;
	for (const FSelectedInstance& Instance : CurrentSelection)
	{
		if (ensure(Instance.Component))
		{
			SourceComponents.AddUnique(Instance.Component);
		}
	}
	// Also add any components that have the same static mesh and material
	TArray<UInstancedStaticMeshComponent*> ComponentsToFullySelect = SourceComponents;
	for (UInstancedStaticMeshComponent* TargetComponent : TargetComponents)
	{
		if (!SourceComponents.Contains(TargetComponent))
		{
			for (UInstancedStaticMeshComponent* SourceComponent : SourceComponents)
			{
				if (TargetComponent->GetStaticMesh() == SourceComponent->GetStaticMesh()
					&& TargetComponent->GetMaterials() == SourceComponent->GetMaterials())
				{
					ComponentsToFullySelect.Add(TargetComponent);
					break;
				}
			}
		}
	}

	// Select all instances in these components
	TArray<FSelectedInstance> NewSelection = CurrentSelection;
	for (UInstancedStaticMeshComponent* Component : ComponentsToFullySelect)
	{
		int32 NumIndices = Component->GetInstanceCount();
		for (int32 Index = 0; Index < NumIndices; ++Index)
		{
			if (Component->IsValidInstance(Index))
			{
				NewSelection.AddUnique(FSelectedInstance(Component, Index,
					MeshScene->GetMeshBoundingBox(Component, Index)));
			}
		}
	}

	if (NewSelection.Num() != CurrentSelection.Num())
	{
		UpdateSelectionInternal(NewSelection, true);
	}
}

void UISMEditorTool::RebuildMeshScene()
{
	using namespace UE::Geometry;

	MeshScene = MakeShared<FMeshSceneAdapter>();
	TArray<UActorComponent*> TempComponents;
	for (UInstancedStaticMeshComponent* ISMC : TargetComponents)
	{
		TempComponents.Add(ISMC);
	}
	MeshScene->AddComponents(TempComponents);
	FMeshSceneAdapterBuildOptions BuildOptions;
	BuildOptions.bThickenThinMeshes = false;
	BuildOptions.bFilterTinyObjects = false;
	BuildOptions.bOnlySurfaceMaterials = false;
	BuildOptions.bBuildSpatialDataStructures = true;
	MeshScene->Build(BuildOptions);
	MeshScene->BuildSpatialEvaluationCache();

	AllMeshBoundingBoxes.Reset();
	MeshScene->GetMeshBoundingBoxes(AllMeshBoundingBoxes);
	UpdatePreviewGeometry();
}


void UISMEditorTool::OnTransformCompleted()
{
	if (bMeshSceneDirty == false)
	{
		MeshScene->FastUpdateTransforms(true);
		
		AllMeshBoundingBoxes.Reset();
		MeshScene->GetMeshBoundingBoxes(AllMeshBoundingBoxes);
		UpdatePreviewGeometry();

		ParallelFor(CurrentSelection.Num(), [&](int32 k)
		{
			FSelectedInstance& Instance = CurrentSelection[k];
			Instance.WorldBounds = MeshScene->GetMeshBoundingBox(Instance.Component, Instance.Index);
		});
	}
}



void UISMEditorTool::NotifySceneModified()
{
	bMeshSceneDirty = true;
}

void UISMEditorTool::InternalNotifySceneModified(const TArray<UInstancedStaticMeshComponent*>& ComponentList, bool bAddToScene)
{
	if (bAddToScene)
	{
		for (UInstancedStaticMeshComponent* Component : ComponentList)
		{
			TargetComponents.Add(Component);
		}
	}
	else
	{
		for (UInstancedStaticMeshComponent* Component : ComponentList)
		{
			TargetComponents.Remove(Component);
		}
	}

	NotifySceneModified();
}


void UISMEditorTool::UpdateSelectionInternal(const TArray<FSelectedInstance>& NewSelection, bool bEmitChange)
{
	UpdateSelectionInternal(NewSelection, bEmitChange, LOCTEXT("SelectionChange", "Selection Change"));
}

void UISMEditorTool::UpdateSelectionInternal(const TArray<FSelectedInstance>& NewSelection, bool bEmitChange, const FText& TransactionLabel)
{
	if (bEmitChange)
	{
		TUniquePtr<FISMEditorSelectionChange> SelectionChange = MakeUnique<FISMEditorSelectionChange>();
		SelectionChange->OldSelection = CurrentSelection;
		SelectionChange->NewSelection = NewSelection;
		GetToolManager()->EmitObjectChange(this, MoveTemp(SelectionChange), TransactionLabel);
	}

	CurrentSelection = NewSelection;
	OnSelectionUpdated();
}

void UISMEditorTool::UpdateSelectionFromUndoRedo(const TArray<FSelectedInstance>& NewSelection)
{
	CurrentSelection = NewSelection;
	OnSelectionUpdated();
}


void FISMEditorSelectionChange::Apply(UObject* Object)
{
	Cast<UISMEditorTool>(Object)->UpdateSelectionFromUndoRedo(NewSelection);
}

void FISMEditorSelectionChange::Revert(UObject* Object)
{
	Cast<UISMEditorTool>(Object)->UpdateSelectionFromUndoRedo(OldSelection);
}

FString FISMEditorSelectionChange::ToString() const { return TEXT("FISMEditorSceneChange"); }





void FISMEditorSceneChange::Apply(UObject* Object)
{
	Cast<UISMEditorTool>(Object)->InternalNotifySceneModified(Components, bAdded);
}

void FISMEditorSceneChange::Revert(UObject* Object)
{
	Cast<UISMEditorTool>(Object)->InternalNotifySceneModified(Components, !bAdded);
}

FString FISMEditorSceneChange::ToString() const { return TEXT("FISMEditorSceneChange"); }



#undef LOCTEXT_NAMESPACE
