// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeMeshesTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "Selection/ToolSelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"



#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "CompositionOps/VoxelMergeMeshesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MergeMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMergeMeshesTool"




void UMergeMeshesTool::SetupProperties()
{
	//Super::SetupProperties();
	//RemoveToolPropertySource(VoxProperties);

	UBaseCreateFromSelectedTool::SetupProperties();
	MergeProps = NewObject<UMergeMeshesToolProperties>(this);
	MergeProps->RestoreProperties(this);
	AddToolPropertySource(MergeProps);

	SetToolDisplayName(LOCTEXT("UMergeMeshesToolName", "Voxel Merge Meshes"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Compute a CSG Union of the input meshes using voxelization techniques.UVs, sharp edges, and small / thin features will be lost.Increase Voxel Count to enhance accuracy."),
		EToolMessageLevel::UserNotification);

}

void UMergeMeshesTool::SaveProperties()
{
	UBaseCreateFromSelectedTool::SaveProperties();
	//Super::SaveProperties();
	MergeProps->SaveProperties(this);
}


void UMergeMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	Super::ConvertInputsAndSetPreviewMaterials(bSetPreviewMesh);

	if (HasOpenBoundariesInMeshInputs())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("WarnOpenEdges", "Open edges found: some artifacts will result."),
			EToolMessageLevel::UserWarning);
	}
}

TUniquePtr<FDynamicMeshOperator> UMergeMeshesTool::MakeNewOperator()
{
	TUniquePtr<FVoxelMergeMeshesOp> MergeOp = MakeUnique<FVoxelMergeMeshesOp>();
	MergeOp->VoxelCount = MergeProps->VoxelCount;
	MergeOp->AdaptivityD = MergeProps->MeshAdaptivity;
	MergeOp->IsoSurfaceD = MergeProps->OffsetDistance;
	MergeOp->bAutoSimplify = MergeProps->bAutoSimplify;
	MergeOp->Transforms.SetNum(Targets.Num());
	MergeOp->Meshes.SetNum(Targets.Num());
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		MergeOp->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		MergeOp->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}
	return MergeOp;
}


FString UMergeMeshesTool::GetCreatedAssetName() const
{
	return TEXT("VoxelMergedMeshes");
}

FText UMergeMeshesTool::GetActionName() const
{
	return LOCTEXT("VoxelMergeMeshes", "Voxel Merge Meshes");
}





#undef LOCTEXT_NAMESPACE

