// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshAttributePaintToolV2.h"

#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshAttributePaintToolV2)

DEFINE_LOG_CATEGORY_STATIC(LogMeshAttributePaintToolV2, Warning, All);

#define LOCTEXT_NAMESPACE "MeshAttributePaintToolV2"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// ToolBuilder
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UMeshSurfacePointTool* UMeshAttributePaintToolV2Builder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshAttributePaintToolV2* PaintTool = NewObject<UMeshAttributePaintToolV2>(SceneState.ToolManager);
	PaintTool->SetWorld(SceneState.World);
	return PaintTool;
}

const FToolTargetTypeRequirements& UMeshAttributePaintToolV2Builder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements ToolRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		});
	
	return ToolRequirements;
}

void UMeshAttributePaintToolV2Properties::Initialize(const TArray<FName>& AttributeNames, bool bInitialize)
{
	Attributes.Reset(AttributeNames.Num());
	for (const FName& AttributeName : AttributeNames)
	{
		Attributes.Add(AttributeName.ToString());
	}

	if (bInitialize) {
		Attribute = (Attributes.Num() > 0) ? Attributes[0] : TEXT("");
	}
}

int32 UMeshAttributePaintToolV2Properties::GetSelectedAttributeIndex()
{
	ensure(INDEX_NONE == -1);
	int32 FoundIndex = Attributes.IndexOfByKey(Attribute);
	return FoundIndex;
}

bool UMeshAttributePaintToolV2::SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex)
{
	OutInitialAttributeIndex = INDEX_NONE;
	
	using namespace UE::Geometry;
	FDynamicMeshAttributeSet* Attributes = InOutToolMesh.Attributes();
	
	if (!Attributes)
	{
		return false;
	}

	if (Attributes->NumWeightLayers() == 0)
	{
		return false;
	}

	OutInitialAttributeIndex = 0;

	TArray<FName> AttributeNames;
	const int32 NumLayers = Attributes->NumWeightLayers();
	for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		FDynamicMeshWeightAttribute* WeightLayer = Attributes->GetWeightLayer(LayerIdx);
		AttributeNames.Add(WeightLayer->GetName());
	}

	constexpr bool bSelectInitialSelection = true;
	AttribProps->Initialize(AttributeNames, bSelectInitialSelection);

	return true;
}

void UMeshAttributePaintToolV2::CommitToolMesh(FDynamicMesh3& InToolMesh)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->BeginUndoTransaction(LOCTEXT("MeshAttributePaintToolV2_TransactionName", "Paint Weights"));

		// commit the tool mesh cache instead of the sculpt mesh
		// setting ModifiedTopology to true to force a full dyna mesh update such that weight layer changes
		// are converted to mesh description as well;
		// todo: update weight layers only
		constexpr bool bForceModifiedTopology = true;
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, InToolMesh, bForceModifiedTopology);
		
		ToolManager->EndUndoTransaction();
	}	
}

void UMeshAttributePaintToolV2::Setup()
{
	AttribProps = NewObject<UMeshAttributePaintToolV2Properties>(this);
	AttribProps->RestoreProperties(this);
	AddToolPropertySource(AttribProps);

	AttribProps->SetFlags(RF_Transactional);
	
	Super::Setup();

	SelectedAttributeWatcher.Initialize([this]()
	{
		return AttribProps->GetSelectedAttributeIndex();
	},
[this](int32 NewAttributeIndex)
	{
		SetAttributeToPaint(NewAttributeIndex);
		
	}, AttribProps->GetSelectedAttributeIndex());
}

void UMeshAttributePaintToolV2::OnTick(float DeltaTime)
{
	SelectedAttributeWatcher.CheckAndUpdate();
	
	Super::OnTick(DeltaTime);
}



#undef LOCTEXT_NAMESPACE

