// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshModelingToolsEditorOnlyExp.h"

#define LOCTEXT_NAMESPACE "FMeshModelingToolsEditorOnlyExpModule"

void FMeshModelingToolsEditorOnlyExpModule::StartupModule()
{
}

void FMeshModelingToolsEditorOnlyExpModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMeshModelingToolsEditorOnlyExpModule, MeshModelingToolsEditorOnlyExp)
