// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTangentsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescription.h"
#include "ToolSetupUtil.h"
#include "ToolDataVisualizer.h"

#include "AssetUtils/MeshDescriptionUtil.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"

#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "GroupTopology.h"
#include "DynamicMeshEditor.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selections/GeometrySelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTangentsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshTangentsTool"

/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UMeshTangentsToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		});
	return TypeRequirements;
}

USingleSelectionMeshEditingTool* UMeshTangentsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshTangentsTool>(SceneState.ToolManager);
}

void UMeshTangentsToolBuilder::InitializeNewTool(USingleSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	NewTool->SetTarget(Target);
	NewTool->SetWorld(SceneState.World);

	if (UMeshTangentsTool* TangentsTool = Cast<UMeshTangentsTool>(NewTool))
	{
		UE::Geometry::FGeometrySelection Selection;
		if (UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Target, Selection))
		{
			TangentsTool->SetGeometrySelection(MoveTemp(Selection));
		}
	}
}

bool UMeshTangentsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

/*
 * Tool
 */
UMeshTangentsTool::UMeshTangentsTool()
{
}



void UMeshTangentsTool::Setup()
{
	UInteractiveTool::Setup();

	UE::ToolTarget::HideSourceObject(Target);

	// make our preview mesh
	AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(TargetActor->GetWorld(), FTransform::Identity);
	PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);
	// configure materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);
	// configure mesh
	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	FDynamicMesh3 InputMeshWithTangents = UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams);
	PreviewMesh->ReplaceMesh(MoveTemp(InputMeshWithTangents));

	// make a copy of initialized mesh and tangents
	PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(ReadMesh);
		InitialTangents = MakeShared<FMeshTangentsf, ESPMode::ThreadSafe>(InputMesh.Get());
		InitialTangents->CopyTriVertexTangents(ReadMesh);
	});

	// initialize our properties
	Settings = NewObject<UMeshTangentsToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
	UVChannelProperties->RestoreProperties(this);
	PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { UVChannelProperties->Initialize(&ReadMesh, false); });
	UVChannelProperties->ValidateSelection(true);
	AddToolPropertySource(UVChannelProperties);

	Settings->WatchProperty(Settings->CalculationMethod, [this](EMeshTangentsType) { Compute->InvalidateResult(); });
	Settings->WatchProperty(Settings->LineLength, [this](float) { bLengthDirty = true; });
	Settings->WatchProperty(Settings->LineThickness, [this](float) { bThicknessDirty = true; });
	Settings->WatchProperty(Settings->bShowTangents, [this](bool) { bVisibilityChanged = true; });
	Settings->WatchProperty(Settings->bShowNormals, [this](bool) { bVisibilityChanged = true; });
	Settings->WatchProperty(Settings->bCompareWithMikkt, [this](bool) { ComputeMikkTDeviations(ComputeDegenerateTris()); });
	UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue) { Compute->InvalidateResult(); });

	if (InputGeometrySelection.IsEmpty() == false)
	{
		GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
		GeometrySelectionVizProperties->RestoreProperties(this);
		AddToolPropertySource(GeometrySelectionVizProperties);
		GeometrySelectionVizProperties->Initialize(this);
		GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(InputGeometrySelection.ElementType);
		GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(InputGeometrySelection.TopologyType);
		GeometrySelectionVizProperties->bEnableShowEdgeSelectionVertices = true;
		// TODO Enable this but note we need to compute a ROI which only includes triangles incident to the
		//      polygroup feature eg do not include all triangles in the groups incident to a polygroup edge
		//GeometrySelectionVizProperties->bEnableShowTriangleROIBorder = true;

		// Compute group topology if the selection has Polygroup topology, and do nothing otherwise
		// Currently it is only possible to make a polygroup geometry selection using polygroup set stored directly in the mesh
		FGroupTopology GroupTopology(InputMesh.Get(), InputGeometrySelection.TopologyType == EGeometryTopologyType::Polygroup);

		// Compute the overlay selection and a proxy triangle vertex selection used to make edge selections behave like
		// vertex selections. See :EdgeSelectionsBehaveLikeVertexSelections
		if (InputGeometrySelection.TopologyType == EGeometryTopologyType::Polygroup)
		{
			ConvertPolygroupSelectionToIncidentOverlaySelection(
				*InputMesh,
				GroupTopology,
				InputGeometrySelection,
				EditTriangles,
				EditVertices,
				&TriangleVertexGeometrySelection);
		}
		else
		{
			ConvertTriangleSelectionToOverlaySelection(
				*InputMesh,
				InputGeometrySelection,
				EditTriangles,
				EditVertices,
				&TriangleVertexGeometrySelection);
		}

		// Setup input geometry selection visualization
		FTransform ApplyTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
		GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
		GeometrySelectionViz->CreateInWorld(GetTargetWorld(), ApplyTransform);
		InitializeGeometrySelectionVisualization(
			GeometrySelectionViz,
			GeometrySelectionVizProperties,
			*InputMesh,
			InputGeometrySelection,
			&GroupTopology,
			!TriangleVertexGeometrySelection.IsEmpty() ? &TriangleVertexGeometrySelection : nullptr);
	}

	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(TargetActor->GetWorld(), PreviewMesh->GetTransform());

	Compute = MakeUnique<TGenericDataBackgroundCompute<FMeshTangentsd>>();
	Compute->Setup(this);
	Compute->OnOpCompleted.AddLambda([this](const UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshTangentsd>* Op)
		{
			if (((FCalculateTangentsOp*)(Op))->bNoAttributesError)
			{
				bHasDisplayedNoAttributeError = true;
				GetToolManager()->DisplayMessage(
					LOCTEXT("TangentsNoAttributesError", "Error: Source mesh did not have tangents."),
					EToolMessageLevel::UserWarning);
			}
			else if (bHasDisplayedNoAttributeError)
			{
				bHasDisplayedNoAttributeError = false;
				GetToolManager()->DisplayMessage(
					FText(),
					EToolMessageLevel::UserWarning);
			}
		});
	Compute->OnResultUpdated.AddLambda( [this](const TUniquePtr<FMeshTangentsd>& NewResult) { OnTangentsUpdated(NewResult); } );
	Compute->InvalidateResult();

	SetToolDisplayName(LOCTEXT("ToolName", "Edit Tangents"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Configure or Recalculate Tangents on a Static Mesh Asset (disables autogenerated Tangents and Normals)"),
		EToolMessageLevel::UserNotification);
}

void UMeshTangentsTool::SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn)
{
	InputGeometrySelection = MoveTemp(SelectionIn);
}



void UMeshTangentsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	PreviewGeometry->Disconnect();
	PreviewMesh->Disconnect();

	Settings->SaveProperties(this);
	UVChannelProperties->SaveProperties(this);

	if (GeometrySelectionViz)
	{
		GeometrySelectionViz->Disconnect();
	}

	if (GeometrySelectionVizProperties)
	{
		GeometrySelectionVizProperties->SaveProperties(this);
	}

	// Restore (unhide) the source meshes
	UE::ToolTarget::ShowSourceObject(Target);

	TUniquePtr<FMeshTangentsd> Tangents = Compute->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("UpdateTangents", "Update Tangents"));

		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Target));
		if (StaticMeshComponent != nullptr)
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			if (ensure(StaticMesh != nullptr))
			{
				StaticMesh->Modify();

				// disable auto-generated normals and tangents build settings
				UE::MeshDescription::FStaticMeshBuildSettingChange SettingsChange;
				SettingsChange.AutoGeneratedNormals = UE::MeshDescription::EBuildSettingBoolChange::Disable;
				SettingsChange.AutoGeneratedTangents = UE::MeshDescription::EBuildSettingBoolChange::Disable;
				UE::MeshDescription::ConfigureBuildSettings(StaticMesh, 0, SettingsChange);
			}
		}

		CopyToOverlays(*Tangents, *InputMesh);

		FConversionToMeshDescriptionOptions Options;
		Options.bUpdatePositions = false;
		Options.bUpdateNormals = true;
		Options.bUpdateTangents = true;
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, *InputMesh, false, Options);

		GetToolManager()->EndUndoTransaction();
	}
}

void UMeshTangentsTool::OnTick(float DeltaTime)
{
	Compute->Tick(DeltaTime);

	if (bThicknessDirty || bLengthDirty || bVisibilityChanged)
	{
		UpdateVisualization(bThicknessDirty, bLengthDirty);
		bThicknessDirty = bLengthDirty = bVisibilityChanged = false;
	}

	if (GeometrySelectionViz)
	{
		UpdateGeometrySelectionVisualization(GeometrySelectionViz, GeometrySelectionVizProperties);
	}
}


void UMeshTangentsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (Settings->bCompareWithMikkt && Deviations.Num() > 0)
	{
		FToolDataVisualizer Visualizer;
		Visualizer.BeginFrame(RenderAPI);
		Visualizer.SetTransform(PreviewMesh->GetTransform());
		for (const FMikktDeviation& ErrorPt : Deviations)
		{
			if (ErrorPt.MaxAngleDeg > Settings->CompareWithMikktThreshold)
			{
				Visualizer.DrawPoint(ErrorPt.VertexPos, FLinearColor(0.95f, 0.05f, 0.05f), 4.0f * Settings->LineThickness, false);
				Visualizer.DrawLine<FVector3f>(ErrorPt.VertexPos, ErrorPt.VertexPos + Settings->LineLength * ErrorPt.MikktTangent, FLinearColor(0.95f, 0.05f, 0.05f), 2.0f * Settings->LineThickness, false);
				Visualizer.DrawLine<FVector3f>(ErrorPt.VertexPos, ErrorPt.VertexPos + Settings->LineLength * ErrorPt.MikktBitangent, FLinearColor(0.05f, 0.95f, 0.05f), 2.0f * Settings->LineThickness, false);

				Visualizer.DrawLine<FVector3f>(ErrorPt.VertexPos, ErrorPt.VertexPos + (1.1f * Settings->LineLength) * ErrorPt.OtherTangent, FLinearColor(0.95f, 0.50f, 0.05f), Settings->LineThickness, false);
				Visualizer.DrawLine<FVector3f>(ErrorPt.VertexPos, ErrorPt.VertexPos + (1.1f * Settings->LineLength) * ErrorPt.OtherBitangent, FLinearColor(0.05f, 0.95f, 0.95f), Settings->LineThickness, false);
			}
		}
		Visualizer.EndFrame();
	}
}


bool UMeshTangentsTool::CanAccept() const
{
	return Super::CanAccept() && Compute->HaveValidResult();
}


TUniquePtr<TGenericDataOperator<FMeshTangentsd>> UMeshTangentsTool::MakeNewOperator()
{
	TUniquePtr<FCalculateTangentsOp> TangentsOp = MakeUnique<FCalculateTangentsOp>();

	TangentsOp->SourceMesh = InputMesh;
	TangentsOp->SourceTangents = InitialTangents;
	TangentsOp->CalculationMethod = Settings->CalculationMethod;
	TangentsOp->TargetUVLayer = UVChannelProperties->GetSelectedChannelIndex(true /*use 0 on failure*/);

	return TangentsOp;
}


void UMeshTangentsTool::UpdateVisualization(bool bThicknessChanged, bool bLengthChanged)
{
	ULineSetComponent* TangentLines = PreviewGeometry->FindLineSet(TEXT("Tangents"));
	ULineSetComponent* NormalLines = PreviewGeometry->FindLineSet(TEXT("Normals"));
	if (TangentLines == nullptr || NormalLines == nullptr)
	{
		return;
	}

	if (bThicknessChanged)
	{
		float Thickness = Settings->LineThickness;
		TangentLines->SetAllLinesThickness(Thickness);
		NormalLines->SetAllLinesThickness(Thickness);
	}

	if (bLengthChanged)
	{
		float LineLength = Settings->LineLength;
		TangentLines->SetAllLinesLength(LineLength);
		NormalLines->SetAllLinesLength(LineLength);
	}

	PreviewGeometry->SetLineSetVisibility(TEXT("Tangents"), Settings->bShowTangents);
	PreviewGeometry->SetLineSetVisibility(TEXT("Normals"), Settings->bShowNormals);
}


void UMeshTangentsTool::OnTangentsUpdated(const TUniquePtr<FMeshTangentsd>& NewResult)
{
	const float LineLength = Settings->LineLength;
	const float Thickness = Settings->LineThickness;

	const TSet<int32> DegenerateTris = ComputeDegenerateTris();

	// update Tangents rendering line set
	PreviewGeometry->CreateOrUpdateLineSet(TEXT("Tangents"), InputMesh->MaxTriangleID(),
		[&](int32 Index, TArray<FRenderableLine>& Lines) 
	{
		bool bValid = InputMesh->IsTriangle(Index) && !DegenerateTris.Contains(Index);
		bool bIncludedTriangle = EditTriangles.IsEmpty() || EditTriangles.Contains(Index);
		if (bValid && bIncludedTriangle)
		{
			FIndex3i Vids = InputMesh->GetTriangle(Index);
			for (int j = 0; j < 3; ++j)
			{
				bool bIncludedVertex = EditVertices.IsEmpty() || EditVertices.Contains(Vids[j]);
				if (bIncludedVertex)
				{
					FVector3d Origin = InputMesh->GetVertex(Vids[j]);

					FVector3d Tangent, Bitangent;
					NewResult->GetPerTriangleTangent(Index, j, Tangent, Bitangent);

					Lines.Add(FRenderableLine((FVector)Origin, (FVector)Origin + LineLength * (FVector)Tangent, FColor(240,15,15), Thickness));
					Lines.Add(FRenderableLine((FVector)Origin, (FVector)Origin + LineLength * (FVector)Bitangent, FColor(15,240,15), Thickness));
				}
			}
		}
	}, 6);


	// update Normals rendering line set
	const FDynamicMeshNormalOverlay* NormalOverlay = InputMesh->Attributes()->PrimaryNormals();
	PreviewGeometry->CreateOrUpdateLineSet(TEXT("Normals"), NormalOverlay->MaxElementID(),
		[&](int32 Index, TArray<FRenderableLine>& Lines) 
	{
		if (NormalOverlay->IsElement(Index))
		{
			int32 ParentVtx = NormalOverlay->GetParentVertex(Index);
			bool bIncluded = InputGeometrySelection.IsEmpty() || EditVertices.Contains(ParentVtx);
			if (bIncluded)
			{
				FVector3f Normal = NormalOverlay->GetElement(Index);
				FVector3f Origin = (FVector3f)InputMesh->GetVertex(ParentVtx);
				Lines.Add(FRenderableLine((FVector)Origin, (FVector)Origin + LineLength * (FVector)Normal, FColor(15,15,240), Thickness));
			}
		}
	}, 1);

	ComputeMikkTDeviations(DegenerateTris);

	PreviewMesh->DeferredEditMesh([&](FDynamicMesh3& EditMesh)
	{
		CopyToOverlays(*NewResult, EditMesh);
	}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::VertexNormals, false);

	UpdateVisualization(false, false);
}


void UMeshTangentsTool::CopyToOverlays(const FMeshTangentsd& Tangents, FDynamicMesh3& Mesh)
{
	if (ensure(Mesh.HasAttributes()) == false || ensure(Mesh.Attributes()->NumNormalLayers() == 3) == false)
	{
		return;
	}
	
	if (InputGeometrySelection.IsEmpty())
	{
		Tangents.CopyToOverlays(Mesh);
	}
	else
	{
		FDynamicMesh3 MeshCopy;
		{
			bool bCopyNormals = false;
			bool bCopyColors = false;
			bool bCopyUVs = false;
			bool bCopyAttributes = true;
			MeshCopy.Copy(Mesh, bCopyNormals, bCopyColors, bCopyUVs, bCopyAttributes);
		}

		// Copy all tangents to the MeshCopy overlays
		Tangents.CopyToOverlays(MeshCopy);

		// Copy the subset of tangents corresponding to the geometry selection to the Mesh overlays
		FDynamicMeshEditor Editor(&Mesh);
		Editor.AppendElementSubset(
			&MeshCopy,
			EditTriangles,
			EditVertices,
			MeshCopy.Attributes()->PrimaryTangents(),
			Mesh.Attributes()->PrimaryTangents());
		Editor.AppendElementSubset(
			&MeshCopy,
			EditTriangles,
			EditVertices,
			MeshCopy.Attributes()->PrimaryBiTangents(),
			Mesh.Attributes()->PrimaryBiTangents());
	}
}

TSet<int32> UMeshTangentsTool::ComputeDegenerateTris() const
{
	TSet<int32> DegenerateTris;
	if (!Settings->bShowDegenerates || (Settings->bCompareWithMikkt && Settings->CalculationMethod != EMeshTangentsType::MikkTSpace))
	{
		FMeshTangentsd DegenTangents(InputMesh.Get());
		int32 UVLayer = UVChannelProperties->GetSelectedChannelIndex(true /*use 0 on failure*/);
		DegenTangents.ComputeTriangleTangents(InputMesh->Attributes()->GetUVLayer(UVLayer));
		DegenerateTris = TSet<int32>(DegenTangents.GetDegenerateTris());
	}
	return DegenerateTris;
}


void UMeshTangentsTool::ComputeMikkTDeviations(const TSet<int32>& DegenerateTris)
{
	// calculate deviation between what we have and MikkT, if necessary
	Deviations.Reset();
	if (Settings->bCompareWithMikkt && Settings->CalculationMethod == EMeshTangentsType::FastMikkTSpace)
	{
		int32 UVLayer = UVChannelProperties->GetSelectedChannelIndex(true /*use 0 on failure*/);
		
		FProgressCancel TmpCancel;
		FCalculateTangentsOp MikktOp;
		MikktOp.SourceMesh = InputMesh;
		MikktOp.CalculationMethod = EMeshTangentsType::MikkTSpace;
		MikktOp.TargetUVLayer = UVLayer;
		MikktOp.CalculateResult(&TmpCancel);
		TUniquePtr<FMeshTangentsd> MikktTangents = MikktOp.ExtractResult();

		FCalculateTangentsOp NewOp;
		NewOp.SourceMesh = InputMesh;
		NewOp.CalculationMethod = EMeshTangentsType::FastMikkTSpace;
		NewOp.TargetUVLayer = UVLayer;
		NewOp.CalculateResult(&TmpCancel);
		TUniquePtr<FMeshTangentsd> NewTangents = NewOp.ExtractResult();

		for (int32 Index : InputMesh->TriangleIndicesItr())
		{
			bool bValid = !DegenerateTris.Contains(Index);
			bool bIncludedTriangle = EditTriangles.IsEmpty() || EditTriangles.Contains(Index);
			if (bValid && bIncludedTriangle)
			{
				FIndex3i Vids = InputMesh->GetTriangle(Index);
				for (int j = 0; j < 3; ++j)
				{
					bool bIncludedVertex = EditVertices.IsEmpty() || EditVertices.Contains(Vids[j]);
					if (bIncludedVertex)
					{
						FVector3f TangentMikkt, BitangentMikkt;
						MikktTangents->GetPerTriangleTangent<FVector3f, float>(Index, j, TangentMikkt, BitangentMikkt);
						UE::Geometry::Normalize(TangentMikkt);
						UE::Geometry::Normalize(BitangentMikkt);
						ensure(UE::Geometry::IsNormalized(TangentMikkt));
						ensure(UE::Geometry::IsNormalized(BitangentMikkt));

						FVector3f TangentNew, BitangentNew;
						NewTangents->GetPerTriangleTangent<FVector3f, float>(Index, j, TangentNew, BitangentNew);
						UE::Geometry::Normalize(TangentNew); 
						UE::Geometry::Normalize(BitangentNew);
						ensure(UE::Geometry::IsNormalized(TangentNew));
						ensure(UE::Geometry::IsNormalized(BitangentNew));

						float TangentAngleDeg = UE::Geometry::AngleD(TangentMikkt, TangentNew);
						float BiTangentAngleDeg = UE::Geometry::AngleD(BitangentMikkt, BitangentNew);
						float MaxAngleDeg = FMathf::Max(TangentAngleDeg, BiTangentAngleDeg);

						if (MaxAngleDeg > 0.5f)
						{
							FMikktDeviation Deviation;

							Deviation.MaxAngleDeg    = MaxAngleDeg;
							Deviation.TriangleID     = Index;
							Deviation.TriVertIndex   = j;
							Deviation.VertexPos      = (FVector3f)InputMesh->GetVertex(Vids[j]);
							Deviation.MikktTangent   = TangentMikkt;
							Deviation.MikktBitangent = BitangentMikkt;
							Deviation.OtherTangent   = TangentNew;
							Deviation.OtherBitangent = BitangentNew;
							
							Deviations.Add(Deviation);
						}
					}
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

