// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshToVolumeTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/World.h"
#include "MeshSimplification.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingToolTargetUtil.h"

#include "Engine/BlockingVolume.h"
#include "Components/BrushComponent.h"
#include "Engine/Polys.h"
#include "Model.h"
#include "BSPOps.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshToVolumeTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshToVolumeTool"

/*
 * ToolBuilder
 */

bool UMeshToVolumeToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// We don't want to allow this tool to run on selected volumes
	return ToolBuilderUtil::CountSelectedActorsOfType<AVolume>(SceneState) == 0 && Super::CanBuildTool(SceneState);
}

USingleSelectionMeshEditingTool* UMeshToVolumeToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshToVolumeTool>(SceneState.ToolManager);
}


class FCalculateVolumeOp : public TGenericDataOperator<FDynamicMeshFaceArray>
{
public:
	virtual ~FCalculateVolumeOp() {}

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMesh;

	// parameters
	EMeshToVolumeMode ConversionMode;
	UE::Conversion::FMeshToVolumeOptions MeshToVolumeOptions;

	// error flags
	bool bTooManyTriangles = false;

	//
	// TGenericDataOperator implementation
	// 

	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		auto SimplifyToMax = [this](FDynamicMesh3& Mesh, int32 TriangleCount)
		{
			FVolPresMeshSimplification Simplifier(&Mesh);
			Simplifier.SimplifyToTriangleCount(TriangleCount);
		};

		if (ConversionMode == EMeshToVolumeMode::MinimalPolygons)
		{
			UE::Conversion::GetPolygonFaces(*SourceMesh, MeshToVolumeOptions, *Result);
		}
		else
		{
			bTooManyTriangles = MeshToVolumeOptions.bAutoSimplify 
				&& (SourceMesh->TriangleCount() > MeshToVolumeOptions.MaxTriangles);

			if (Progress && Progress->Cancelled())
			{
				return;
			}

			if (bTooManyTriangles)
			{
				FDynamicMesh3 LocalMesh = *SourceMesh;
				LocalMesh.DiscardAttributes();
				SimplifyToMax(LocalMesh, MeshToVolumeOptions.MaxTriangles);
				UE::Conversion::GetTriangleFaces(LocalMesh, *Result);
			}
			else
			{
				UE::Conversion::GetTriangleFaces(*SourceMesh, *Result);
			}
		}
	}
};


/*
 * Tool
 */
UMeshToVolumeTool::UMeshToVolumeTool()
{
	SetToolDisplayName(LOCTEXT("MeshToVolumeToolName", "Mesh To Volume"));
}


void UMeshToVolumeTool::Setup()
{
	UInteractiveTool::Setup();

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(UE::ToolTarget::GetTargetActor(Target)->GetWorld(), FTransform::Identity);
	PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	PreviewMesh->ReplaceMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	InputMesh->Copy(*PreviewMesh->GetMesh());

	VolumeEdgesSet = NewObject<ULineSetComponent>(PreviewMesh->GetRootComponent());
	VolumeEdgesSet->SetupAttachment(PreviewMesh->GetRootComponent());
	VolumeEdgesSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	VolumeEdgesSet->RegisterComponent();

	UE::ToolTarget::HideSourceObject(Target);

	Settings = NewObject<UMeshToVolumeToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->ConversionMode, 
		[this](EMeshToVolumeMode NewMode) { Compute->InvalidateResult(); });
	Settings->WatchProperty(Settings->bPreserveGroupBoundaries, 
		[this](bool NewValue) { Compute->InvalidateResult(); });
	Settings->WatchProperty(Settings->bAutoSimplify,
		[this](bool NewValue) { Compute->InvalidateResult(); });
	Settings->WatchProperty(Settings->SimplifyMaxTriangles,
		[this](int32 NewValue) { Compute->InvalidateResult(); });


	HandleSourcesProperties = NewObject<UOnAcceptHandleSourcesPropertiesSingle>(this);
	HandleSourcesProperties->RestoreProperties(this);
	AddToolPropertySource(HandleSourcesProperties);

	Compute = MakeUnique<TGenericDataBackgroundCompute<FDynamicMeshFaceArray>>();
	Compute->Setup(this);
	Compute->OnResultUpdated.AddLambda([this](const TUniquePtr<FDynamicMeshFaceArray>& NewResult)
	{
		UpdateLineSet(*NewResult);
	});
	Compute->OnOpCompleted.AddLambda([this](const TGenericDataOperator<FDynamicMeshFaceArray>* UncastOp)
	{
		const FCalculateVolumeOp* Op = static_cast<const FCalculateVolumeOp*>(UncastOp);
		if (Op->bTooManyTriangles)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("LargeFaceCount", "Mesh has large face count; output Volume representation has been automatically simplified"),
				EToolMessageLevel::UserWarning);
		}
		else
		{
			GetToolManager()->DisplayMessage({}, EToolMessageLevel::UserWarning);
		}
	});
	Compute->InvalidateResult();
	

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Convert a Static Mesh to a Volume, or update an existing Volume"),
		EToolMessageLevel::UserNotification);

	// check for errors in input mesh
	if (auto It = InputMesh->BoundaryEdgeIndicesItr(); It.begin() != It.end())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OpenBoundaryEdges", "Input Mesh is non-Closed and may produce a broken Volume"),
			EToolMessageLevel::UserWarning);
	}
}

void UMeshToVolumeTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	HandleSourcesProperties->SaveProperties(this);

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	UE::ToolTarget::ShowSourceObject(Target);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshToVolumeToolTransactionName", "Create Volume"));

		AActor* TargetOwnerActor = UE::ToolTarget::GetTargetActor(Target);
		UWorld* TargetOwnerWorld = TargetOwnerActor->GetWorld();
		FTransform SetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);

		AVolume* TargetVolume = nullptr;

		const bool bHasTargetVolume = Settings->TargetVolume.IsValid();
		if (bHasTargetVolume == false)
		{
			FRotator Rotation(0.0f, 0.0f, 0.0f);
			FActorSpawnParameters SpawnInfo;
			FTransform NewActorTransform = FTransform::Identity;
			UClass* VolumeClass = Settings->NewVolumeType.Get();
			if (VolumeClass)
			{
				TargetVolume = (AVolume*)TargetOwnerWorld->SpawnActor(VolumeClass, &NewActorTransform, SpawnInfo);
			}
			else
			{
				TargetVolume = TargetOwnerWorld->SpawnActor<ABlockingVolume>(FVector::ZeroVector, Rotation, SpawnInfo);
			}
			TargetVolume->BrushType = EBrushType::Brush_Add;
			UModel* Model = NewObject<UModel>(TargetVolume);
			TargetVolume->Brush = Model;
			TargetVolume->GetBrushComponent()->Brush = TargetVolume->Brush;
		}
		else
		{
			TargetVolume = Settings->TargetVolume.Get();
			SetTransform = TargetVolume->GetActorTransform();
			TargetVolume->Modify(true);
			TargetVolume->GetBrushComponent()->Modify();
		}

		// Note: InputMesh parameter not actually used by DynamicMeshToVolume
		TUniquePtr<FDynamicMeshFaceArray> Faces = Compute->Shutdown();
		UE::Conversion::DynamicMeshToVolume(*InputMesh, *Faces, TargetVolume);
		TargetVolume->SetActorTransform(SetTransform);
		TargetVolume->PostEditChange();

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), TargetVolume);

		TArray<AActor*> Actors;
		Actors.Add(TargetOwnerActor);
		HandleSourcesProperties->ApplyMethod(Actors, GetToolManager());

		GetToolManager()->EndUndoTransaction();
	}
}

void UMeshToVolumeTool::OnTick(float DeltaTime)
{
	Compute->Tick(DeltaTime);
}

void UMeshToVolumeTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

bool UMeshToVolumeTool::CanAccept() const
{
	return Super::CanAccept() && Compute->HaveValidResult();
}


void UMeshToVolumeTool::UpdateLineSet(FDynamicMeshFaceArray& FaceArr)
{
	FColor BoundaryEdgeColor(240, 15, 15);
	float BoundaryEdgeThickness = 0.5;
	float BoundaryEdgeDepthBias = 2.0f;

	VolumeEdgesSet->Clear();
	for (const UE::Conversion::FDynamicMeshFace& Face : FaceArr)
	{
		int32 NumV = Face.BoundaryLoop.Num();
		for (int32 k = 0; k < NumV; ++k)
		{
			VolumeEdgesSet->AddLine(
				(FVector)Face.BoundaryLoop[k], (FVector)Face.BoundaryLoop[(k+1)%NumV],
				BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}

}

TUniquePtr<UE::Geometry::TGenericDataOperator<FDynamicMeshFaceArray>> UMeshToVolumeTool::MakeNewOperator()
{
	TUniquePtr<FCalculateVolumeOp> VolumeOp = MakeUnique<FCalculateVolumeOp>();

	VolumeOp->SourceMesh = InputMesh;
	VolumeOp->ConversionMode = Settings->ConversionMode;

	VolumeOp->MeshToVolumeOptions.bAutoSimplify = Settings->bAutoSimplify;
	VolumeOp->MeshToVolumeOptions.MaxTriangles = Settings->SimplifyMaxTriangles;
	VolumeOp->MeshToVolumeOptions.bRespectGroupBoundaries = Settings->bPreserveGroupBoundaries;

	return VolumeOp;
}


#undef LOCTEXT_NAMESPACE

