// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexAttributePaintBrushOps.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexAttributePaintBrushOps)


bool FMeshVertexAttributePaintToolBrushOpBase::IgnoreZeroMovements() const
{
	return true;
}

//
// Erase Brush
//


void FMeshVertexAttributePaintToolEraseBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	TArray<double>& InOutROIBuffer)
{
	UMeshVertexAttributePaintToolEraseBrushOpProps* Props = GetPropertySetAs<UMeshVertexAttributePaintToolEraseBrushOpProps>();
	const double EraseAttribute = (double)Props->GetAttribute();

	// TODO: Add something here to get the old value so we can subtract (clamped) the AttributeValue from it.
	// TODO: Handle the stamp's properties for fall off, etc..

	check(InOutROIBuffer.Num() == Vertices.Num());

	for (int32 k = 0; k < Vertices.Num(); ++k)
	{
		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const int32 VertIdx = Vertices[k];
			const FVector3d VertexPos = Mesh->GetVertex(VertIdx);
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				continue;
			}
		}
		InOutROIBuffer[k] = EraseAttribute;
	}
}


//
// Smooth Brush
//

void FMeshVertexAttributePaintToolSmoothBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	TArray<double>& InOutROIBuffer)
{
	// If the tool didn't bind a cotan provider, fall back to uniform-weight smoothing: the static
	// kernel treats an empty SmoothEdgeWeights array as "give every neighbor weight 1.0".
	static const TArray<double> EmptySmoothEdgeWeights;
	const TArray<double>& SmoothEdgeWeights = GetSmoothEdgeWeights ? GetSmoothEdgeWeights() : EmptySmoothEdgeWeights;
	ApplyStampByVerticesStatic(Mesh, Stamp, Vertices, GetVertexWeightFunc, SmoothEdgeWeights, InOutROIBuffer, bApplyRadiusLimit, Falloff);
}


/*static*/ void FMeshVertexAttributePaintToolSmoothBrushOp::ApplyStampByVerticesStatic(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	const TArray<double>& SmoothEdgeWeights,
	TArray<double>& InOutROIBuffer,
	bool bApplyRadiusLimit,
	TSharedPtr<FMeshSculptFallofFunc> FalloffFunc)
{
	auto FalloffFunction = [&Stamp, &FalloffFunc](const FVector& Position)
		{
			if (FalloffFunc)
			{
				return (float)FalloffFunc->Evaluate(Stamp, Position);
			}
			return 1.0f;
		};
	FMeshVertexAttributePaintToolSmoothBrushOp::ApplyToVerticesStatic(
		Mesh, Vertices, GetVertexWeightFunc, SmoothEdgeWeights, InOutROIBuffer, Stamp.Power,
		bApplyRadiusLimit, Stamp.LocalFrame.Origin, Stamp.Radius, FalloffFunction);
}

/*static*/ void FMeshVertexAttributePaintToolSmoothBrushOp::ApplyToVerticesStatic(
	const FDynamicMesh3* Mesh,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	const TArray<double>& SmoothEdgeWeights,
	TArray<double>& InOutROIBuffer,
	const double SmoothingFactor,
	bool bApplyRadiusLimit,
	const FVector Center,
	const float Radius,
	FalloffFunction FalloffFunc
)
{
	const int32 NumVertices = Vertices.Num();
	check(InOutROIBuffer.Num() == NumVertices);


	const bool bUseSmoothEdgeWeights = (SmoothEdgeWeights.Num() > 0);

	ParallelFor(NumVertices,
		[&Vertices, &Mesh, &InOutROIBuffer, &GetVertexWeightFunc, &SmoothEdgeWeights, bUseSmoothEdgeWeights,
		 bApplyRadiusLimit, &Center, &Radius, &SmoothingFactor, &FalloffFunc](int32 BufferIndex)
		{
			const int32 VertexIndex = Vertices[BufferIndex];

			float FalloffScalar = 1.0;
			if (bApplyRadiusLimit)
			{
				const FVector3d VertexPos = Mesh->GetVertex(VertexIndex);
				const double DistanceSquared = (VertexPos - Center).SquaredLength();
				if (DistanceSquared >= Radius * Radius)
				{
					return;
				}
				if (FalloffFunc)
				{
					FalloffScalar = FalloffFunc(VertexPos);
				}
			}

			// Cotan-weighted one-ring average. Cotan weights can be negative; clamp to keep the
			// operator on the simplex (mirrors SmoothBoneWeights.cpp / SmoothDynamicMeshAttributes.cpp).
			double WeightedSum = 0.0;
			double TotalWeight = 0.0;
			for (const int32 NeighborIndex : Mesh->VtxVerticesItr(VertexIndex))
			{
				double EdgeWeight = 1.0;
				if (bUseSmoothEdgeWeights)
				{
					const int32 EdgeId = Mesh->FindEdge(VertexIndex, NeighborIndex);
					if (EdgeId == FDynamicMesh3::InvalidID || !SmoothEdgeWeights.IsValidIndex(EdgeId))
					{
						continue;
					}
					EdgeWeight = FMath::Max(0.0, SmoothEdgeWeights[EdgeId]);
					if (EdgeWeight <= 0.0)
					{
						continue;
					}
				}
				WeightedSum += GetVertexWeightFunc(NeighborIndex) * EdgeWeight;
				TotalWeight += EdgeWeight;
			}

			const double CurrentValue = GetVertexWeightFunc(VertexIndex);
			const double OneRingAverage = (TotalWeight > 0.0) ? (WeightedSum / TotalWeight) : CurrentValue;
			const double NewValue = FMath::Lerp(CurrentValue, OneRingAverage, SmoothingFactor * FalloffScalar);
			InOutROIBuffer[BufferIndex] = FMath::Clamp(NewValue, 0.0, 1.0);
		},
		EParallelForFlags::None
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMeshVertexAttributePaintToolPaintBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	TArray<double>& InOutROIBuffer)
{
	check(InOutROIBuffer.Num() == Vertices.Num());

	if (UMeshVertexAttributePaintToolPaintBrushOpProps* Props = GetPropertySetAs<UMeshVertexAttributePaintToolPaintBrushOpProps>())
	{
		ApplyStampByVerticesStatic(Mesh, Stamp, Vertices, GetVertexWeightFunc, InOutROIBuffer, Props->EditOperation, Props->GetAttribute(), bApplyRadiusLimit, Falloff);
	}
}

/*static*/ void FMeshVertexAttributePaintToolPaintBrushOp::ApplyStampByVerticesStatic(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	TArray<double>& InOutROIBuffer,
	EMeshVertexAttributePaintToolEditOperation EditOperation,
	const double ToolValue,
	bool bApplyRadiusLimit,
	TSharedPtr<FMeshSculptFallofFunc> FalloffFunc)
{
	auto FalloffFunction = [&Stamp, &FalloffFunc](const FVector& Position)
		{
			if (FalloffFunc)
			{
				return (float)FalloffFunc->Evaluate(Stamp, Position);
			}
			return 1.0f;
		};
	FMeshVertexAttributePaintToolPaintBrushOp::ApplyToVerticesStatic(
		Mesh, Vertices, GetVertexWeightFunc, InOutROIBuffer, EditOperation, ToolValue,
		bApplyRadiusLimit, Stamp.LocalFrame.Origin, Stamp.Radius, FalloffFunction);
}

/*static*/ void FMeshVertexAttributePaintToolPaintBrushOp::ApplyToVerticesStatic(
	const FDynamicMesh3* Mesh,
	const TArray<int32>& Vertices,
	TFunctionRef<double(int32)> GetVertexWeightFunc,
	TArray<double>& InOutROIBuffer,
	EMeshVertexAttributePaintToolEditOperation EditOperation,
	const double ToolValue,
	bool bApplyRadiusLimit,
	const FVector Center,
	const float Radius,
	FalloffFunction FalloffFunc
)
{

	ensure(EditOperation != EMeshVertexAttributePaintToolEditOperation::Relax);

	const int32 NumVertices = Vertices.Num();
	check(InOutROIBuffer.Num() == NumVertices);

	ParallelFor(NumVertices,
		[bApplyRadiusLimit, ToolValue, EditOperation, &Center, &Radius, &Mesh, &Vertices, &InOutROIBuffer, &GetVertexWeightFunc, &FalloffFunc]
		(int32 BufferIndex)
		{
			const int32 VertexIndex = Vertices[BufferIndex];
			const double CurrentValue = GetVertexWeightFunc(VertexIndex);

			float FalloffScalar = 1.0;
			if (bApplyRadiusLimit)
			{
				const FVector3d VertexPos = Mesh->GetVertex(VertexIndex);
				const double DistanceSquared = (VertexPos - Center).SquaredLength();
				if (DistanceSquared >= Radius * Radius)
				{
					return;
				}
				if (FalloffFunc.IsSet())
				{
					FalloffScalar = FalloffFunc(VertexPos);
				}
			}

			double NewValue = CurrentValue;

			switch (EditOperation)
			{
			case EMeshVertexAttributePaintToolEditOperation::Add:
				NewValue = CurrentValue + ToolValue;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Replace:
				NewValue = ToolValue;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Multiply:
				NewValue = CurrentValue * ToolValue;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Invert:
				NewValue = 1.0 - CurrentValue; // no falloff
				break;
			case EMeshVertexAttributePaintToolEditOperation::Relax:
				ensure(false); // dispatch should happen upstream
			}

			NewValue = FMath::Lerp(CurrentValue, NewValue, (double)FalloffScalar);
			InOutROIBuffer[BufferIndex] = FMath::Clamp(NewValue, 0.0, 1.0);
		},
		EParallelForFlags::None
	);
}
