// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexAttributePaintToolBase.h"

#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "ContextObjectStore.h"
#include "MeshVertexAttributePaintBrushOps.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/Engine.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "IPersonaEditorModeManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Logging/LogMacros.h"
#include "Math/Box.h"
#include "ModelingToolTargetUtil.h"
#include "Polygon2.h"
#include "PreviewMesh.h"
#include "SceneView.h"
#include "StaticMeshAttributes.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshVertexSelection.h"
#include "Solvers/PrecomputedMeshWeightData.h"
#include "Spatial/PointHashGrid3.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "ToolSetupUtil.h"
#include "Util/BufferUtil.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Generators/RectangleMeshGenerator.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexAttributePaintToolBase)

DEFINE_LOG_CATEGORY_STATIC(LogMeshVertexAttributePaintToolBase, Warning, All);

#define LOCTEXT_NAMESPACE "MeshVertexAttributePaintToolBase"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::MeshVertexAttributePaintToolBase::Private
{
	static const FString MirrorBrushGizmoType = TEXT("MeshVertexAttributePaintToolMirrorGizmo");

	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
	
	static void ShowEditorMessage(ELogVerbosity::Type InMessageType, const FText& InMessage)
	{
		FNotificationInfo Notification(InMessage);
		Notification.bUseSuccessFailIcons = true;
		Notification.ExpireDuration = 5.0f;

		SNotificationItem::ECompletionState State = SNotificationItem::CS_Success;

		switch (InMessageType)
		{
		case ELogVerbosity::Warning:
			UE_LOGF(LogMeshVertexAttributePaintToolBase, Warning, "%ls", *InMessage.ToString());
			break;
		case ELogVerbosity::Error:
			State = SNotificationItem::CS_Fail;
			UE_LOGF(LogMeshVertexAttributePaintToolBase, Error, "%ls", *InMessage.ToString());
			break;
		default:
			break; // don't log anything unless a warning or error
		}

		FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(State);
	}

	/**
	* A wrapper change for a dynamic mesh change
	*/
	class  FMeshChange : public FToolCommandChange
	{
	public:
		FMeshChange(TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChangeIn) :
			DynamicMeshChange(MoveTemp(DynamicMeshChangeIn))
		{
			ensure(DynamicMeshChange);
		};

		virtual void Apply(UObject* Object) override
		{
			UDynamicMesh* ToolDynamicMesh = CastChecked<UDynamicMesh>(Object);
			ToolDynamicMesh->EditMesh([this](FDynamicMesh3& InMesh)
			{
				DynamicMeshChange->Apply(&InMesh, false);
			});	
		}

		virtual void Revert(UObject* Object) override
		{
			UDynamicMesh* ToolDynamicMesh = CastChecked<UDynamicMesh>(Object);
			ToolDynamicMesh->EditMesh([this](FDynamicMesh3& InMesh)
			{
				DynamicMeshChange->Apply(&InMesh, true);
			});	
		}

		virtual FString ToString() const override
		{
			return TEXT("MeshVertexAttributePaintToolMeshChange");
		}

	protected:
		TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange;
	};

	/**
	 * Per-stamp bookkeeping for a brush stroke or selection-mode operation (one transaction).
	 * Lifecycle is Begin(mode, VertexData) -> per-stamp (PrepareStampInput / op writes ROIBuffer /
	 * FinalizeStampOutput) -> End(), wrapped by BeginChange / EndChange.
	 *
	 * The accumulator does not own its read source. The transaction-level snapshot lives on
	 * FMeshVertexAttributePaintToolData::PreChangeWeights (session-lifetime; refreshed eagerly
	 * by the tool's transaction methods so it always matches the current mesh state at every
	 * BeginChange). The accumulator borrows a pointer to VertexData and (in accumulate mode) to
	 * the active weight layer; both pointers are valid only between Begin and End.
	 *
	 * Two modes, picked at Begin:
	 *
	 *   - GatedByMaxFalloff: each vertex receives at most one stamp's effect per transaction.
	 *     Used by paint ops (Add/Replace/Multiply) and the relax brush when bAccumulate=false.
	 *     Reader pulls VertexData->GetPreChangeValue(V) (immutable across the transaction).
	 *     MaxFalloff + PreStampScratch gate stamp output.
	 *
	 *   - Accumulate: stamps compound across the transaction. Used by the relax brush when
	 *     bAccumulate=true and by selection-mode Relax. Reader pulls VertexData->GetValue(V)
	 *     which reads the current mesh attribute value, so the next stamp's read sees the
	 *     previous stamp's writes after SyncMeshWithWeightBuffer.
	 *
	 * Storage:
	 *   - MaxFalloff: MaxVertexID floats (gated only). Tracks the strongest per-vertex falloff seen.
	 *   - PreStampScratch: ROI-sized doubles (gated only). Stashed pre-stamp values for revert.
	 */
	class FStrokeAccumulator
	{
	public:
		using EMode = EStampAccumulatorMode;

		void Begin(EMode InMode, FMeshVertexAttributePaintToolData& InVertexData)
		{
			ensure(!bActive);
			bActive = true;
			Mode = InMode;
			VertexData = &InVertexData;
			if (Mode == EMode::GatedByMaxFalloff)
			{
				MaxFalloffPerVertex.SetNumZeroed(VertexData->GetNumVerts());
			}
		}

		void End()
		{
			bActive = false;
			VertexData = nullptr;
			MaxFalloffPerVertex.Reset();
			PreStampScratch.Reset();
		}

		TFunction<double(int32)> MakeVertexWeightFunc() const
		{
			if (Mode == EMode::GatedByMaxFalloff)
			{
				return [this](int32 V) -> double
				{
					return VertexData->GetPreChangeValue(V);
				};
			}
			// Accumulate: read the live mesh attribute value. SyncMeshWithWeightBuffer applies
			// each stamp's ROIBuffer to the mesh before the next stamp's reader runs.
			return [this](int32 V) -> double
			{
				return VertexData->GetValue(V);
			};
		}

		void PrepareStampInput(const TArray<int32>& VertexROI, const TArray<double>& ROIBuffer)
		{
			if (Mode == EMode::GatedByMaxFalloff)
			{
				PreStampScratch = ROIBuffer;
			}
		}


		void FinalizeStampOutput(
			const TArray<int32>& VertexROI,
			TArray<double>& ROIBuffer,
			TFunctionRef<float(int32 VertexIndex)> ComputeFalloff)
		{
			if (Mode != EMode::GatedByMaxFalloff)
			{
				// Accumulate mode: SyncMeshWithWeightBuffer (called by the tool after this) writes
				// ROIBuffer into the mesh attribute, so the next stamp's reader picks it up. No
				// bookkeeping needed here.
				return;
			}

			const int32 NumVerts = VertexROI.Num();
			for (int32 i = 0; i < NumVerts; ++i)
			{
				const int32 V = VertexROI[i];
				if (!MaxFalloffPerVertex.IsValidIndex(V))
				{
					ROIBuffer[i] = PreStampScratch[i];
					continue;
				}
				const float StampFalloff = ComputeFalloff(V);
				float& Max = MaxFalloffPerVertex[V];
				if (StampFalloff > Max)
				{
					Max = StampFalloff;
					// Keep ROIBuffer[i] as the brush op wrote it.
				}
				else
				{
					ROIBuffer[i] = PreStampScratch[i];
				}
			}
		}

	private:
		EMode Mode = EMode::GatedByMaxFalloff;

		// True between Begin() and End(). Drives the double-begin guard.
		bool bActive = false;

		// Borrowed; valid only between Begin/End.
		FMeshVertexAttributePaintToolData* VertexData = nullptr;

		// (Gated only) Per-vertex strongest falloff seen this transaction
		TArray<float> MaxFalloffPerVertex;

		// (Gated only) Scratch for one stamp's "previous best" per ROI vertex; populated by PrepareStampInput
		// and consumed by FinalizeStampOutput. Persists across stamps and transactions
		// so the allocation is reused.
		TArray<double> PreStampScratch;
	};
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Properties
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UMeshVertexAttributePaintToolProperties::UMeshVertexAttributePaintToolProperties()
	: UInteractiveToolPropertySet()
{
	LoadConfig();

	if (DisplayProperties.ColorRamp.IsEmpty())
	{
		CreateDefaultColorRamp();
	}

	constexpr bool bOnlyRGB = true;
	DisplayProperties.GreyScaleColorRamp.SetColorAtTime(0.0f, FLinearColor::Black, bOnlyRGB);
	DisplayProperties.GreyScaleColorRamp.SetColorAtTime(1.0f, FLinearColor::White, bOnlyRGB);

	DisplayProperties.WhiteColorRamp.SetColorAtTime(0.0f, FLinearColor::White, bOnlyRGB);
	DisplayProperties.WhiteColorRamp.SetColorAtTime(1.0f, FLinearColor::White, bOnlyRGB);
}

void UMeshVertexAttributePaintToolProperties::CreateDefaultColorRamp()
{
	ResetColorRamp();
}


void UMeshVertexAttributePaintToolProperties::ResetColorRamp()
{
	constexpr FLinearColor HeatMapColors[] =
	{
		FLinearColor(0.8f, 0.4f, 0.8f), // Purple
		FLinearColor(0.0f, 0.0f, 0.5f), // Dark Blue
		FLinearColor(0.2f, 0.2f, 1.0f), // Light Blue
		FLinearColor(0.0f, 1.0f, 0.0f), // Green
		FLinearColor(1.0f, 1.0f, 0.0f), // Yellow
		FLinearColor(1.0f, 0.65f, 0.0f), // Orange
		FLinearColor(1.0f, 0.0f, 0.0f, 0.0f), // Red
	};
	constexpr int32 NumColors = sizeof(HeatMapColors) / sizeof(FLinearColor);
	ensure(NumColors > 1);

	// Set ColorRamp
	DisplayProperties.ColorRamp.Reset();
	for (int32 Index = 0; Index < NumColors; ++Index)
	{
		const float Time = (float)Index / ((float)NumColors - 1.f);
		DisplayProperties.ColorRamp.SetColorAtTime(Time, HeatMapColors[Index], /*bOnlyRGB*/true);
	}
	DisplayProperties.ColorRamp.GetCurves()[3].CurveToEdit->AddKey(0.f, 1.f); // single alpha  value
}

const FMeshVertexAttributePaintToolBrushConfig& UMeshVertexAttributePaintToolProperties::GetBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode) const
{
	static FMeshVertexAttributePaintToolBrushConfig DefaultConfig;

	switch (BrushMode)
	{
	case EMeshVertexAttributePaintToolEditOperation::Add:
		return BrushConfigAdd;

	case EMeshVertexAttributePaintToolEditOperation::Replace:
		return BrushConfigReplace;

	case EMeshVertexAttributePaintToolEditOperation::Multiply:
		return BrushConfigMultiply;

	case EMeshVertexAttributePaintToolEditOperation::Relax:
		return BrushConfigRelax;

	case EMeshVertexAttributePaintToolEditOperation::Invert:
	default:
		return DefaultConfig;
	};
}

void UMeshVertexAttributePaintToolProperties::SetBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode, const FMeshVertexAttributePaintToolBrushConfig& BrushConfig)
{
	switch (BrushMode)
	{
	case EMeshVertexAttributePaintToolEditOperation::Add:
		BrushConfigAdd = BrushConfig;
		break;

	case EMeshVertexAttributePaintToolEditOperation::Replace:
		BrushConfigReplace = BrushConfig;
		break;

	case EMeshVertexAttributePaintToolEditOperation::Multiply:
		BrushConfigMultiply = BrushConfig;
		break;

	case EMeshVertexAttributePaintToolEditOperation::Relax:
		BrushConfigRelax = BrushConfig;
		break;

	case EMeshVertexAttributePaintToolEditOperation::Invert:
	default:
		// nothing to update
		break;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Data
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMeshVertexAttributePaintToolData::Setup(UDynamicMesh* InToolDynamicMesh, int32 InInitialAttributeIndex)
{
	ToolDynamicMesh = InToolDynamicMesh;

	SetAttributeToEdit(InInitialAttributeIndex);
}


bool FMeshVertexAttributePaintToolData::SetAttributeToEdit(int32 InNewAttributeIndex)
{
	using namespace UE::Geometry;

	if (!ToolDynamicMesh.IsValid())
	{
		return false;
	}

	bool bSuccess = true;
	ToolDynamicMesh->ProcessMesh([&bSuccess, InNewAttributeIndex](const FDynamicMesh3& InMesh)
	{
		const FDynamicMeshAttributeSet* Attributes = InMesh.Attributes();

		if (!Attributes)
		{
			bSuccess = false;
			return;
		}
	
		if (InNewAttributeIndex < 0 || InNewAttributeIndex >= Attributes->NumWeightLayers())
		{
			bSuccess = false;
			return;
		}
	});

	if (bSuccess)
	{
		AttributeIndex = InNewAttributeIndex;
		SyncPreChangeWeightsToCurrentWeights();
	}

	return bSuccess;
}

void FMeshVertexAttributePaintToolData::SyncPreChangeWeightsToCurrentWeights()
{
	SnapshotAllValues(PreChangeWeights);
}

void FMeshVertexAttributePaintToolData::BeginChange()
{
	if (!IsValid())
	{
		return;
	}
	ensure(ActiveWeightEditChangeTracker == nullptr);
	ToolDynamicMesh->ProcessMesh([this](const FDynamicMesh3& InMesh)
	{
		ActiveWeightEditChangeTracker = MakeUnique<UE::Geometry::FDynamicMeshChangeTracker>(&InMesh);
		ActiveWeightEditChangeTracker->BeginChange();
	});	
}

TUniquePtr<UE::MeshVertexAttributePaintToolBase::Private::FMeshChange> FMeshVertexAttributePaintToolData::EndChange()
{
	if (!IsValid())
	{
		return {};
	}
	
	if (!ensure(ActiveWeightEditChangeTracker))
	{
		return {};
	}
	
	TUniquePtr<UE::Geometry::FDynamicMeshChange> EditResult = ActiveWeightEditChangeTracker->EndChange();

	using namespace UE::MeshVertexAttributePaintToolBase;
	TUniquePtr<Private::FMeshChange> MeshChange = MakeUnique<Private::FMeshChange>(MoveTemp(EditResult));
	
	ActiveWeightEditChangeTracker = nullptr;
	
	return MoveTemp(MeshChange);
}

void FMeshVertexAttributePaintToolData::CancelChange()
{
	if (!IsValid())
	{
		return;
	}
	
	if (ensure(ActiveWeightEditChangeTracker))
	{
		ActiveWeightEditChangeTracker->EndChange();
		ActiveWeightEditChangeTracker = nullptr;
	}
}

bool FMeshVertexAttributePaintToolData::IsValid() const
{
	return AttributeIndex != INDEX_NONE;
}

float FMeshVertexAttributePaintToolData::GetPreChangeValue(int32 VertexIdx) const
{
	if (!IsValid() || !ensure(PreChangeWeights.IsValidIndex(VertexIdx)))
	{
		return 0.f;
	}
	return PreChangeWeights[VertexIdx];
}

float FMeshVertexAttributePaintToolData::GetValue(int32 VertexIdx) const
{
	if (!IsValid())
	{
		return 0.f;
	}
	
	float Value = 0.f;

	using namespace  UE::Geometry;
	ToolDynamicMesh->ProcessMesh([this, VertexIdx, &Value](const FDynamicMesh3& InMesh)
	{
		const FDynamicMeshWeightAttribute* ActiveWeightMap = InMesh.Attributes()->GetWeightLayer(AttributeIndex);	
		if (VertexIdx != IndexConstants::InvalidID)
		{
			ActiveWeightMap->GetValue(VertexIdx, &Value);
		}
	});
	
	return Value;
}

float FMeshVertexAttributePaintToolData::GetAverageValue(const TArray<int32>& Vertices) const
{
	if (!IsValid())
	{
		return 0.f;
	}
	
	double AverageValue = 0.f;
	
	using namespace  UE::Geometry;
	ToolDynamicMesh->ProcessMesh([&AverageValue, this, &Vertices](const FDynamicMesh3& InMesh)
	{
		const FDynamicMeshWeightAttribute* ActiveWeightMap = InMesh.Attributes()->GetWeightLayer(AttributeIndex);
	
		if (ActiveWeightMap)
		{
			int32 NumValidValues = 0;
			for (int32 VertexIdx : Vertices)
			{
				if (VertexIdx != IndexConstants::InvalidID)
				{
					float Value = 0.0f;
					ActiveWeightMap->GetValue(VertexIdx, &Value);
					AverageValue += (double)Value;
					++NumValidValues;
				}
			}
			if (NumValidValues > 0)
			{
				AverageValue = AverageValue / (double)NumValidValues;
			}
		}
	});
	
	return (float)AverageValue;	
}

void FMeshVertexAttributePaintToolData::SetValue(int32 VertexIdx, float Value)
{
	if (!IsValid())
	{
		return;
	}

	using namespace  UE::Geometry;
	ToolDynamicMesh->EditMesh([this, VertexIdx, Value](FDynamicMesh3& InMesh)
	{
		FDynamicMeshWeightAttribute* ActiveWeightMap = InMesh.Attributes()->GetWeightLayer(AttributeIndex);
		if (ActiveWeightEditChangeTracker)
		{
			ActiveWeightEditChangeTracker->SaveVertexOneRingTriangles(VertexIdx, true);
		}

		if (VertexIdx != IndexConstants::InvalidID)
		{
			ActiveWeightMap->SetValue(VertexIdx, &Value);
		}
	});
}

void FMeshVertexAttributePaintToolData::SnapshotAllValues(TArray<float>& OutSnapshot) const
{
	OutSnapshot.Reset();
	if (!IsValid())
	{
		return;
	}

	using namespace UE::Geometry;
	ToolDynamicMesh->ProcessMesh([this, &OutSnapshot](const FDynamicMesh3& InMesh)
	{
		const FDynamicMeshWeightAttribute* ActiveWeightMap = InMesh.Attributes()->GetWeightLayer(AttributeIndex);
		if (!ActiveWeightMap)
		{
			return;
		}
		const int32 MaxVID = InMesh.MaxVertexID();
		OutSnapshot.SetNumZeroed(MaxVID);
		for (int32 V = 0; V < MaxVID; ++V)
		{
			if (InMesh.IsVertex(V))
			{
				ActiveWeightMap->GetValue(V, &OutSnapshot[V]);
			}
		}
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Tool
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UMeshVertexAttributePaintToolBase::UMeshVertexAttributePaintToolBase()
	: StrokeAccumulator(MakeShared<UE::MeshVertexAttributePaintToolBase::Private::FStrokeAccumulator>())
{
}

UMeshVertexAttributePaintToolBase::~UMeshVertexAttributePaintToolBase() = default;

UBaseDynamicMeshComponent* UMeshVertexAttributePaintToolBase::GetSculptMeshComponent()
{
	return PreviewMesh ? Cast<UBaseDynamicMeshComponent>(PreviewMesh->GetRootComponent()) : nullptr;
}

void UMeshVertexAttributePaintToolBase::SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior)
{
	// Map the horizontal behavior to change the brush size 
	// (we do not want to use the default as we are internal set to use the adapative brush size but external display world radius)
	//UMeshSculptToolBase::MapHorizontalBrushEditBehaviorToBrushSize(OutBehavior);

	OutBehavior.HorizontalProperty.GetValueFunc = [this]() { return GetCurrentBrushRadius(); };
	OutBehavior.HorizontalProperty.SetValueFunc = [this](float NewValue) { SetBrushRadius(NewValue); };
	OutBehavior.HorizontalProperty.MutateDeltaFunc = [this](float Delta)
		{
			// Scale delta if brush size is in world units.
			//return Delta * (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::World ? (CameraState.Position - LastBrushFrameWorld.Origin).Length() : 1.f);
			return Delta * (CameraState.Position - LastBrushFrameWorld.Origin).Length();
		};
	OutBehavior.HorizontalProperty.Name = LOCTEXT("BrushRadius", "Radius");
	OutBehavior.HorizontalProperty.bEnabled = true;


	// map the vertical behavior to change the attribiute value
	OutBehavior.VerticalProperty.GetValueFunc = [this]()
		{
			return ToolProperties->BrushProperties.AttributeValue;
		};

	OutBehavior.VerticalProperty.SetValueFunc = [this](float NewValue)
		{
			ToolProperties->BrushProperties.AttributeValue = FMath::Min(1.0f, FMath::Max(NewValue, 0.f));
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(FMeshVertexAttributePaintToolBrushProperties::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, AttributeValue)));
			ToolProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		};
	OutBehavior.VerticalProperty.Name = LOCTEXT("AttributeValue", "Value");
	OutBehavior.VerticalProperty.EditRate = 0.005f;
	OutBehavior.VerticalProperty.bEnabled = true;
}

void UMeshVertexAttributePaintToolBase::Setup()
{
	MinimumBrushAdaptiveSizeRatio = 0.001;

	UMeshSculptToolBase::Setup();

	// todo(ccaillaud) : this should be a parameter of the tool ? 
	SetToolDisplayName(LOCTEXT("ToolName", "Paint Weight Maps"));
	
	// hide input Component
	UE::ToolTarget::HideSourceObject(Target);
	
	// make sure we get a copy of the original dynamic mesh 
	{
		FGetMeshParameters GetMeshParams;
		GetMeshParams.bWantMeshTangents = true;
		FDynamicMesh3 DynamicMeshCopy = UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams);
	
		ToolDynamicMesh = NewObject<UDynamicMesh>();
		ToolDynamicMesh->SetMesh(MoveTemp(DynamicMeshCopy));
	}

	ToolDynamicMesh->EditMesh([this](FDynamicMesh3& InMesh)
	{
		int32 InitialAttributeIndex = INDEX_NONE;
		// give derived class a chance to customize the tool mesh and decide which attribute to paint into first
		if (SetupToolMesh(InMesh, InitialAttributeIndex))
		{
			VertexData.Setup(ToolDynamicMesh, InitialAttributeIndex);
		}
	});
	
	// create the preview mesh
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(TargetWorld, FTransform::Identity);
	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);
	ToolDynamicMesh->ProcessMesh([this](const FDynamicMesh3& InToolMesh)
	{
		PreviewMesh->ReplaceMesh(InToolMesh);
	});
		
	// assign materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);



	UBaseDynamicMeshComponent* PreviewDynaMeshComponent = GetSculptMeshComponent();
	// bake just the scale into the mesh to avoid skewed stamps
	// Note: this transform does not include translation and rotation ( only scale part of 3x3 transform )
	// Note2: not baking rotation because it may break mirror/symmetry
	InitialTargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d WorldTranslation = InitialTargetTransform.GetTranslation();
	UE::Geometry::FQuaterniond WorldRotation = InitialTargetTransform.GetRotation();
	// clamp scaling just to be consistent with other tools. We don't really care about inversion because the tool
	// does not change mesh geometry.
	UE::Geometry::FTransformSRT3d BakedTransform;
	BakedTransform.SetScale(InitialTargetTransform.GetScale());
	BakedTransform.ClampMinimumScale(0.01);
	
	PreviewDynaMeshComponent->ApplyTransform(BakedTransform, false);
	CurTargetTransform = InitialTargetTransform;
	CurTargetTransform.SetScale(FVector3d::One());
	PreviewDynaMeshComponent->SetWorldTransform((FTransform)CurTargetTransform);

	// make sure the dynamic mesh has all the necessary attributes
	FDynamicMesh3* PreviewDynaMesh = GetSculptMesh();
	check(PreviewDynaMesh);
	PreviewDynaMesh->EnableVertexColors(FVector3f::One());
	PreviewDynaMesh->Attributes()->EnablePrimaryColors();
	PreviewDynaMesh->Attributes()->PrimaryColors()->CreatePerVertex(0.f);
	MeshLocalBounds = PreviewDynaMesh->GetBounds(true);
	
	TFuture<void> PrecomputeFuture = Async(UE::MeshVertexAttributePaintToolBase::Private::WeightPaintToolAsyncExecTarget, [this]()
		{
			PrecomputeSeamEdges();
		});

	TFuture<void> OctreeFuture = Async(UE::MeshVertexAttributePaintToolBase::Private::WeightPaintToolAsyncExecTarget, [this, &PreviewDynaMesh]()
		{
			RebuildOctree();
		});

	// setup mesh selector
	MeshSelector = NewObject<UToolMeshSelector>(this);
	auto OnSelectionChangedLambda = [this]() { /*OnSelectionModified();*/ };
	MeshSelector->InitialSetup(TargetWorld, this, OnSelectionChangedLambda);
	MeshSelector->SetMesh(PreviewMesh, PreviewMesh->GetTransform());

	// initialize brush radius range interval, brush properties
	UMeshSculptToolBase::InitializeBrushSizeRange(MeshLocalBounds);

	// initialize properties
	ToolProperties = NewObject<UMeshVertexAttributePaintToolProperties>(this);
	{
		ToolProperties->RestoreProperties(this);
		ToolProperties->DisplayProperties.ColorRamp.OnColorCurveChangedDelegate.AddUObject(this, &UMeshVertexAttributePaintToolBase::OnColorRampChanged);
	}
	AddToolPropertySource(ToolProperties);

	// make sure the selector is in sync with the settings
	MeshSelector->SetComponentSelectionMode(ToolProperties->SelectionProperties.ComponentSelectionMode);
	MeshSelector->SetSelectionTool(EMeshSelectorTool::Marquee);
	MeshSelector->GetSelectionMechanic()->SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnTickAndRelease);
	
	InitializeIndicator();
	InitializeMirrorIndicator();

	// initialize our properties
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, false);
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = true;
	UMeshSculptToolBase::BrushProperties->bShowLazyness = false;
	UMeshSculptToolBase::BrushProperties->FlowRate = 0.0f;
	CalculateBrushRadius();

	PaintBrushOpProperties = NewObject<UMeshVertexAttributePaintToolPaintBrushOpProps>(this);
	RegisterBrushType(PaintBrushId, LOCTEXT("Paint", "Paint"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FMeshVertexAttributePaintToolPaintBrushOp>(); }),
		PaintBrushOpProperties);

	// Smooth op properties shared by both primary & secondary smooth brush
	SmoothBrushOpProperties = NewObject<UMeshVertexAttributePaintToolSmoothBrushOpProps>(this);

	auto MakeSmoothBrushOpFactory = [this]()
		{
			return MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]()
				{
					TUniquePtr<FMeshVertexAttributePaintToolSmoothBrushOp> SmoothOp = MakeUnique<FMeshVertexAttributePaintToolSmoothBrushOp>();
					SmoothOp->GetSmoothEdgeWeights = [WeakTool = TWeakObjectPtr<UMeshVertexAttributePaintToolBase>(this)]() -> const TArray<double>&
					{
						static const TArray<double> Empty;
						if (UMeshVertexAttributePaintToolBase* Tool = WeakTool.Get())
						{
							return Tool->CotangentEdgeWeights;
						}
						return Empty;
					};
					return SmoothOp;
				});
		};

	RegisterBrushType(SmoothBrushId, LOCTEXT("RelaxBrushType", "Relax"), MakeSmoothBrushOpFactory(), SmoothBrushOpProperties);
	RegisterSecondaryBrushType(SmoothBrushId, LOCTEXT("SecondaryRelaxBrushType", "Relax"), MakeSmoothBrushOpFactory(), SmoothBrushOpProperties);

	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::ViewProperties, false);

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();


	UpdateBrushType(ToolProperties->BrushProperties.BrushMode);

	// mesh element display is used to show seams and other features ofthe mesh
	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);

	MeshElementsDisplay->CreateInWorld(TargetWorld, PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowBorders = ToolProperties->DisplayProperties.bShowBorders;
		MeshElementsDisplay->Settings->bShowUVSeams = ToolProperties->DisplayProperties.bShowUVSeams;
		MeshElementsDisplay->Settings->bShowNormalSeams = ToolProperties->DisplayProperties.bShowNormalSeams;
		MeshElementsDisplay->Settings->bShowTangentSeams = ToolProperties->DisplayProperties.bShowTangentSeams;
		MeshElementsDisplay->Settings->bShowColorSeams = ToolProperties->DisplayProperties.bShowColorSeams;
		MeshElementsDisplay->Settings->ThicknessScale = ToolProperties->DisplayProperties.ThicknessScale;
		MeshElementsDisplay->Settings->WireframeColor = ToolProperties->DisplayProperties.WireframeColor;
		MeshElementsDisplay->Settings->BoundaryEdgeColor = ToolProperties->DisplayProperties.BoundaryEdgeColor;
		MeshElementsDisplay->Settings->UVSeamColor = ToolProperties->DisplayProperties.UVSeamColor;
		MeshElementsDisplay->Settings->NormalSeamColor = ToolProperties->DisplayProperties.NormalSeamColor;
		MeshElementsDisplay->Settings->TangentSeamColor = ToolProperties->DisplayProperties.TangentSeamColor;
		MeshElementsDisplay->Settings->ColorSeamColor = ToolProperties->DisplayProperties.ColorSeamColor;
		MeshElementsDisplay->Settings->bShowGroupBoundaries = ToolProperties->DisplayProperties.bShowGroupBoundaries;
		MeshElementsDisplay->Settings->GroupBoundaryColor = ToolProperties->DisplayProperties.GroupBoundaryColor;
		MeshElementsDisplay->Settings->DepthBias = ToolProperties->DisplayProperties.DepthBias;
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		ProcessFunc(*GetSculptMesh());
		});

	// disable view properties
	SetViewPropertiesEnabled(false);
	// Apply the material mode matching the restored ColorMode — otherwise FullMaterial renders flat white until the user toggles the mode.
	UpdateMaterialMode(ToolProperties->DisplayProperties.ColorMode == EMeshVertexAttributePaintToolColorMode::FullMaterial
		? EMeshEditingMaterialModes::ExistingMaterial
		: EMeshEditingMaterialModes::VertexColor);
	UpdateWireframeVisibility(false);
	UpdateFlatShadingSetting(false);

	PrecomputeFuture.Wait();
	OctreeFuture.Wait();

	if (FDynamicMesh3* SculptMesh = GetSculptMesh())
	{
		UE::MeshDeformation::ConstructEdgeCotanWeightsDataArray(*SculptMesh, CotangentEdgeWeights);
	}

	// update colors
	UpdatePreview();

	LoadBrushConfig(ToolProperties->BrushProperties.BrushMode);
	SetBrushMode(ToolProperties->BrushProperties.BrushMode);

	if (ToolProperties->bSyncBrushRadiusAcrossModes)
	{
		ToolProperties->SetSharedBrushSize(GetBrushAdaptiveSize());
	}

	SetPrimaryFalloffType(EMeshSculptFalloffType::Smooth);

}

void UMeshVertexAttributePaintToolBase::InitializeIndicator()
{
	UMeshSculptToolBase::InitializeIndicator();
	if (BrushIndicatorMesh)
	{
		BrushIndicatorMesh->SetOverrideWireframeRenderMaterial(BrushIndicatorMaterial);
	}
}

void UMeshVertexAttributePaintToolBase::InitializeMirrorIndicator()
{
	// register and spawn brush indicator gizmo
	GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(UE::MeshVertexAttributePaintToolBase::Private::MirrorBrushGizmoType, NewObject<UBrushStampIndicatorBuilder>());
	MirrorBrushIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UBrushStampIndicator>(UE::MeshVertexAttributePaintToolBase::Private::MirrorBrushGizmoType, FString(), this);
	MirrorBrushIndicatorMesh = MakeBrushIndicatorMesh(this, TargetWorld);
	MirrorBrushIndicatorMesh->SetOverrideWireframeRenderMaterial(BrushIndicatorMaterial);
	MirrorBrushIndicator->AttachedComponent = MirrorBrushIndicatorMesh->GetRootComponent();
	MirrorBrushIndicator->LineThickness = 1.0;
	MirrorBrushIndicator->bDrawIndicatorLines = true;
	MirrorBrushIndicator->bDrawRadiusCircle = false;
	MirrorBrushIndicator->LineColor = FLinearColor(0.9f, 0.4f, 0.4f);

	MirrorPlaneMesh = NewObject<UPreviewMesh>(this);
	MirrorPlaneMesh->CreateInWorld(TargetWorld, PreviewMesh->GetTransform());
	UE::Geometry::FRoundedRectangleMeshGenerator Gen;
	Gen.Origin = { 0,0,0 };
	Gen.Normal = { 0,0,1 };
	Gen.Width = 100.0;
	Gen.Height = 100.0;
	Gen.WidthVertexCount = 10;
	Gen.HeightVertexCount = 10;
	Gen.bScaleUVByAspectRatio = false;
	Gen.Radius = 10.0;
	Gen.AngleSamples = 8;
	Gen.SharpCorners = UE::Geometry::ERoundedRectangleCorner::None;
	Gen.Generate();
	FDynamicMesh3 PlaneMesh(&Gen);
	MirrorPlaneMesh->UpdatePreview(&PlaneMesh);
	//BrushIndicatorMaterial = ToolSetupUtil::GetDefaultBrushVolumeMaterial(GetToolManager());
	if (BrushIndicatorMaterial)
	{
		MirrorPlaneMesh->SetMaterial(BrushIndicatorMaterial);
	}

	// make sure raytracing is disabled on the brush indicator
	Cast<UDynamicMeshComponent>(MirrorPlaneMesh->GetRootComponent())->SetEnableRaytracing(false);
	MirrorPlaneMesh->SetShadowsEnabled(false);
	MirrorPlaneMesh->SetOverrideWireframeRenderMaterial(BrushIndicatorMaterial);

	UpdateMirrorPlaneMesh();
}

void UMeshVertexAttributePaintToolBase::SetFocusInViewport() const
{
	// TODO: check if this is needed
}

void UMeshVertexAttributePaintToolBase::UpdateMirrorPlaneMesh()
{
	if (MirrorPlaneMesh)
	{
		FTransform PlaneMeshTransform = FTransform::Identity;

		const bool bIsBrushMirrorVisible = IsInBrushMode() && IsBrushMirroringEnabled() && IsMirrorPlaneWidgetVisible() && !bIsMirrorHiddenBecauseOfBrushStroke;
		const bool bIsMeshMirrorVisible = !IsInBrushMode() && IsMirrorPlaneWidgetVisible();
		MirrorPlaneMesh->SetVisible(bIsBrushMirrorVisible || bIsMeshMirrorVisible);

		switch(GetMirrorAxis())
		{
		case EAxis::X:
			PlaneMeshTransform = FTransform(FRotator(90, 0, 0));
			break;
		case EAxis::Y:
			PlaneMeshTransform = FTransform(FRotator(0, 0, 90));
			break;
		case EAxis::Z:
			PlaneMeshTransform = FTransform::Identity;
		}

		const FVector MeshExtents = MeshLocalBounds.Extents();
		
		// scale of plane
		FVector PlaneScale(1.0);
		switch (GetMirrorAxis())
		{
		case EAxis::X:
			PlaneScale.X = MeshExtents.Z;
			PlaneScale.Y = MeshExtents.Y;
			break;
		case EAxis::Y:
			PlaneScale.X = MeshExtents.X;
			PlaneScale.Y = MeshExtents.Z;
			break;
		case EAxis::Z:
			PlaneScale.X = MeshExtents.X;
			PlaneScale.Y = MeshExtents.Y;
		}

		// the plane is originally 100 units and we want 10% over the size of the box
		constexpr double ScaleAdjustment = 0.02 * 1.10;
		PlaneMeshTransform.SetScale3D(PlaneScale * ScaleAdjustment);

		// Position of the plane
		FVector PlanePosition = MeshLocalBounds.Center();
		if (!IsBrushMirroringInObjectSpace())
		{
			switch (GetMirrorAxis())
			{
			case EAxis::X:
				PlanePosition.X = 0;
				break;
			case EAxis::Y:
				PlanePosition.Y = 0;
				break;
			case EAxis::Z:
				PlanePosition.Z = 0;
			}
		}
		PlaneMeshTransform.SetTranslation(PlanePosition);
		
		const bool bIsObjectSpaceMirror = IsBrushMirroringInObjectSpace();
		if (bIsObjectSpaceMirror)
		{
			PlaneMeshTransform *= CurTargetTransform;
		}
		
		MirrorPlaneMesh->SetTransform(PlaneMeshTransform);
	}
}

UE::Geometry::FFrame3d UMeshVertexAttributePaintToolBase::ComputeMirroredFrame(const UE::Geometry::FFrame3d& InLocalSpaceFrame) const
{
	UE::Geometry::FFrame3d OutFrame{ InLocalSpaceFrame };

	const bool bIsObjectSpaceMirror = IsBrushMirroringInObjectSpace();
	const EAxis::Type MirrorAxis = GetMirrorAxis();
	FVector MirrorCenter = MeshLocalBounds.Center();

	if (!bIsObjectSpaceMirror)
	{
		// Transform the frame to world space before mirroring
		OutFrame.Transform(CurTargetTransform);
		MirrorCenter = FVector::ZeroVector;
	}

	// Mirroring is in bounds space 
	FTransform LocalTransform = OutFrame.ToFTransform();
	LocalTransform.SetTranslation(LocalTransform.GetTranslation() - MirrorCenter);
	LocalTransform.Mirror(MirrorAxis, EAxis::None);
	LocalTransform.SetTranslation(LocalTransform.GetTranslation() + MirrorCenter);
	OutFrame = UE::Geometry::FFrame3d(LocalTransform);
	
	if (!bIsObjectSpaceMirror)
	{
		// Transform back the frame to local space after mirroring
		OutFrame.Transform(CurTargetTransform.InverseUnsafe());
	}

	return OutFrame;
}

void UMeshVertexAttributePaintToolBase::UpdateMirrorIndicator()
{
	if (BrushIndicatorMesh && BrushIndicator)
	{
		MirrorBrushIndicator->bVisible = BrushIndicator->bVisible && IsBrushMirroringEnabled();
		MirrorBrushIndicatorMesh->SetVisible(BrushIndicatorMesh->IsVisible() && IsBrushMirroringEnabled());
		MirrorBrushIndicator->bDrawRadiusCircle = BrushIndicator->bDrawRadiusCircle;

		FTransform MirrorBrushTransform = ComputeMirroredFrame(HoverStamp.LocalFrame).ToFTransform();

		// Project onto the surface
		MirrorBrushTriangleID = IndexConstants::InvalidID;
		if (const FDynamicMesh3* Mesh = GetSculptMesh())
		{
			const double RayMaxDistance = 10.0;
			const FVector MirrorBrushNormal = MirrorBrushTransform.GetUnitAxis(EAxis::Z);
			const FVector RayDirection = -MirrorBrushNormal;
			const FVector RayOrigin = MirrorBrushTransform.GetTranslation() + MirrorBrushNormal * (RayMaxDistance * 0.5);

			double OutRayParameter = TNumericLimits<double>::Max();
			const int32 HitTID = Octree.FSparseDynamicOctree3::FindNearestHitObject(
				FRay3d(RayOrigin, RayDirection),
				[&](int tid) { return Mesh->GetTriBounds(tid); },
				[&](int tid, const FRay3d& Ray) 
				{
					UE::Geometry::FIntrRay3Triangle3d Intr = 
						UE::Geometry::TMeshQueries<FDynamicMesh3>::TriangleIntersection(*Mesh, tid, Ray);
					if (Intr.IntersectionType == EIntersectionType::Point)
					{
						OutRayParameter = FMath::Min(OutRayParameter, Intr.RayParameter);
						return Intr.RayParameter;
					}
					return TNumericLimits<double>::Max();

				}, RayMaxDistance);
			if (HitTID != IndexConstants::InvalidID && Mesh->IsTriangle(HitTID))
			{
				MirrorBrushTriangleID = HitTID;
				MirrorBrushTransform.SetTranslation(RayOrigin + RayDirection * OutRayParameter);
			}
		}

		if (IsBrushMirroringInObjectSpace())
		{
			MirrorBrushTransform *= CurTargetTransform;
		}
		
		MirrorBrushIndicator->Update(
			(float)GetCurrentBrushRadius(),
			MirrorBrushTransform,
			1.0f - (float)GetCurrentBrushFalloff());

		UpdateMirrorPlaneMesh();
	}
}


void UMeshVertexAttributePaintToolBase::Shutdown(EToolShutdownType ShutdownType)
{
	if (MeshElementsDisplay)
	{
		// make sure we move back all settings to the display properties so they can be saved in the config
		if (ToolProperties && MeshElementsDisplay->Settings)
		{
			ToolProperties->DisplayProperties.bShowBorders = MeshElementsDisplay->Settings->bShowBorders;
			ToolProperties->DisplayProperties.bShowUVSeams = MeshElementsDisplay->Settings->bShowUVSeams;
			ToolProperties->DisplayProperties.bShowNormalSeams = MeshElementsDisplay->Settings->bShowNormalSeams;
			ToolProperties->DisplayProperties.bShowTangentSeams = MeshElementsDisplay->Settings->bShowTangentSeams;
			ToolProperties->DisplayProperties.bShowColorSeams = MeshElementsDisplay->Settings->bShowColorSeams;
			ToolProperties->DisplayProperties.ThicknessScale = MeshElementsDisplay->Settings->ThicknessScale;
			ToolProperties->DisplayProperties.WireframeColor = MeshElementsDisplay->Settings->WireframeColor;
			ToolProperties->DisplayProperties.BoundaryEdgeColor = MeshElementsDisplay->Settings->BoundaryEdgeColor;
			ToolProperties->DisplayProperties.UVSeamColor = MeshElementsDisplay->Settings->UVSeamColor;
			ToolProperties->DisplayProperties.NormalSeamColor = MeshElementsDisplay->Settings->NormalSeamColor;
			ToolProperties->DisplayProperties.TangentSeamColor = MeshElementsDisplay->Settings->TangentSeamColor;
			ToolProperties->DisplayProperties.ColorSeamColor = MeshElementsDisplay->Settings->ColorSeamColor;
			ToolProperties->DisplayProperties.bShowGroupBoundaries = MeshElementsDisplay->Settings->bShowGroupBoundaries;
			ToolProperties->DisplayProperties.GroupBoundaryColor = MeshElementsDisplay->Settings->GroupBoundaryColor;
			ToolProperties->DisplayProperties.DepthBias = MeshElementsDisplay->Settings->DepthBias;
		}
		MeshElementsDisplay->Disconnect();
		MeshElementsDisplay = nullptr;
	}

	// make sure the tool properties are saved for the next session
	if (ToolProperties)
	{
		SaveBrushConfig(ToolProperties->BrushProperties.BrushMode);

		ToolProperties->DisplayProperties.ColorRamp.OnColorCurveChangedDelegate.RemoveAll(this);
		ToolProperties->SaveProperties(this);
		ToolProperties->SaveConfig();
	}

	if (MeshSelector)
	{
		MeshSelector->Shutdown();
		MeshSelector = nullptr;
	}

	// Using our custom commit method
	if (ShutdownType == EToolShutdownType::Accept)
	{
		ToolDynamicMesh->EditMesh([this](FDynamicMesh3& Mesh)
		{
			CommitToolMesh(Mesh);
		});
	}

	// Clearing the preview mesh so base class commit is No-Op
	if (PreviewMesh)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	if (MirrorBrushIndicatorMesh)
	{
		MirrorBrushIndicatorMesh->Disconnect();
		MirrorBrushIndicatorMesh = nullptr;
	}

	if (MirrorPlaneMesh)
	{
		MirrorPlaneMesh->Disconnect();
		MirrorPlaneMesh = nullptr;
	}

	UE::ToolTarget::ShowSourceObject(Target);
	
	UMeshSculptToolBase::Shutdown(ShutdownType);

	BrushIndicatorMesh = nullptr;
	GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(UE::MeshVertexAttributePaintToolBase::Private::MirrorBrushGizmoType);
}


void UMeshVertexAttributePaintToolBase::EnableValuePicker(bool bEnable)
{
	bPendingPickWeight = bEnable;
	SetFocusInViewport();
}

bool UMeshVertexAttributePaintToolBase::IsValuePickerEnabled() const
{
	return bPendingPickWeight;
}

void UMeshVertexAttributePaintToolBase::EnableBrushMirroring(bool bEnable)
{
	if (ToolProperties)
	{
		ToolProperties->MirrorProperties.bEnableBrushMirroring = bEnable;
		SetFocusInViewport();
	}
}

bool UMeshVertexAttributePaintToolBase::IsBrushMirroringEnabled() const
{
	return ToolProperties ? ToolProperties->MirrorProperties.bEnableBrushMirroring: false;
}

void UMeshVertexAttributePaintToolBase::SetMirrorPlaneWidgetVisible(bool bVisible)
{
	if (ToolProperties)
	{
		ToolProperties->MirrorProperties.bMirrorPlaneWidgetVisible = bVisible;
		UpdateMirrorPlaneMesh();
		SetFocusInViewport();
	}
}

bool UMeshVertexAttributePaintToolBase::IsMirrorPlaneWidgetVisible()
{
	return ToolProperties ? ToolProperties->MirrorProperties.bMirrorPlaneWidgetVisible : true;
}

bool UMeshVertexAttributePaintToolBase::IsBrushMirroringInObjectSpace() const
{
	return ToolProperties ? ToolProperties->MirrorProperties.bObjectSpace : false;
}

void UMeshVertexAttributePaintToolBase::SetMirrorAxis(EAxis::Type InAxis)
{
	if (ToolProperties)
	{
		ToolProperties->MirrorProperties.MirrorAxis = InAxis;
		UpdateMirrorPlaneMesh();

		SetFocusInViewport();
	}
}

EAxis::Type UMeshVertexAttributePaintToolBase::GetMirrorAxis() const
{
	return ToolProperties ? ToolProperties->MirrorProperties.MirrorAxis.GetValue() : EAxis::X;
}

void UMeshVertexAttributePaintToolBase::SetMeshSelectionTool(EMeshSelectorTool SelectorTool)
{
	if (MeshSelector)
	{
		MeshSelector->SetSelectionTool(SelectorTool);
		SetFocusInViewport();
	}
}

EMeshSelectorTool UMeshVertexAttributePaintToolBase::GetMeshSelectionTool() const
{
	return MeshSelector
		? MeshSelector->GetSelectionTool()
		: EMeshSelectorTool::Ray;
}
FInputChord UMeshVertexAttributePaintToolBase::GreyscaleDisplayInputChord = FInputChord(EModifierKey::Control, EKeys::NumPadSeven);
FInputChord UMeshVertexAttributePaintToolBase::ColorMapDisplayInputChord = FInputChord(EModifierKey::Control, EKeys::NumPadEight);
FInputChord UMeshVertexAttributePaintToolBase::MaterialColorDisplayInputChord = FInputChord(EModifierKey::Control, EKeys::NumPadSix);

FInputChord UMeshVertexAttributePaintToolBase::VerticesSelectionModeInputChord = FInputChord(EModifierKey::None, EKeys::One);
FInputChord UMeshVertexAttributePaintToolBase::EdgesSelectionModeInputChord = FInputChord(EModifierKey::None, EKeys::Two);
FInputChord UMeshVertexAttributePaintToolBase::FacesSelectionModeInputChord = FInputChord(EModifierKey::None, EKeys::Three);

FInputChord UMeshVertexAttributePaintToolBase::GrowSelectionInputChord = FInputChord(EModifierKey::None, EKeys::Period);
FInputChord UMeshVertexAttributePaintToolBase::ShrinkSelectionInputChord = FInputChord(EModifierKey::None, EKeys::Comma);
FInputChord UMeshVertexAttributePaintToolBase::FloodSelectionInputChord = FInputChord(EModifierKey::Control, EKeys::A);

FInputChord UMeshVertexAttributePaintToolBase::InvertMirrorDirectionInputChord = FInputChord(EModifierKey::Control, EKeys::M);
FInputChord UMeshVertexAttributePaintToolBase::MirrorValuesInputChord = FInputChord(EModifierKey::None, EKeys::M);

FInputChord UMeshVertexAttributePaintToolBase::CopyAverageValueInputChord = FInputChord(EModifierKey::Control, EKeys::C);
FInputChord UMeshVertexAttributePaintToolBase::PasteAverageValueInputChord = FInputChord(EModifierKey::Control, EKeys::V);

FInputChord UMeshVertexAttributePaintToolBase::ToggleValuePickerInputChord = FInputChord(EModifierKey::None, EKeys::T);

bool UMeshVertexAttributePaintToolBase::SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex)
{
	// Add a dummy layer to store the painted weights for testing purposes, the base tool does not actually commit the tool mesh to the tool target
	const int32 NumAttributeLayers = InOutToolMesh.Attributes()->NumWeightLayers();
	InOutToolMesh.Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
	UE::Geometry::FDynamicMeshWeightAttribute* ActiveWeightMap = InOutToolMesh.Attributes()->GetWeightLayer(NumAttributeLayers);
	ActiveWeightMap->SetName(FName("PaintLayer"));

	OutInitialAttributeIndex = NumAttributeLayers;
	
	return true;
}

bool UMeshVertexAttributePaintToolBase::SetAttributeToPaint(int32 InNewAttributeIndex)
{
	if (!VertexData.IsValid())
	{
		return false;
	}

	bool bSuccess = VertexData.SetAttributeToEdit(InNewAttributeIndex);

	if (bSuccess)
	{
		UpdatePreview();
	}

	return bSuccess;
}

void UMeshVertexAttributePaintToolBase::CommitToolMesh(FDynamicMesh3& InToolMesh)
{
	// Base Tool does not commit anything.
}

void UMeshVertexAttributePaintToolBase::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	// IMPORTANT : we do not register the base class actions as they do not map enough to the wanted actions for this tool

	int32 ActionID = (int32)EStandardToolActions::BaseClientDefinedActionID + 500;

	//-------------------------------------------------------------------------------------------------------
	// Color modes

	ActionSet.RegisterAction(this, ActionID,
		TEXT("MeshDisplayGreyscale"),
		LOCTEXT("MeshDisplayGreyscale", "Greyscale Display"),
		LOCTEXT("MeshDisplayGreyscaleTooltip", "Use greyscale color ramp to display values"),
		GreyscaleDisplayInputChord.GetModifierKey(), GreyscaleDisplayInputChord.Key,
		[this]() { SetColorMode(EMeshVertexAttributePaintToolColorMode::Greyscale); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("MeshDisplayColorRamp"),
		LOCTEXT("MeshDisplayColorRamp", "Color Ramp Display"),
		LOCTEXT("MeshDisplayColorRampTooltip", "Use custom color ramp to display values"),
		ColorMapDisplayInputChord.GetModifierKey(), ColorMapDisplayInputChord.Key,
		[this]() { SetColorMode(EMeshVertexAttributePaintToolColorMode::Ramp); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("MeshDisplayFullMaterial"),
		LOCTEXT("MeshDisplayFullMaterial", "Full Material Display"),
		LOCTEXT("MeshDisplayFullMaterialTooltip", "Use normal material color to display values"),
		MaterialColorDisplayInputChord.GetModifierKey(), MaterialColorDisplayInputChord.Key,
		[this]() { SetColorMode(EMeshVertexAttributePaintToolColorMode::FullMaterial); });
	++ActionID;

	//-------------------------------------------------------------------------------------------------------
	// Component selection mode

	ActionSet.RegisterAction(this, ActionID,
		TEXT("ComponentSelectionModeVertices"),
		LOCTEXT("ComponentSelectionModeVertices", "Vertex Selection Mode"),
		LOCTEXT("ComponentSelectionModeVerticesTooltip", "Enable vertex selection mode"),
		VerticesSelectionModeInputChord.GetModifierKey(), VerticesSelectionModeInputChord.Key,
		[this]() { SetComponentSelectionMode(EComponentSelectionMode::Vertices); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("ComponentSelectionModeEdges"),
		LOCTEXT("ComponentSelectionModeEdges", "Edge Selection Mode"),
		LOCTEXT("ComponentSelectionModeEdgesTooltip", "Enable edge selection mode"),
		EdgesSelectionModeInputChord.GetModifierKey(), EdgesSelectionModeInputChord.Key,
		[this]() { SetComponentSelectionMode(EComponentSelectionMode::Edges); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("ComponentSelectionModeFaces"),
		LOCTEXT("ComponentSelectionModeFaces", "Face Selection Mode"),
		LOCTEXT("ComponentSelectionModeFacesTooltip", "Enable face selection mode"),
		FacesSelectionModeInputChord.GetModifierKey(), FacesSelectionModeInputChord.Key,
		[this]() { SetComponentSelectionMode(EComponentSelectionMode::Faces); });
	++ActionID;

	//-------------------------------------------------------------------------------------------------------
	// Selection actions

	ActionSet.RegisterAction(this, ActionID,
		TEXT("SelectionGrow"),
		LOCTEXT("SelectionGrow", "Grow Selection"),
		LOCTEXT("SelectionGrowTooltip", "Grow the current selection"),
		GrowSelectionInputChord.GetModifierKey(), GrowSelectionInputChord.Key,
		[this]() { GrowSelection(); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("SelectionShrink"),
		LOCTEXT("SelectionShrink", "Shrink Selection"),
		LOCTEXT("SelectionShrinkTooltip", "Shrink the current selection"),
		ShrinkSelectionInputChord.GetModifierKey(), ShrinkSelectionInputChord.Key,
		[this]() { ShrinkSelection(); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("SelectionFlood"),
		LOCTEXT("SelectionFlood", "Flood Selection"),
		LOCTEXT("SelectionFloodTooltip", "Flood the current selection"),
		FloodSelectionInputChord.GetModifierKey(), FloodSelectionInputChord.Key,
		[this]() { FloodSelection(); });
	++ActionID;

	//-------------------------------------------------------------------------------------------------------
	// Mirror actions

	ActionSet.RegisterAction(this, ActionID,
		TEXT("MirrorDirection"),
		LOCTEXT("MirrorDirection", "Invert Mirror Direction"),
		LOCTEXT("MirrorDirectionTooltip", "Invert the current mirror direction"),
		InvertMirrorDirectionInputChord.GetModifierKey(), InvertMirrorDirectionInputChord.Key,
		[this]() 
		{ 
			if (ToolProperties)
			{
				ToolProperties->MirrorProperties.MirrorDirection =
					(ToolProperties->MirrorProperties.MirrorDirection == EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative)
					? EMeshVertexAttributePaintToolMirrorDirection::NegativeToPositive
					: EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative;
			}
		});
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("MirrorValues"),
		LOCTEXT("MirrorValues", "Mirror Values"),
		LOCTEXT("MirrorValuesTooltip", "Mirror the current selection values"),
		MirrorValuesInputChord.GetModifierKey(), MirrorValuesInputChord.Key,
		[this]() { OnMirrorCommand(); });
	++ActionID;

	//-------------------------------------------------------------------------------------------------------
	// Copy Paste

	ActionSet.RegisterAction(this, ActionID,
		TEXT("CopyValue"),
		LOCTEXT("CopyValue", "Copy Average Value"),
		LOCTEXT("CopyValueTooltip", "Copy the current selection average value"),
		CopyAverageValueInputChord.GetModifierKey(), CopyAverageValueInputChord.Key,
		[this]() { CopyAverageFromSelectionToClipboard(); });
	++ActionID;

	ActionSet.RegisterAction(this, ActionID,
		TEXT("PasteValue"),
		LOCTEXT("PasteValue", "Paste Value"),
		LOCTEXT("PasteValueTooltip", "Paste value onto the current selection"),
		PasteAverageValueInputChord.GetModifierKey(), PasteAverageValueInputChord.Key,
		[this]() { PasteValueToSelectionFromClipboard(); });
	++ActionID;

	//-------------------------------------------------------------------------------------------------------
	// Value picking

	ActionSet.RegisterAction(this, ActionID,
		TEXT("PickValueUnderCursor"),
		LOCTEXT("PickValueUnderCursor", "Pick Value"),
		LOCTEXT("PickValueUnderCursorTooltip", "Set the active value to that currently under the cursor"),
		ToggleValuePickerInputChord.GetModifierKey(), ToggleValuePickerInputChord.Key,
		[this]() { EnableValuePicker(!IsValuePickerEnabled()); });
	++ActionID;

};

void UMeshVertexAttributePaintToolBase::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}

FBox UMeshVertexAttributePaintToolBase::GetWorldSpaceFocusBox()
{
	if (ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh)
	{
		if (PreviewMesh && PreviewMesh->GetMesh())
		{
			const TArray<int32>& SelectedVertices = GetSelectedVertices();
			if (!SelectedVertices.IsEmpty())
			{
				const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
				const FTransform3d Transform(PreviewMesh->GetTransform());

				FBox MeshBounds(EForceInit::ForceInit);
				for (const int32 VertexID : SelectedVertices)
				{
					MeshBounds += Transform.TransformPosition(Mesh->GetVertex(VertexID));
				}
				return static_cast<FBox>(MeshBounds);
			}
			return FBox(MeshLocalBounds);
		}

	}
	return Super::GetWorldSpaceFocusBox();
}

float UMeshVertexAttributePaintToolBase::GetBrushMinRadius() const
{
	return BrushProperties? BrushProperties->BrushSize.WorldSizeRange.Min * 0.5f: 0.001f;
}

float UMeshVertexAttributePaintToolBase::GetBrushMaxRadius() const
{
	return BrushProperties ? BrushProperties->BrushSize.WorldSizeRange.Max * 0.5f: 20.0f;
}

float UMeshVertexAttributePaintToolBase::GetBrushAdaptiveSize() const
{
	return BrushProperties ? BrushProperties->BrushSize.AdaptiveSize : 1.0f;
}

void UMeshVertexAttributePaintToolBase::SetBrushAdaptiveSize(float AdaptiveSize)
{
	if (BrushProperties)
	{
		if (UMeshSculptToolBase::BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
		{
			BrushProperties->BrushSize.AdaptiveSize = FMath::Clamp(AdaptiveSize, 0.0f, 1.0f);
		}
		else
		{
			BrushProperties->BrushSize.WorldRadius = FMath::Clamp(AdaptiveSize * GetBrushMaxRadius(), BrushProperties->BrushSize.WorldSizeRange.Min, BrushProperties->BrushSize.WorldSizeRange.Max);
		}
		NotifyOfPropertyChangeByTool(BrushProperties);
		CalculateBrushRadius();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::SetBrushRadius(float NewRadius)
{
	if (BrushProperties)
	{
		if (UMeshSculptToolBase::BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
		{
			BrushProperties->BrushSize.AdaptiveSize = FMath::Clamp(NewRadius / FMath::Max(0.0001f, GetBrushMaxRadius()), 0.0f, 1.0f);
		}
		else
		{
			BrushProperties->BrushSize.WorldRadius = FMath::Clamp(NewRadius, BrushProperties->BrushSize.WorldSizeRange.Min, BrushProperties->BrushSize.WorldSizeRange.Max);
		}

		if (ToolProperties && ToolProperties->bSyncBrushRadiusAcrossModes)
		{
			ToolProperties->SetSharedBrushSize(GetBrushAdaptiveSize());
		}

		NotifyOfPropertyChangeByTool(BrushProperties);
		CalculateBrushRadius();
		SetFocusInViewport();
	}
}

float UMeshVertexAttributePaintToolBase::GetBrushFalloff() const
{
	return BrushProperties ? BrushProperties->BrushFalloffAmount : 1.0f;
}

void UMeshVertexAttributePaintToolBase::SetBrushFalloff(float NewFalloff)
{
	if (BrushProperties)
	{
		BrushProperties->BrushFalloffAmount = FMath::Clamp(NewFalloff, 0.0f, 1.0f);
		NotifyOfPropertyChangeByTool(BrushProperties);
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::SaveBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode)
{
	if (ToolProperties)
	{
		// Save the current brush config 
		FMeshVertexAttributePaintToolBrushConfig BrushConfigToSave;
		BrushConfigToSave.BrushSize = GetBrushAdaptiveSize();
		BrushConfigToSave.Value = ToolProperties->BrushProperties.AttributeValue;
		BrushConfigToSave.Falloff = GetBrushFalloff();
		ToolProperties->SetBrushConfig(BrushMode, BrushConfigToSave);

		if (ToolProperties->bSyncBrushRadiusAcrossModes)
		{
			ToolProperties->SetSharedBrushSize(BrushConfigToSave.BrushSize);
		}
	}
}

void UMeshVertexAttributePaintToolBase::LoadBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode)
{
	const FMeshVertexAttributePaintToolBrushConfig& NewBrushConfig = ToolProperties->GetBrushConfig(BrushMode);
	SetBrushAdaptiveSize(NewBrushConfig.BrushSize);

	float UseValue = NewBrushConfig.Value;
	if (BrushMode == EMeshVertexAttributePaintToolEditOperation::Multiply ||
		BrushMode == EMeshVertexAttributePaintToolEditOperation::Add)
	{
		UseValue = FMath::Clamp(UseValue, 0.0f, 1.0f);
	}
	ToolProperties->BrushProperties.AttributeValue = UseValue;

	SetBrushFalloff(NewBrushConfig.Falloff);

	FPropertyChangedEvent ValuePropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, AttributeValue)));
	ToolProperties->PostEditChangeProperty(ValuePropertyChangedEvent);

}

void UMeshVertexAttributePaintToolBase::SetEditingMode(EMeshVertexAttributePaintToolEditMode Editmode)
{
	if (ToolProperties)
	{
		ToolProperties->EditingMode = Editmode;
		SetFocusInViewport();
	}
}

EMeshVertexAttributePaintToolEditMode UMeshVertexAttributePaintToolBase::GetEditingMode() const
{
	return ToolProperties->EditingMode;
}

void UMeshVertexAttributePaintToolBase::SetBrushMode(EMeshVertexAttributePaintToolEditOperation NewBrushMode)
{
	if (ToolProperties && BrushProperties)
	{
		const EMeshVertexAttributePaintToolEditOperation CurrentBrushMode = ToolProperties->BrushProperties.BrushMode;
		ToolProperties->BrushProperties.BrushMode = NewBrushMode;

		// Save the current brush config 
		SaveBrushConfig(CurrentBrushMode);

		// Load the new brush config 
		LoadBrushConfig(NewBrushMode);

		// Send notification for the changed properties
		FPropertyChangedEvent BrushModePropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolBrushProperties, BrushMode)));
		ToolProperties->PostEditChangeProperty(BrushModePropertyChangedEvent);

		ToolProperties->SaveConfig();
		BrushProperties->SaveConfig();

		UpdateBrushType(NewBrushMode);

		SetFocusInViewport();
	}
}

bool UMeshVertexAttributePaintToolBase::IsInBrushMode() const
{
	return (ToolProperties && ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Brush);
}

void UMeshVertexAttributePaintToolBase::SetBrushAreaMode(EMeshVertexPaintBrushAreaType BrushAreaMode)
{
	if (ToolProperties)
	{
		ToolProperties->BrushProperties.BrushAreaMode = BrushAreaMode;
		SetFocusInViewport();
	}
}

bool UMeshVertexAttributePaintToolBase::IsVolumetricBrush() const
{
	return (ToolProperties && ToolProperties->BrushProperties.BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
}

void UMeshVertexAttributePaintToolBase::OnBeginStroke(const FRay& WorldRay)
{
	if (!VertexData.IsValid())
	{
		return;
	}

	bIsMirrorHiddenBecauseOfBrushStroke = (ToolProperties && ToolProperties->MirrorProperties.bHideOnBrushStroke);

	UpdateBrushPosition(WorldRay);

	const bool bPaintOpActive =
		(ToolProperties->BrushProperties.BrushMode != EMeshVertexAttributePaintToolEditOperation::Relax);
		
	if (PaintBrushOpProperties && bPaintOpActive)
	{
		float UseValue = ToolProperties->BrushProperties.AttributeValue;
		if (GetInInvertStroke())
		{
			switch (ToolProperties->BrushProperties.BrushMode)
			{
			case EMeshVertexAttributePaintToolEditOperation::Add:
				UseValue *= -1.0f;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Replace:
				UseValue = 1.0f - UseValue;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Multiply:
				UseValue = 1.0f + UseValue;
				break;
			case EMeshVertexAttributePaintToolEditOperation::Invert:
			case EMeshVertexAttributePaintToolEditOperation::Relax:
			default:
				break;
			}
		}
		PaintBrushOpProperties->AttributeValue = UseValue;
		PaintBrushOpProperties->EditOperation = ToolProperties->BrushProperties.BrushMode;
		PaintBrushOpProperties->Strength = 1.0f; // Attribute painting only need the attribute value 
		PaintBrushOpProperties->SetFalloff(UMeshSculptToolBase::BrushProperties->BrushFalloffAmount);
	}

	// Setup for Primary & Secondary Relax
	if (SmoothBrushOpProperties)
	{
		SmoothBrushOpProperties->Strength = ToolProperties->BrushProperties.AttributeValue;
		SmoothBrushOpProperties->Falloff = UMeshSculptToolBase::BrushProperties->BrushFalloffAmount;
	}

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = GetBrushFrameWorld();
	LastStamp.LocalFrame = GetBrushFrameLocal();
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActivePressure() * GetCurrentBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	FSculptBrushOptions SculptOptions;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	bFirstStampPending = true;
	
	// begin change here? or wait for first stamp?
	BeginChange();
	
	// Pick the stamp accumulation mode for this stroke. Only the smooth/relax brush ever accumulates;
	// every other brush goes through max-falloff gating.
	const bool bRelaxActive =
		(ToolProperties->BrushProperties.BrushMode == EMeshVertexAttributePaintToolEditOperation::Relax)
		|| GetInSmoothingStroke();
		
	const bool bAccumulating = bRelaxActive && ToolProperties->RelaxBrushAdvancedConfig.bAccumulate;
	
	using namespace UE::MeshVertexAttributePaintToolBase::Private;
	StrokeAccumulator->Begin(bAccumulating ? EStampAccumulatorMode::Accumulate : EStampAccumulatorMode::GatedByMaxFalloff, VertexData);
}

void UMeshVertexAttributePaintToolBase::OnEndStroke()
{
	if (!VertexData.IsValid())
	{
		return;
	}

	bIsMirrorHiddenBecauseOfBrushStroke = false;

	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	UpdatePreview(&TriangleROI, &VertexROI);

	StrokeAccumulator->End();

	// close change record
	EndChange();

	if (bPendingPickWeight)
	{
		if (GetSculptMesh()->IsTriangle(GetBrushTriangleID())
			&& ToolProperties->BrushProperties.BrushMode != EMeshVertexAttributePaintToolEditOperation::Relax)
		{
			ToolProperties->BrushProperties.AttributeValue = GetCurrentWeightValueUnderBrush();
			NotifyOfPropertyChangeByTool(ToolProperties);
		}
		EnableValuePicker(false);
	}
}

void UMeshVertexAttributePaintToolBase::OnCancelStroke()
{
	if (!VertexData.IsValid())
	{
		return;
	}
	if (const TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp())
	{
		UseBrushOp->CancelStroke();
	}
	StrokeAccumulator->End();
	CancelChange();
}

void UMeshVertexAttributePaintToolBase::UpdateROI(const FSculptBrushStamp& BrushStamp, int32 BrushTriangleID)
{
	SCOPE_CYCLE_COUNTER(VertexAttributePaintTool_UpdateROI);

	const FVector3d& BrushPos = BrushStamp.LocalFrame.Origin;
	const FDynamicMesh3* Mesh = GetSculptMesh();
	const float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	UE::Geometry::FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	TriangleROI.Reset();

	int32 CenterTID = BrushTriangleID;
	if (Mesh->IsTriangle(CenterTID))
	{
		TriangleROI.Add(CenterTID);
	}

	FVector3d CenterNormal = Mesh->IsTriangle(CenterTID) ? Mesh->GetTriNormal(CenterTID) : FVector3d::One();		// One so that normal check always passes

	const bool bVolumetric = IsVolumetricBrush();
	const bool bUseAngleThreshold = ToolProperties->BrushProperties.AngleThreshold < 180.0f;
	const double DotAngleThreshold = FMathd::Cos(ToolProperties->BrushProperties.AngleThreshold * FMathd::DegToRad);
	const bool bStopAtUVSeams = ToolProperties->BrushProperties.bUVSeams;
	const bool bStopAtNormalSeams = ToolProperties->BrushProperties.bNormalSeams;

	auto CheckEdgeCriteria = [&](int32 t1, int32 t2) -> bool
		{
			if (bUseAngleThreshold == false || CenterNormal.Dot(Mesh->GetTriNormal(t2)) > DotAngleThreshold)
			{
				int32 eid = Mesh->FindEdgeFromTriPair(t1, t2);
				if (bStopAtUVSeams == false || UVSeamEdges[eid] == false)
				{
					if (bStopAtNormalSeams == false || NormalSeamEdges[eid] == false)
					{
						return true;
					}
				}
			}
			return false;
		};

	if (bVolumetric)
	{
		Octree.RangeQuery(BrushBox,
			[&](int TriIdx) {
				if ((Mesh->GetTriCentroid(TriIdx) - BrushPos).SquaredLength() < RadiusSqr)
				{
					TriangleROI.Add(TriIdx);
				}
			});
	}
	else
	{
		if (Mesh->IsTriangle(CenterTID))
		{
			TArray<int32> StartROI;
			StartROI.Add(CenterTID);
			UE::Geometry::FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
				[&](int t1, int t2)
				{
					if ((Mesh->GetTriCentroid(t2) - BrushPos).SquaredLength() < RadiusSqr)
					{
						return CheckEdgeCriteria(t1, t2);
					}
					return false;
				});
		}
	}

	// Construct ROI vertex set
	VertexSetBuffer.Reset();
	for (int32 tid : TriangleROI)
	{
		UE::Geometry::FIndex3i Tri = Mesh->GetTriangle(tid);
		VertexSetBuffer.Add(Tri.A);  VertexSetBuffer.Add(Tri.B);  VertexSetBuffer.Add(Tri.C);
	}

	// Apply visibility filter
	if (ToolProperties->BrushProperties.VisibilityFilter != EMeshVertexAttributePaintToolVisibilityType::None)
	{
		TArray<int32> ResultBuffer;
		ApplyVisibilityFilter(VertexSetBuffer, TempROIBuffer, ResultBuffer);
	}

	VertexROI.SetNum(0, EAllowShrinking::No);
	//TODO: If we paint a 2D projection of UVs, these will need to be the 2D vertices not the 3D original mesh vertices
	BufferUtil::AppendElements(VertexROI, VertexSetBuffer);

	// Construct ROI triangle and weight buffers
	ROITriangleBuffer.Reserve(TriangleROI.Num());
	ROITriangleBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : TriangleROI)
	{
		ROITriangleBuffer.Add(tid);
	}
	ROIWeightValueBuffer.SetNum(VertexROI.Num(), EAllowShrinking::No);
	SyncWeightBufferWithMesh();
}

bool UMeshVertexAttributePaintToolBase::UpdateStampPosition(const FRay& WorldRay)
{
	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		UpdateBrushPositionOnSculptMesh(WorldRay, true);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		UpdateBrushPositionOnActivePlane(WorldRay);
		break;
	}

	if (UseBrushOp->GetAlignStampToView() || IsVolumetricBrush())
	{
		AlignBrushToView();
	}

	CurrentStamp = LastStamp;
	CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.WorldFrame = GetBrushFrameWorld();
	CurrentStamp.LocalFrame = GetBrushFrameLocal();
	CurrentStamp.Power = GetActivePressure() * GetCurrentBrushStrength();
	CurrentStamp.Falloff = GetCurrentBrushFalloff();

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	const EAxis::Type MirrorAxis = GetMirrorAxis();

	// update the mirrored stamp
	MirrorCurrentStamp = CurrentStamp;
	MirrorCurrentStamp.LocalFrame = ComputeMirroredFrame(CurrentStamp.LocalFrame);
	MirrorCurrentStamp.WorldFrame = MirrorCurrentStamp.LocalFrame;
	MirrorCurrentStamp.WorldFrame.Transform(CurTargetTransform);
	MirrorCurrentStamp.PrevLocalFrame = ComputeMirroredFrame(CurrentStamp.PrevLocalFrame);
	MirrorCurrentStamp.PrevWorldFrame = MirrorCurrentStamp.PrevLocalFrame;
	MirrorCurrentStamp.PrevWorldFrame.Transform(CurTargetTransform);

	FVector3d MoveDelta = CurrentStamp.LocalFrame.Origin - CurrentStamp.PrevLocalFrame.Origin;

	if (!bFirstStampPending && UseBrushOp->IgnoreZeroMovements() && MoveDelta.SquaredLength() < 0.1 * CurrentBrushRadius)
	{
		return false;
	}

	bFirstStampPending = false;

	return true;
}

bool UMeshVertexAttributePaintToolBase::ApplyStamp(const FSculptBrushStamp& InStamp)
{
	SCOPE_CYCLE_COUNTER(VertexAttributePaintTool_ApplyStamp);

	if (!VertexData.IsValid())
	{
		return false;
	}

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	// yuck
	FMeshVertexAttributePaintToolBrushOpBase* WeightBrushOp = static_cast<FMeshVertexAttributePaintToolBrushOpBase*>(UseBrushOp.Get());
	WeightBrushOp->bApplyRadiusLimit = IsInBrushMode();

	FDynamicMesh3* Mesh = GetSculptMesh();


	StrokeAccumulator->PrepareStampInput(VertexROI, ROIWeightValueBuffer);
	WeightBrushOp->ApplyStampByVertices(Mesh, InStamp, VertexROI, StrokeAccumulator->MakeVertexWeightFunc(), ROIWeightValueBuffer);
	StrokeAccumulator->FinalizeStampOutput(VertexROI, ROIWeightValueBuffer,
		[this, &InStamp, Mesh](int32 V)
		{
			return ComputeStampFalloffAtVertex(InStamp, Mesh->GetVertex(V));
		});

	return SyncMeshWithWeightBuffer();
}

float UMeshVertexAttributePaintToolBase::ComputeStampFalloffAtVertex(const FSculptBrushStamp& Stamp, const FVector3d& VertexPos)
{
	// Delegate the entire "inside brush + falloff" computation to the brush op's own
	// falloff function. Every falloff in StampFalloffs.h returns 0 at and beyond
	// Stamp.Radius, so we get the radial cutoff for free and stay consistent with
	// whatever the op uses to scale the stamp's effect. Doing our own radius pre-check
	// would tie this code to a specific radius choice (Stamp.Radius vs the live
	// GetCurrentBrushRadius() used by UpdateROI) and risk disagreeing with the op.
	const TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	if (!UseBrushOp.IsValid() || !UseBrushOp->Falloff.IsValid())
	{
		return 1.0f;
	}
	return (float)UseBrushOp->Falloff->Evaluate(Stamp, VertexPos);
}

bool UMeshVertexAttributePaintToolBase::SyncMeshWithWeightBuffer()
{
	int32 NumModified = 0;
	if (VertexData.IsValid())
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		const int32 NumVertexROI = VertexROI.Num();
		for (int32 VertexROIIndex = 0; VertexROIIndex < NumVertexROI; ++VertexROIIndex)
		{
			const int VertIdx = VertexROI[VertexROIIndex];
			if (VertIdx != INDEX_NONE)
			{
				const float NewValue = ROIWeightValueBuffer[VertexROIIndex];
				VertexData.SetValue(VertIdx, NewValue);
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

bool UMeshVertexAttributePaintToolBase::SyncWeightBufferWithMesh()
{
	int32 NumModified = 0;
	if (VertexData.IsValid())
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		const int32 NumVertexROI = VertexROI.Num();
		for (int32 VertexROIIndex = 0; VertexROIIndex < NumVertexROI; ++VertexROIIndex)
		{
			const int VertIdx = VertexROI[VertexROIIndex];
			const float CurWeight = VertexData.GetValue(VertIdx);
			if (ROIWeightValueBuffer[VertexROIIndex] != CurWeight)
			{
				ROIWeightValueBuffer[VertexROIIndex] = CurWeight;
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

// TODO(ccaillaud) : move this to a common  place
namespace UE::MeshVertexAttributePaintToolBase::Private
{
	template<typename RealType>
	static bool FindPolylineSelfIntersection(
		const TArray<UE::Math::TVector2<RealType>>& Polyline,
		UE::Math::TVector2<RealType>& IntersectionPointOut,
		UE::Geometry::FIndex2i& IntersectionIndexOut,
		bool bParallel = true)
	{
		int32 N = Polyline.Num();
		std::atomic<bool> bSelfIntersects(false);
		ParallelFor(N - 1, [&](int32 i)
			{
				UE::Geometry::TSegment2<RealType> SegA(Polyline[i], Polyline[i + 1]);
				for (int32 j = i + 2; j < N - 1 && bSelfIntersects == false; ++j)
				{
					UE::Geometry::TSegment2<RealType> SegB(Polyline[j], Polyline[j + 1]);
					if (SegA.Intersects(SegB) && bSelfIntersects == false)
					{
						bool ExpectedValue = false;
						if (std::atomic_compare_exchange_strong(&bSelfIntersects, &ExpectedValue, true))
						{
							UE::Geometry::TIntrSegment2Segment2<RealType> Intersection(SegA, SegB);
							Intersection.Find();
							IntersectionPointOut = Intersection.Point0;
							IntersectionIndexOut = UE::Geometry::FIndex2i(i, j);
							return;
						}
					}
				}
			}, (bParallel) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		return bSelfIntersects;
	}



	template<typename RealType>
	static bool FindPolylineSegmentIntersection(
		const TArray<UE::Math::TVector2<RealType>>& Polyline,
		const UE::Geometry::TSegment2<RealType>& Segment,
		UE::Math::TVector2<RealType>& IntersectionPointOut,
		int& IntersectionIndexOut)
	{

		int32 N = Polyline.Num();
		for (int32 i = 0; i < N - 1; ++i)
		{
			UE::Geometry::TSegment2<RealType> PolySeg(Polyline[i], Polyline[i + 1]);
			if (Segment.Intersects(PolySeg))
			{
				UE::Geometry::TIntrSegment2Segment2<RealType> Intersection(Segment, PolySeg);
				Intersection.Find();
				IntersectionPointOut = Intersection.Point0;
				IntersectionIndexOut = i;
				return true;
			}
		}
		return false;
	}


	bool ApproxSelfClipPolyline(TArray<FVector2f>& Polyline)
	{
		int32 N = Polyline.Num();

		// handle already-closed polylines
		if (UE::Geometry::Distance(Polyline[0], Polyline[N - 1]) < 0.0001f)
		{
			return true;
		}

		FVector2f IntersectPoint;
		UE::Geometry::FIndex2i IntersectionIndex(-1, -1);
		bool bSelfIntersects = FindPolylineSelfIntersection(Polyline, IntersectPoint, IntersectionIndex);
		if (bSelfIntersects)
		{
			TArray<FVector2f> NewPolyline;
			NewPolyline.Add(IntersectPoint);
			for (int32 i = IntersectionIndex.A; i <= IntersectionIndex.B; ++i)
			{
				NewPolyline.Add(Polyline[i]);
			}
			NewPolyline.Add(IntersectPoint);
			Polyline = MoveTemp(NewPolyline);
			return true;
		}


		FVector2f StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		UE::Geometry::FLine2f StartLine(Polyline[0], StartDirOut);
		FVector2f EndDirOut = UE::Geometry::Normalized(Polyline[N - 1] - Polyline[N - 2]);
		UE::Geometry::FLine2f EndLine(Polyline[N - 1], EndDirOut);
		UE::Geometry::FIntrLine2Line2f LineIntr(StartLine, EndLine);
		bool bIntersects = false;
		if (LineIntr.Find())
		{
			bIntersects = LineIntr.IsSimpleIntersection() && (LineIntr.Segment1Parameter > 0) && (LineIntr.Segment2Parameter > 0);
			if (bIntersects)
			{
				Polyline.Add(StartLine.PointAt(LineIntr.Segment1Parameter));
				Polyline.Add(StartLine.Origin);
				return true;
			}
		}


		UE::Geometry::FAxisAlignedBox2f Bounds;
		for (const FVector2f& P : Polyline)
		{
			Bounds.Contain(P);
		}
		float Size = Bounds.DiagonalLength();

		FVector2f StartPos = Polyline[0] + 0.001f * StartDirOut;
		if (FindPolylineSegmentIntersection(Polyline, UE::Geometry::FSegment2f(StartPos, StartPos + 2 * Size * StartDirOut), IntersectPoint, IntersectionIndex.A))
		{
			return true;
		}

		FVector2f EndPos = Polyline[N - 1] + 0.001f * EndDirOut;
		if (FindPolylineSegmentIntersection(Polyline, UE::Geometry::FSegment2f(EndPos, EndPos + 2 * Size * EndDirOut), IntersectPoint, IntersectionIndex.A))
		{
			return true;
		}

		return false;
	}
}

bool UMeshVertexAttributePaintToolBase::HasSelection() const
{
	if (MeshSelector)
	{
		return MeshSelector->IsAnyComponentSelected();
	}
	return false;
}

void UMeshVertexAttributePaintToolBase::GrowSelection() const
{
	if (MeshSelector)
	{
		MeshSelector->GrowSelection();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::ShrinkSelection() const
{
	if (MeshSelector)
	{
		MeshSelector->ShrinkSelection();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::InvertSelection() const
{
	if (MeshSelector)
	{
		MeshSelector->InvertSelection();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::FloodSelection() const
{
	if (MeshSelector)
	{
		MeshSelector->FloodSelection();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::SelectBorder() const
{
	if (MeshSelector)
	{
		MeshSelector->SelectBorder();
		SetFocusInViewport();
	}
}



void UMeshVertexAttributePaintToolBase::SetComponentSelectionMode(EComponentSelectionMode NewMode)
{
	if (ToolProperties)
	{
		ToolProperties->SelectionProperties.ComponentSelectionMode = NewMode;
		if (MeshSelector)
		{
			MeshSelector->SetComponentSelectionMode(NewMode);
		}
		SetFocusInViewport();
	}
}

const TArray<int32>& UMeshVertexAttributePaintToolBase::GetSelectedVertices() const
{
	static TArray<int32> EmptySelection;
	if (MeshSelector)
	{
		return MeshSelector->GetSelectedVertices();
	}
	return EmptySelection;
}

void UMeshVertexAttributePaintToolBase::ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation Operation, float InValue, bool bWithTransaction)
{
	if (!VertexData.IsValid())
	{
		return;
	}

	VertexROI = UMeshVertexAttributePaintToolBase::GetSelectedVertices();
	if (VertexROI.IsEmpty())
	{
		return;
	}

	if (bWithTransaction)
	{
		BeginChange();
	}
	using namespace UE::MeshVertexAttributePaintToolBase::Private;

	if (FDynamicMesh3* Mesh = GetSculptMesh())
	{
		ROIWeightValueBuffer.SetNum(VertexROI.Num(), EAllowShrinking::No);
		SyncWeightBufferWithMesh();

		if (Operation == EMeshVertexAttributePaintToolEditOperation::Relax)
		{
			// Relax compounds across N iterations — each iteration's smoothing reads the
			// previous iteration's writes
			constexpr int32 NumRelaxIterations = 5;

			auto GetVertexWeightFunc = [this](int32 V) -> double
				{
					return VertexData.GetValue(V);
				};
			
			for (int32 Itr = 0; Itr < NumRelaxIterations; ++Itr)
			{
				FMeshVertexAttributePaintToolSmoothBrushOp::ApplyToVerticesStatic(
					Mesh, VertexROI,
					GetVertexWeightFunc, CotangentEdgeWeights, ROIWeightValueBuffer,
					/*SmoothingFactor*/InValue,
					/*bApplyRadiusLimit*/false, CurrentStamp.LocalFrame.Origin, CurrentStamp.Radius);

				SyncMeshWithWeightBuffer();
			}
		}
		else
		{
			auto GetVertexWeightFunc = [this](int32 V) -> double
				{
					return VertexData.GetPreChangeValue(V);
				};

			FMeshVertexAttributePaintToolPaintBrushOp::ApplyToVerticesStatic(
				Mesh, VertexROI,
				GetVertexWeightFunc, ROIWeightValueBuffer,
				Operation, InValue,
				/*bApplyRadiusLimit*/false, CurrentStamp.LocalFrame.Origin, CurrentStamp.Radius);

			SyncMeshWithWeightBuffer();
		}
	}

	// update colors
	UpdatePreview();
	if (bWithTransaction)
	{
		EndChange();
	}
	SetFocusInViewport();
}

void UMeshVertexAttributePaintToolBase::PruneSelection(float Threshold)
{
	if (!VertexData.IsValid())
	{
		return;
	}

	const TArray<int32>& SelectVertices = UMeshVertexAttributePaintToolBase::GetSelectedVertices();
	if (SelectVertices.IsEmpty())
	{
		return;
	}

	BeginChange();

	for (int32 VertexIdx : SelectVertices)
	{
		float CurrentValue = VertexData.GetValue(VertexIdx);
		float PrunedValue = CurrentValue >= Threshold ? CurrentValue : 0.0f;
		VertexData.SetValue(VertexIdx, PrunedValue);
	}

	UpdatePreview();
	EndChange();

	SetFocusInViewport();
}

namespace UE::MeshVertexAttributePaintToolBase::Private
{
	const FString CopyAverageFromSelectionToClipboardIdentifier = TEXT("UE_MeshVertexAttributePaintTool_AverageValue:");
}

void UMeshVertexAttributePaintToolBase::CopyAverageFromSelectionToClipboard()
{
	using namespace UE::MeshVertexAttributePaintToolBase::Private;

	if (!VertexData.IsValid())
	{
		return;
	}

	const TArray<int32>& SelectVertices = GetSelectedVertices();
	if (SelectVertices.IsEmpty())
	{
		const FText NotificationText = FText::FromString(TEXT("No vertices were selected. Nothing was copied to the clipboard."));
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	const float AverageValue = VertexData.GetAverageValue(SelectVertices);

	// copy to clipboard
	const FString ClipboardString = FString::Format(TEXT("{0}{1}"), { CopyAverageFromSelectionToClipboardIdentifier, FString::SanitizeFloat(AverageValue) });
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);

	// notify user
	const FText NotificationText = FText::FromString("Selection average value copied to clipboard.");
	ShowEditorMessage(ELogVerbosity::Log, NotificationText);

	SetFocusInViewport();
}

void UMeshVertexAttributePaintToolBase::PasteValueToSelectionFromClipboard()
{
	using namespace UE::MeshVertexAttributePaintToolBase::Private;

	if (!HasSelection())
	{
		const FText NotificationText = FText::FromString("No vertices were selected. Nothing was pasted.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	// get the clipboard content and check if it matches our format
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.StartsWith(CopyAverageFromSelectionToClipboardIdentifier))
	{
		const FText NotificationText = FText::FromString("Failed to paste value from clipboard. Format is incompatible.");
		ShowEditorMessage(ELogVerbosity::Fatal, NotificationText);
		return;
	}

	// parse the value from the clipboard string
	const FString StringValue = ClipboardContent.RightChop(CopyAverageFromSelectionToClipboardIdentifier.Len());
	float ValueToPaste = 0.0f;
	if (StringValue.IsNumeric())
	{
		LexFromString(ValueToPaste, *StringValue);
		ValueToPaste = FMath::Clamp(ValueToPaste, 0.0f, 1.0f);

		ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation::Replace, ValueToPaste);
	}
	else
	{
		const FText NotificationText = FText::FromString("Failed to paste value from clipboard, Value from the clipboard is not numeric.");
		ShowEditorMessage(ELogVerbosity::Warning, NotificationText);
	}
	
	// notify user
	const FText NotificationText = FText::FromString("Pasted weights.");
	ShowEditorMessage(ELogVerbosity::Log, NotificationText);

	SetFocusInViewport();
}

void UMeshVertexAttributePaintToolBase::OnSelectionModified()
{

}


bool UMeshVertexAttributePaintToolBase::HaveVisibilityFilter() const
{
	return (ToolProperties && ToolProperties->BrushProperties.VisibilityFilter != EMeshVertexAttributePaintToolVisibilityType::None);
}

void UMeshVertexAttributePaintToolBase::ApplyVisibilityFilter(TSet<int32>& Vertices, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer)
{
	ROIBuffer.SetNum(0, EAllowShrinking::No);
	ROIBuffer.Reserve(Vertices.Num());
	for (int32 VertexIdx : Vertices)
	{
		ROIBuffer.Add(VertexIdx);
	}

	OutputBuffer.Reset();
	ApplyVisibilityFilter(ROIBuffer, OutputBuffer);

	Vertices.Reset();
	for (int32 VertexIdx : OutputBuffer)
	{
		Vertices.Add(VertexIdx);
	}
}

void UMeshVertexAttributePaintToolBase::ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices)
{
	if (!HaveVisibilityFilter())
	{
		VisibleVertices = Vertices;
		return;
	}

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 NumVertices = Vertices.Num();

	VisibilityFilterBuffer.SetNum(NumVertices, EAllowShrinking::No);
	ParallelFor(NumVertices, [&](int32 idx)
		{
			VisibilityFilterBuffer[idx] = true;
			UE::Geometry::FVertexInfo VertexInfo;
			Mesh->GetVertex(Vertices[idx], VertexInfo, true, false, false);
			const FVector3d Centroid = VertexInfo.Position;
			const FVector3d FaceNormal = (FVector3d)VertexInfo.Normal;
			const FVector3d CameraToVertexNormal = UE::Geometry::Normalized(Centroid - LocalEyePosition);
			if (!FaceNormal.IsZero() && FaceNormal.Dot(CameraToVertexNormal) > 0)
			{
				VisibilityFilterBuffer[idx] = false;
			}
			else if (ToolProperties->BrushProperties.VisibilityFilter == EMeshVertexAttributePaintToolVisibilityType::Unoccluded)
			{
				const FVector RayDirection = CameraToVertexNormal;
				const FVector RayOrigin = LocalEyePosition;
				const double MaxDistance = (LocalEyePosition - Centroid).Size() - UE_SMALL_NUMBER;
				const int32 HitTID = Octree.FindNearestHitObject(FRay3d(RayOrigin, RayDirection), MaxDistance);
				if (HitTID != IndexConstants::InvalidID && Mesh->IsTriangle(HitTID))
				{
					VisibilityFilterBuffer[idx] = false;
				}
			}
		});

	VisibleVertices.Reset();
	for (int32 k = 0; k < NumVertices; ++k)
	{
		if (VisibilityFilterBuffer[k])
		{
			VisibleVertices.Add(Vertices[k]);
		}
	}
}



int32 UMeshVertexAttributePaintToolBase::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	if (!IsInBrushMode())
	{
		return IndexConstants::InvalidID;
	}

	if (GetBrushCanHitBackFaces())
	{
		return Octree.FindNearestHitObject(LocalRay);
	}
	else
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();

		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		int HitTID = Octree.FindNearestHitObject(LocalRay,
			[this, Mesh, &LocalEyePosition](int TriangleID) {
				FVector3d Normal, Centroid;
				double Area;
				Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
				return Normal.Dot((Centroid - LocalEyePosition)) < 0;
			});
		return HitTID;
	}
}

int32 UMeshVertexAttributePaintToolBase::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	check(false);
	return IndexConstants::InvalidID;
}

void UMeshVertexAttributePaintToolBase::UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay)
{
	using namespace UE::Geometry;

	CurrentBaryCentricCoords = FVector3d::Zero();

	const FDynamicMesh3* const Mesh = GetSculptMesh();

	if (Mesh->IsTriangle(TriangleID))
	{
		FTriangle3d Triangle;
		Mesh->GetTriVertices(TriangleID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();
		CurrentBaryCentricCoords = Query.TriangleBaryCoords;
	}
}

bool UMeshVertexAttributePaintToolBase::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false;
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	if (bHit && (UseBrushOp->GetAlignStampToView() || IsVolumetricBrush()))
	{
		AlignBrushToView();
	}

	return bHit;
}

FInputRayHit UMeshVertexAttributePaintToolBase::BeginHoverSequenceHitTest(const FInputDeviceRay& Ray)
{
	FInputRayHit RayHit = Super::BeginHoverSequenceHitTest(Ray);
	bBrushIsHoverMesh = RayHit.bHit;
	return RayHit;
}

bool UMeshVertexAttributePaintToolBase::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);
	}
	return true;
}


void UMeshVertexAttributePaintToolBase::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh)
	{
		if (MeshSelector)
		{
			MeshSelector->DrawHUD(Canvas, RenderAPI);
		}
	}

	if (BrushEditBehavior.IsValid())
	{
		BrushEditBehavior->DrawHUD(Canvas, RenderAPI);
	}

	const bool bInteractiveBrushIsAdjusting = (BrushEditBehavior.IsValid() && BrushEditBehavior->IsEditing());

	// when the user adjust the brush interactively we hide the value on brush
	if (!bInteractiveBrushIsAdjusting && IsInBrushMode())
	{
		if (Canvas && RenderAPI && BrushIndicator && bBrushIsHoverMesh)
		{
			// Display the value under the brush
			const float BrushRadius = BrushIndicator->BrushRadius;
			const FVector HoverPosition = HoverStamp.WorldFrame.ToFTransform().GetTranslation();

			FVector2D ScreenBrushPos2D;
			static FVector2D Offset(0,75.0);
			RenderAPI->GetSceneView()->WorldToPixel(HoverPosition, ScreenBrushPos2D);
			ScreenBrushPos2D += Offset;
			ScreenBrushPos2D /= Canvas->GetDPIScale();

			// Render value under at brush even in mesh mode
			const FText BaseText = bPendingPickWeight
				? LOCTEXT("MeshVertexAttributePaintToolPaintTool_PickValueAtBrush_TextFormat", "Pick Value: {0}")
				: LOCTEXT("MeshVertexAttributePaintToolPaintTool_ValueAtBrush_TextFormat", "Value at brush: {0}");
			const FText ValueAtBrushText = FText::Format(BaseText, FText::AsNumber(ToolProperties->BrushProperties.ValueAtBrush));
			FCanvasTextItem TextItem(ScreenBrushPos2D, ValueAtBrushText, GEngine->GetLargeFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.bCentreX = true;
			Canvas->DrawItem(TextItem);
		}
	}

	if (ToolProperties && ToolProperties->DisplayProperties.bShowValues && VertexData.IsValid())
	{
		if (Canvas && RenderAPI)
		{
			if (const FDynamicMesh3* Mesh = GetSculptMesh())
			{
				FViewCameraState StateOut;
				GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
				const FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
				const FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

				auto IsVertexVisible =
					[&LocalEyePosition, &Octree = Octree](const FDynamicMesh3& Mesh, const int32 VtxIndex)
					{
						UE::Geometry::FVertexInfo VertexInfo;
						Mesh.GetVertex(VtxIndex, VertexInfo, true, false, false);
						const FVector3d Centroid = VertexInfo.Position;
						const FVector3d FaceNormal = FVector3d(VertexInfo.Normal);
						const FVector3d CameraToVertexNormal = UE::Geometry::Normalized(Centroid - LocalEyePosition);
						if (!FaceNormal.IsZero() && FaceNormal.Dot(CameraToVertexNormal) > 0)
						{
							return false;
						}
						const FVector RayDirection = CameraToVertexNormal;
						const FVector RayOrigin = LocalEyePosition;
						const double MaxDistance = (LocalEyePosition - Centroid).Size() - UE_SMALL_NUMBER;
						const int32 HitTID = Octree.FindNearestHitObject(FRay3d(RayOrigin, RayDirection), MaxDistance);
						if (HitTID != IndexConstants::InvalidID && Mesh.IsTriangle(HitTID))
						{
							return false;
						}
						return true;
					};

				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.AlwaysSign = false;
				NumberFormattingOptions.UseGrouping = false;
				NumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;
				NumberFormattingOptions.MinimumIntegralDigits = 1;
				NumberFormattingOptions.MaximumIntegralDigits = 6;
				NumberFormattingOptions.MinimumFractionalDigits = 2;
				NumberFormattingOptions.MaximumFractionalDigits = 2;

				const float MinValue = ToolProperties->DisplayProperties.MinValue;
				const float MaxValue = ToolProperties->DisplayProperties.MaxValue;
				const float ValueRange = MaxValue - MinValue;

				UFont* EngineLargeFont = GEngine->GetLargeFont();

				const int32 MaxVertexId = Mesh->MaxVertexID();
				TArray<bool> VertexVisibility;
				VertexVisibility.Init(false, MaxVertexId);

				ParallelFor(Mesh->MaxVertexID(), [this, &VertexVisibility, &Mesh, IsVertexVisible](int32 VertexID)
					{
						if (Mesh->IsVertex(VertexID))
						{
							if (IsVertexVisible(*Mesh, VertexID))
							{
								VertexVisibility[VertexID] = true;
							}
						}
					});

				auto DrawValue = 
					[this, &VertexVisibility, &MinValue, &MaxValue, &ValueRange, &NumberFormattingOptions, &Canvas, &RenderAPI, &Mesh]
					(int32 VertexID)
					{
						if (VertexVisibility.IsValidIndex(VertexID) && VertexVisibility[VertexID])
						{
							const float NormalizedValue = VertexData.GetValue(VertexID);
							const float RemappedValue = MinValue + NormalizedValue * ValueRange;
							const FText ValueText = FText::AsNumber(RemappedValue, &NumberFormattingOptions);

							const FVector3d VtxPosition = Mesh->GetVertex(VertexID);
							FVector2D ScreenBrushPos2D;
							RenderAPI->GetSceneView()->WorldToPixel(VtxPosition, ScreenBrushPos2D);
							ScreenBrushPos2D /= Canvas->GetDPIScale();

							FCanvasTextItem TextItem(ScreenBrushPos2D, ValueText, GEngine->GetLargeFont(), FLinearColor::White);
							TextItem.EnableShadow(FLinearColor::Black);
							TextItem.bCentreX = true;
							Canvas->DrawItem(TextItem);
						}
					};

				if (IsInBrushMode() || ToolProperties->DisplayProperties.bShowValuesOnlySelected == false)
				{
					for (int32 VertexID = 0; VertexID < MaxVertexId; ++VertexID)
					{
						DrawValue(VertexID);
					}
				}
				else
				{
					const TArray<int32>& SelectedVertices = GetSelectedVertices();
					for (const int32 VertexID : SelectedVertices)
					{
						DrawValue(VertexID);
					}
				}
			}
		}
	}
}

void UMeshVertexAttributePaintToolBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSculptToolBase::Render(RenderAPI);

	if (ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh)
	{
		if (MeshSelector)
		{
			MeshSelector->Render(RenderAPI);
		}
	}
}


void UMeshVertexAttributePaintToolBase::UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode)
{
	if (MaterialMode == EMeshEditingMaterialModes::VertexColor)
	{
		constexpr bool bUseTwoSidedMaterial = true;
		ActiveOverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), bUseTwoSidedMaterial);
		if (ensure(ActiveOverrideMaterial != nullptr))
		{
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
			ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
		}
		GetSculptMeshComponent()->SetShadowsEnabled(false);
	}
	else
	{
		UMeshSculptToolBase::UpdateMaterialMode(MaterialMode);
	}
}

void UMeshVertexAttributePaintToolBase::OnTick(float DeltaTime)
{
	UMeshSculptToolBase::OnTick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);

	const bool bIsMeshEditMode = (ToolProperties->EditingMode == EMeshVertexAttributePaintToolEditMode::Mesh);
	if (MeshSelector)
	{
		MeshSelector->SetIsEnabled(bIsMeshEditMode);
		MeshSelector->Tick(DeltaTime);
	}

	ConfigureIndicator(IsVolumetricBrush());
	SetIndicatorVisibility(!bIsMeshEditMode && bBrushIsHoverMesh && !bPendingPickWeight);
	UpdateMirrorIndicator();

	SCOPE_CYCLE_COUNTER(VertexAttributePaintTool_Tick);

	// Get value at brush location
	if (IsInBrushMode())
	{
		if (GetSculptMesh()->IsTriangle(GetBrushTriangleID()))
		{
			const double HitWeightValue = GetCurrentWeightValueUnderBrush();
			ToolProperties->BrushProperties.ValueAtBrush = HitWeightValue;
		}
	}
	else
	{
		ToolProperties->BrushProperties.ValueAtBrush = GetAverageWeightValueFromSelection();
	}

	auto ExecuteStampOperation = [this](int StampIndex, const FRay& StampRay)
		{
			SCOPE_CYCLE_COUNTER(VertexAttributePaintTool_Tick_ApplyStampBlock);

			FDynamicMesh3* Mesh = GetSculptMesh();

			// update sculpt ROI
			UpdateROI(CurrentStamp, GetBrushTriangleID());


			// apply the stamp
			bool bWeightsModified = ApplyStamp(CurrentStamp);

			if (bWeightsModified)
			{
				UpdatePreview(&TriangleROI, &VertexROI);
			}

			LastStamp = CurrentStamp;
			LastStamp.TimeStamp = FDateTime::Now();

			// now mirror stamp 
			if (IsBrushMirroringEnabled())
			{
				bWeightsModified = false;
				// update sculpt ROI
				UpdateROI(MirrorCurrentStamp, MirrorBrushTriangleID);

				// apply the stamp
				bWeightsModified = ApplyStamp(MirrorCurrentStamp);

				if (bWeightsModified)
				{
					UpdatePreview(&TriangleROI, &VertexROI);
				}
			}

		};

	if (IsInBrushMode() && !bPendingPickWeight)
	{
		ProcessPerTickStamps(
			[this](const FRay& StampRay) -> bool {
				return UpdateStampPosition(StampRay);
			}, ExecuteStampOperation);
	}

	if (UToolsContextCursorAPI* ToolsContextCursorAPI = GetToolManager()->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>())
	{
		if (bPendingPickWeight)
		{
			ToolsContextCursorAPI->SetCursorOverride(EMouseCursor::EyeDropper);
		}
		else
		{
			ToolsContextCursorAPI->ClearCursorOverride();
		}
	}

}


bool UMeshVertexAttributePaintToolBase::CanAccept() const
{
	return bAnyChangeMade;
}

void UMeshVertexAttributePaintToolBase::BeginChange()
{
	if (!VertexData.IsValid())
	{
		return;
	}
	LongTransactions.Open(LOCTEXT("WeightPaintChange", "Weight Stroke"), GetToolManager());
	VertexData.BeginChange();
}

void UMeshVertexAttributePaintToolBase::EndChange()
{
	if (!VertexData.IsValid())
	{
		return;
	}

	bAnyChangeMade = true;

	using namespace UE::MeshVertexAttributePaintToolBase;
	TUniquePtr<Private::FMeshChange> MeshChange = VertexData.EndChange();

	TUniquePtr<TWrappedToolCommandChange<Private::FMeshChange>> NewChange = MakeUnique<TWrappedToolCommandChange<Private::FMeshChange>>();
	NewChange->WrappedChange = MoveTemp(MeshChange);
	NewChange->AfterModify = [WeakTool = TWeakObjectPtr<UMeshVertexAttributePaintToolBase>(this)](bool bRevert)
		{
			if (UMeshVertexAttributePaintToolBase* Tool = WeakTool.Get(); Tool && Tool->VertexData.IsValid())
			{
				Tool->VertexData.SyncPreChangeWeightsToCurrentWeights();
				Tool->UpdatePreview();
			}
		};

	GetToolManager()->EmitObjectChange(ToolDynamicMesh, MoveTemp(NewChange), LOCTEXT("WeightPaintChange", "Weight Stroke"));

	VertexData.SyncPreChangeWeightsToCurrentWeights();

	LongTransactions.Close(GetToolManager());
}

void UMeshVertexAttributePaintToolBase::CancelChange()
{
	if (!VertexData.IsValid())
	{
		return;
	}
	VertexData.CancelChange();
	VertexData.SyncPreChangeWeightsToCurrentWeights();
	LongTransactions.Close(GetToolManager());
}


void UMeshVertexAttributePaintToolBase::PrecomputeSeamEdges()
{
	const FDynamicMesh3* Mesh = GetSculptMesh();

	const UE::Geometry::FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();
	const UE::Geometry::FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->PrimaryUV();
	UVSeamEdges.SetNum(Mesh->MaxEdgeID());
	NormalSeamEdges.SetNum(Mesh->MaxEdgeID());
	ParallelFor(Mesh->MaxEdgeID(), [&](int32 eid)
		{
			if (Mesh->IsEdge(eid))
			{
				UVSeamEdges[eid] = UVs ? UVs->IsSeamEdge(eid) : false;
				NormalSeamEdges[eid] = Normals ? Normals->IsSeamEdge(eid) : false;
			}
		});
}

float UMeshVertexAttributePaintToolBase::GetCurrentWeightValueUnderBrush() const
{
	using namespace UE::Geometry;

	if (!VertexData.IsValid())
	{
		return 0.f;
	}

	float Value = 0;
	if (ToolProperties)
	{
		switch (ToolProperties->BrushProperties.ValueQueryType)
		{

		case EMeshVertexAttributePaintToolValueQueryType::Interpolated:
		{
			const int32 Tid = GetBrushTriangleID();
			if (Tid != IndexConstants::InvalidID)
			{
				const FDynamicMesh3* Mesh = GetSculptMesh();
				const FIndex3i Vertices = Mesh->GetTriangle(Tid);
				Value = 0.0f;
				for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
				{
					const float VertexWeight = VertexData.GetValue(Vertices[TriangleVertexIndex]);
					Value += CurrentBaryCentricCoords[TriangleVertexIndex] * VertexWeight;
				}
			}
		}
		break;

		case EMeshVertexAttributePaintToolValueQueryType::NearestVertexFast:
		{
			const int32 VertexID = GetBrushNearestVertex();
			Value = VertexData.GetValue(VertexID);
		}
		break;

		case EMeshVertexAttributePaintToolValueQueryType::NearestVertexAccurate:
		{
			const int32 VertexID = GetBrushNearestVertexAccurate();
			Value = VertexData.GetValue(VertexID);
		}

		break;
		}
	}
	return Value;
}

int32 UMeshVertexAttributePaintToolBase::GetBrushNearestVertex() const
{
	int TriangleVertex = 0;

	if (CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Y && CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Z)
	{
		TriangleVertex = 0;
	}
	else
	{
		if (CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.X && CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.Z)
		{
			TriangleVertex = 1;
		}
		else
		{
			TriangleVertex = 2;
		}
	}
	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 Tid = GetBrushTriangleID();
	if (Tid == IndexConstants::InvalidID)
	{
		return IndexConstants::InvalidID;
	}

	UE::Geometry::FIndex3i Vertices = Mesh->GetTriangle(Tid);
	return Vertices[TriangleVertex];
}


int32 UMeshVertexAttributePaintToolBase::GetBrushNearestVertexAccurate() const
{
	using namespace UE::Geometry;

	int32 NearestVertexIndex = INDEX_NONE;
	const int32 Tid = GetBrushTriangleID();

	if (Tid != IndexConstants::InvalidID)
	{
		const FDynamicMesh3* const Mesh = GetSculptMesh();

		FVector3d PointOnSurface(0, 0, 0);
		const FIndex3i Vertices = Mesh->GetTriangle(Tid);
		for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
		{
			PointOnSurface += CurrentBaryCentricCoords[TriangleVertexIndex] * Mesh->GetVertex(Vertices[TriangleVertexIndex]);
		}

		double MinDist = UE_BIG_NUMBER;
		for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
		{
			const int32 VertexIndex = Vertices[TriangleVertexIndex];
			const FVector3d& VertexPosition = Mesh->GetVertex(VertexIndex);
			const double CurrDist = FVector3d::Distance(VertexPosition, PointOnSurface);
			if (CurrDist < MinDist)
			{
				MinDist = CurrDist;
				NearestVertexIndex = VertexIndex;
			}
		}
	}

	return NearestVertexIndex;
}

float UMeshVertexAttributePaintToolBase::GetAverageWeightValueFromSelection() const
{
	if (!VertexData.IsValid())
	{
		return 0.f;
	}
	return VertexData.GetAverageValue(GetSelectedVertices());
}

void UMeshVertexAttributePaintToolBase::RebuildOctree()
{
	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& InMesh)
	{
		Octree = {};
			
		// initialize dynamic octree
		if (InMesh.TriangleCount() > 100000)
		{
			Octree.RootDimension = MeshLocalBounds.MaxDim() / 10.0;
			Octree.SetMaxTreeDepth(4);
		}
		else
		{
			Octree.RootDimension = MeshLocalBounds.MaxDim();
			Octree.SetMaxTreeDepth(8);
		}
		Octree.Initialize(&InMesh);	
	});
}

void UMeshVertexAttributePaintToolBase::UpdateBrushType(EMeshVertexAttributePaintToolEditOperation BrushMode)
{
	static const FText BaseMessage = LOCTEXT("OnStartTool", "Hold [Shift] to Relax values, Hold [Ctrl] to Invert Value, [/] and S/D change Size (+Shift to small-step)");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	if (ToolProperties)
	{
		ToolProperties->BrushProperties.BrushMode = BrushMode;
	}
	// When the user picks Relax as their primary brush mode, route directly to the smooth op as
	// primary
	const int32 PrimaryBrushIdToUse =
		(BrushMode == EMeshVertexAttributePaintToolEditOperation::Relax) ? SmoothBrushId : PaintBrushId;
	SetActivePrimaryBrushType(PrimaryBrushIdToUse);
	SetActiveSecondaryBrushType(SmoothBrushId);

	SetToolPropertySourceEnabled(PaintBrushOpProperties, false);
	SetToolPropertySourceEnabled(SmoothBrushOpProperties, false);
	SetToolPropertySourceEnabled(GizmoProperties, false);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}

void UMeshVertexAttributePaintToolBase::OnColorRampChanged(TArray<FRichCurve*> Curves)
{
	if (ToolProperties)
	{
		ToolProperties->SaveConfig();
	}
	UpdatePreview();
}

void UMeshVertexAttributePaintToolBase::SetColorMode(EMeshVertexAttributePaintToolColorMode NewColorMode)
{
	if (ToolProperties)
	{
		ToolProperties->DisplayProperties.ColorMode = NewColorMode;
		ToolProperties->SaveConfig();

		// FullMaterial shows the asset's real materials with no weight visualisation.
		// Greyscale and Ramp swap to the vertex-color override material so the painted weights render.
		UpdateMaterialMode(NewColorMode == EMeshVertexAttributePaintToolColorMode::FullMaterial
			? EMeshEditingMaterialModes::ExistingMaterial
			: EMeshEditingMaterialModes::VertexColor);

		UpdatePreview();

		FPropertyChangedEvent ColorModePropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMeshVertexAttributePaintToolDisplayProperties, ColorMode)));
		ToolProperties->PostEditChangeProperty(ColorModePropertyChangedEvent);

		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::UpdatePreview(const TSet<int32>* TrianglesToUpdate, const TArray<int32>* VerticesToUpdate)
{
	UpdateVertexColorOverlay(TrianglesToUpdate);
}

void UMeshVertexAttributePaintToolBase::UpdateVertexColorOverlay(const TSet<int32>* TrianglesToUpdate)
{
	if (!VertexData.IsValid())
	{
		return;
	}

	auto ValueToColor = [this](float Value) -> FVector4f
		{
			switch (ToolProperties->DisplayProperties.ColorMode)
			{
			case EMeshVertexAttributePaintToolColorMode::Greyscale:
				return FMath::Lerp(FLinearColor::Black, FLinearColor::White, Value);
			case EMeshVertexAttributePaintToolColorMode::Ramp:
				return ToolProperties->DisplayProperties.ColorRamp.GetLinearColorValue(Value);
			case EMeshVertexAttributePaintToolColorMode::FullMaterial:
				return FLinearColor::White;
			}
			return FLinearColor::White;
		};

	auto SetColorsFromWeights = [&](FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshColorOverlay& ColorOverlay, int TriangleID)
		{
			const UE::Geometry::FIndex3i Tri = Mesh.GetTriangle(TriangleID);
			const UE::Geometry::FIndex3i ColorElementTri = ColorOverlay.GetTriangle(TriangleID);

			for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				const float Value = VertexData.GetValue(Tri[TriVertIndex]);
				ColorOverlay.SetElement(ColorElementTri[TriVertIndex], ValueToColor(Value));
			}
		};

	// update mesh with new value colors
	PreviewMesh->DeferredEditMesh(
		[this, &TrianglesToUpdate, &SetColorsFromWeights](FDynamicMesh3& Mesh)
		{
			check(Mesh.HasAttributes());
			check(Mesh.Attributes()->PrimaryColors());
			
			if (UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors())
			{
				if (TrianglesToUpdate)
				{
					for (const int TriangleID : *TrianglesToUpdate)
					{
						SetColorsFromWeights(Mesh, *ColorOverlay, TriangleID);
					}
				}
				else
				{
					for (const int TriangleID : Mesh.TriangleIndicesItr())
					{
						SetColorsFromWeights(Mesh, *ColorOverlay, TriangleID);
					}
				}
			}
		}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::VertexColors, false);
	
	GetToolManager()->PostInvalidation();
}

void UMeshVertexAttributePaintToolBase::OnMirrorCommand()
{
	if (IsInBrushMode())
	{
		EnableBrushMirroring(!IsBrushMirroringEnabled());
	}
	else
	{
		MirrorValues();
	}
}

void UMeshVertexAttributePaintToolBase::MirrorValues()
{
	if (ToolProperties)
	{
		if (!VertexData.IsValid())
		{
			return;
		}

		VertexROI = UMeshVertexAttributePaintToolBase::GetSelectedVertices();
		if (VertexROI.IsEmpty())
		{
			return;
		}

		BeginChange();

		// mirror command builds mirror data based on the rest pose of the mesh ( preview mesh maybe deformed by derived tools)
		ToolDynamicMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
		{
			MirrorData.EnsureMirrorDataIsUpdated(Mesh, ToolProperties->MirrorProperties.MirrorAxis, ToolProperties->MirrorProperties.MirrorDirection, ToolProperties->MirrorProperties.bObjectSpace);

			TArray<int32> VerticesToUpdate;
			MirrorData.FindMirroredIndices(Mesh, VertexROI, VerticesToUpdate);

			// get the selected vertices values
			ROIWeightValueBuffer.SetNum(VertexROI.Num(), EAllowShrinking::No);
			SyncWeightBufferWithMesh();

			// now swap VertexROI with VerticesToUpdate ( ROIWeightValueBuffer map 1:1 to VerticesToUpdate)
			Swap(VertexROI, VerticesToUpdate);

			// apply the values back to the mesh with the mirrored indices
			SyncMeshWithWeightBuffer();
		});

		EndChange();
		
		// update colors
		UpdatePreview();
		SetFocusInViewport();
	}
}

void UMeshVertexAttributePaintToolBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMeshVertexAttributePaintToolBase* This = CastChecked<UMeshVertexAttributePaintToolBase>(InThis);
	Collector.AddReferencedObject(This->PreviewMesh);
	Collector.AddReferencedObject(This->MeshSelector);
	Collector.AddReferencedObject(This->MeshElementsDisplay);
	// TODO: Dataflow Context should be added in Dataflow version of the paint tool
	Super::AddReferencedObjects(InThis, Collector);
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// 
// MirrorData
// 
/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool UMeshVertexAttributePaintToolBase::FMirrorData::IsPointOnTargetMirrorSide(const FVector& InPoint) const
{
	if (Axis == EAxis::None)
	{
		return false;
	}
	const int32 AxisIndex = (Axis - 1);
	if (Direction == EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative && InPoint[AxisIndex] >= PlaneOffsets[AxisIndex])
	{
		return false; // target is negative side, but point is on positive side
	}
	if (Direction == EMeshVertexAttributePaintToolMirrorDirection::NegativeToPositive && InPoint[AxisIndex] <= PlaneOffsets[AxisIndex])
	{
		return false; // target is positive side, but vertex is on negative side
	}
	return true;
}

void UMeshVertexAttributePaintToolBase::FMirrorData::EnsureMirrorDataIsUpdated(const FDynamicMesh3& Mesh, EAxis::Type InMirrorAxis, EMeshVertexAttributePaintToolMirrorDirection InMirrorDirection, bool bInObjectSpace)
{
	if (VertexMap.Num() > 0 && InMirrorAxis == Axis && InMirrorDirection == Direction && bObjectSpace == bInObjectSpace)
	{
		// already initialized, just re-use cached data
		return;
	}

	// need to re-initialize
	Axis = InMirrorAxis;
	Direction = InMirrorDirection;
	bObjectSpace = bInObjectSpace;
	VertexMap.Reset();

	const int32 NumVertices = Mesh.VertexCount();

	// build a spatial hash grid
	constexpr float HashGridCellSize = 2.0f; // the length of the cell size in the point hash grid
	UE::Geometry::TPointHashGrid3f<int32> VertHash(HashGridCellSize, INDEX_NONE);
	VertHash.Reserve(NumVertices);

	PlaneOffsets = FVector{ 0, 0, 0 };
	if (bObjectSpace)
	{
		const UE::Geometry::FAxisAlignedBox3d MeshBounds = Mesh.GetBounds(true);
		PlaneOffsets = MeshBounds.Center();
	}

	for (int32 VertexID : Mesh.VertexIndicesItr())
	{
		const FVector VertexPos = Mesh.GetVertex(VertexID);
		VertHash.InsertPointUnsafe(VertexID, static_cast<FVector3f>(VertexPos));
	}

	// generate a map of point IDs on the target side, to their equivalent vertex ID on the source side 
	for (int32 TargetVertexID : Mesh.VertexIndicesItr())
	{
		const FVector SourcePosition = Mesh.GetVertex(TargetVertexID);

		// flip position across the mirror axis
		FVector3f MirroredPosition = FVector3f(SourcePosition);
		const int32 AxisIndex = (Axis - 1);
		MirroredPosition[AxisIndex] = ((MirroredPosition[AxisIndex] - PlaneOffsets[AxisIndex]) *  (-1.0f)) + PlaneOffsets[AxisIndex];

		// Query spatial hash near mirrored position, gradually increasing search radius until at least 1 point is found
		TPair<int32, double> ClosestMirroredPoint = { INDEX_NONE, TNumericLimits<double>::Max() };
		float SearchRadius = HashGridCellSize;
		while (ClosestMirroredPoint.Key == INDEX_NONE)
		{
			ClosestMirroredPoint = VertHash.FindNearestInRadius(
				MirroredPosition,
				SearchRadius,
				[&Mesh, MirroredPosition](int32 VID)
				{
					return FVector3f::DistSquared(FVector3f(Mesh.GetVertex(VID)), MirroredPosition);
				});

			SearchRadius += HashGridCellSize;

			// forcibly break out if search radius gets bigger than the maximum search radius
			static float MaxSearchRadius = 15.f; // TODO we may want to expose this value to the user...
			if (SearchRadius >= MaxSearchRadius)
			{
				break;
			}
		}

		if (ClosestMirroredPoint.Key != INDEX_NONE)
		{
			VertexMap.FindOrAdd(TargetVertexID, ClosestMirroredPoint.Key);
		}
	}
}

void UMeshVertexAttributePaintToolBase::FMirrorData::FindMirroredIndices(const FDynamicMesh3& Mesh, const TArray<int32>& SelectedVertices, TArray<int32>& OutVerticesToUpdate)
{
	check(Axis != EAxis::None);

	// results have a 1:1 mapping to the input vertices
	OutVerticesToUpdate.SetNum(SelectedVertices.Num());

	// we need to convert selection to the equivalent target vertex indices (on the target side of the mirror plane)
	// if a vertex is already on the target side, great
	// if the user selected vertices on the source side, we convert them to the mirrored equivalent on the target side
	TSet<VertexIndex> TargetVertices;
	TArray<VertexIndex> MissingVertices;
	for (int32 Index = 0; Index < SelectedVertices.Num(); ++Index)
	{
		const int32 SelectedVertexID = SelectedVertices[Index];
		int32& TargetVertexID = OutVerticesToUpdate[Index];

		// get the mirrored index or the invaliod index if not found
		OutVerticesToUpdate[Index] = VertexMap.FindRef(SelectedVertexID, INDEX_NONE);

		// if the selected vertex is on the wrong side, reset to an invalid index
		if (Mesh.IsVertex(TargetVertexID))
		{
			const FVector TargetPosition = Mesh.GetVertex(TargetVertexID);
			if (!IsPointOnTargetMirrorSide(TargetPosition))
			{
				OutVerticesToUpdate[Index] = INDEX_NONE;
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

