// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapeSprayTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "DynamicMesh/MeshNormals.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShapeSprayTool)

using namespace UE::Geometry;

// localization namespace
#define LOCTEXT_NAMESPACE "UShapeSprayTool"

/*
 * ToolBuilder
 */


UMeshSurfacePointTool* UShapeSprayToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UShapeSprayTool* NewTool = NewObject<UShapeSprayTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}



/*
 * Tool
 */

UShapeSprayToolProperties::UShapeSprayToolProperties()
{
	DropSpeed = 0.5;
	Color = FLinearColor(0.25f, 0.08f, 0.32f);
	bRandomColor = false;
	ObjectSize = 20.0;
	NumSplats = 1;
}


UShapeSprayTool::UShapeSprayTool()
{
}


void UShapeSprayTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}


void UShapeSprayTool::Setup()
{
	UDynamicMeshBrushTool::Setup();

	// add settings object
	Settings = NewObject<UShapeSprayToolProperties>(this, TEXT("Settings"));
	AddToolPropertySource(Settings);

	UpdateShapeMesh();
	Random.Initialize(31337);

	// create dynamic mesh component to use for live preview
	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	AccumMeshComponent = NewObject<UDynamicMeshComponent>(TargetComponent->GetOwnerActor(), "SprayMesh");
		AccumMeshComponent->SetupAttachment(TargetComponent->GetOwnerActor()->GetRootComponent());
	AccumMeshComponent->RegisterComponent();

	UMaterialInterface* VtxColorMaterial = GetToolManager()->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	check(VtxColorMaterial);
	if (VtxColorMaterial)
	{
		AccumMeshComponent->SetMaterial(0, VtxColorMaterial);
		AccumMeshComponent->GetMesh()->EnableVertexColors(FVector3f::One());
		AccumMeshComponent->GetMesh()->EnableVertexNormals(FVector3f::UnitX());
	}
}



void UShapeSprayTool::Shutdown(EToolShutdownType ShutdownType)
{
	UDynamicMeshBrushTool::Shutdown(ShutdownType);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		EmitResult();
	}

	if (AccumMeshComponent != nullptr)
	{
		AccumMeshComponent->UnregisterComponent();
		AccumMeshComponent->DestroyComponent();
		AccumMeshComponent = nullptr;
	}

}


void UShapeSprayTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	UDynamicMeshBrushTool::OnPropertyModified(PropertySet, Property);
	if (PropertySet == Settings)
	{
		if (Settings->Material != nullptr && AccumMeshComponent->GetMaterial(0) != Settings->Material)
		{
			AccumMeshComponent->SetMaterial(0, Settings->Material);
		}
	}

}



void UShapeSprayTool::OnBeginDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnBeginDrag(Ray);
}

void UShapeSprayTool::OnUpdateDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnUpdateDrag(Ray);

	FFrame3f WorldFrame((FVector3f)LastBrushStamp.WorldPosition, (FVector3f)LastBrushStamp.WorldNormal);
	FTransform3d Transform(Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform());


	FDynamicMesh3* Mesh = AccumMeshComponent->GetMesh();

	float Radius = LastBrushStamp.Radius;
	float DiscardT = 0.8 + (1-Settings->DropSpeed)*0.19;

	for (int k = 0; k < Settings->NumSplats; ++k)
	{
		if (Random.GetFraction() < DiscardT)
		{
			continue;
		}

		float Angle = Random.GetFraction() * FMathf::TwoPi;
		float Rad = Random.GetFraction() * Radius;
		FVector3f PlanePt(Rad*FMathf::Cos(Angle), Rad*FMathf::Sin(Angle), 0);
		FVector3f WorldPt = WorldFrame.PointAt(PlanePt);
		FVector3f SampleRayDir = WorldPt - (FVector3f)Ray.Origin; 
		UE::Geometry::Normalize(SampleRayDir);
		FRay WorldRay((FVector)Ray.Origin, (FVector)SampleRayDir);
		FHitResult Hit;
		if (HitTest(WorldRay, Hit))
		{
			float ObjSize = (1.0f + (Random.GetFraction()-0.5f) ) *  (Settings->ObjectSize);

			FFrame3d SurfFrame(
				Transform.InverseTransformPosition((FVector3d)Hit.ImpactPoint),
				Transform.InverseTransformVector((FVector3d)Hit.Normal));
			SplatShape(SurfFrame, ObjSize, Mesh);
		}
	}

	AccumMeshComponent->NotifyMeshUpdated();
}



void UShapeSprayTool::SplatShape(const FFrame3d& LocalFrame, double Scale, FDynamicMesh3* TargetMesh)
{
	FLinearColor UseColor = (Settings->bRandomColor) ? FLinearColor::MakeRandomColor() : Settings->Color;
	FQuaternionf Rotationf = (FQuaternionf)LocalFrame.Rotation;

	VertexMap.Reset();
	VertexMap.SetNum(ShapeMesh.MaxVertexID());
	for (int vid : ShapeMesh.VertexIndicesItr())
	{
		FVector3d Pos = Scale * ShapeMesh.GetVertex(vid);
		Pos = LocalFrame.PointAt(Pos.X, Pos.Y, Pos.Z);
		VertexMap[vid] = TargetMesh->AppendVertex(Pos);

		FVector3f Normal = Rotationf * ShapeMesh.GetVertexNormal(vid);
		TargetMesh->SetVertexNormal(VertexMap[vid], Normal);

		TargetMesh->SetVertexColor(VertexMap[vid], (FVector3f)UseColor);
	}
	for (int tid : ShapeMesh.TriangleIndicesItr())
	{
		FIndex3i Tri = ShapeMesh.GetTriangle(tid);
		TargetMesh->AppendTriangle(VertexMap[Tri.A], VertexMap[Tri.B], VertexMap[Tri.C]);
	}
}


void UShapeSprayTool::OnEndDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnEndDrag(Ray);
}


void UShapeSprayTool::UpdateShapeMesh()
{
	ShapeMesh = FDynamicMesh3();
	FMinimalBoxMeshGenerator BoxGen;
	ShapeMesh.Copy(&BoxGen.Generate());

	ShapeMesh.EnableVertexNormals(FVector3f::UnitX());
	for (int vid : ShapeMesh.VertexIndicesItr())
	{
		FVector3d Pos = ShapeMesh.GetVertex(vid);
		UE::Geometry::Normalize(Pos);
		ShapeMesh.SetVertexNormal(vid, (FVector3f)Pos);
	}
}


bool UShapeSprayTool::CanAccept() const
{
	return AccumMeshComponent->GetMesh()->TriangleCount() > 0;
}






void UShapeSprayTool::EmitResult()
{
	const FDynamicMesh3* Mesh = AccumMeshComponent->GetMesh();
	FTransform3d UseTransform(Cast<IPrimitiveComponentBackedTarget>(Target)->GetOwnerActor()->GetTransform());

	GetToolManager()->BeginUndoTransaction(LOCTEXT("EmitShapeSpray", "Create ShapeSpray"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = (FTransform)UseTransform;
	NewMeshObjectParams.BaseName = TEXT("Spray");
	NewMeshObjectParams.Materials.Add( ToolSetupUtil::GetDefaultMaterial() );
	NewMeshObjectParams.SetMesh(Mesh);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();
}



#undef LOCTEXT_NAMESPACE

