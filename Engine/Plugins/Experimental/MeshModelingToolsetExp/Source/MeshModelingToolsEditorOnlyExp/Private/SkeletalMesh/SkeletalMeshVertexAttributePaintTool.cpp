// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMesh/SkeletalMeshVertexAttributePaintTool.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "ReferenceSkeleton.h"
#include "Components/DynamicMeshComponent.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "DynamicMesh/MeshNormals.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshVertexAttributePaintTool)

#define LOCTEXT_NAMESPACE "SkeletalMeshVertexAttributePaintTool"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Tool Builder
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool USkeletalMeshVertexAttributePaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	USkeletalMeshEditorContextObjectBase* EditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	if (!EditorContext)
	{
		return false;
	}
	return Super::CanBuildTool(SceneState);
}

UMeshSurfacePointTool* USkeletalMeshVertexAttributePaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	USkeletalMeshVertexAttributePaintTool* Tool = NewObject<USkeletalMeshVertexAttributePaintTool>(SceneState.ToolManager);
	Tool->SetWorld(SceneState.World);
	return Tool;
}

const FToolTargetTypeRequirements& USkeletalMeshVertexAttributePaintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements ToolRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		USceneComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass(),
	});
	return ToolRequirements;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Properties
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void USkeletalMeshVertexAttributePaintToolProperties::Initialize(const TArray<FName>& AttributeNames, bool bInitialize)
{
	Attributes.Reset(AttributeNames.Num());
	for (const FName& AttributeName : AttributeNames)
	{
		Attributes.Add(AttributeName.ToString());
	}
	if (bInitialize)
	{
		Attribute = (Attributes.Num() > 0) ? Attributes[0] : TEXT("");
	}
}

int32 USkeletalMeshVertexAttributePaintToolProperties::GetSelectedAttributeIndex()
{
	ensure(INDEX_NONE == -1);
	return Attributes.IndexOfByKey(Attribute);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Tool
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool USkeletalMeshVertexAttributePaintTool::SetupToolMesh(UE::Geometry::FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex)
{
	using namespace UE::Geometry;

	OutInitialAttributeIndex = INDEX_NONE;

	if (EditorContext.IsValid())
	{
		const TArray<int32>& IsolatedTriangles = EditorContext->GetIsolatedTriangles();
		if (!IsolatedTriangles.IsEmpty())
		{
			FullUnposedMesh = InOutToolMesh;
			IsolationSubmesh = FDynamicSubmesh3(&FullUnposedMesh.GetValue(), IsolatedTriangles);
			InOutToolMesh = IsolationSubmesh->GetSubmesh();
		}
	}

	FDynamicMeshAttributeSet* Attributes = InOutToolMesh.Attributes();
	if (!Attributes || Attributes->NumWeightLayers() == 0)
	{
		return false;
	}

	TArray<FName> AttributeNames;
	const int32 NumLayers = Attributes->NumWeightLayers();
	AttributeNames.Reserve(NumLayers);
	for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		AttributeNames.Add(Attributes->GetWeightLayer(LayerIdx)->GetName());
	}

	constexpr bool bSelectInitialSelection = true;
	AttribProps->Initialize(AttributeNames, bSelectInitialSelection);

	OutInitialAttributeIndex = FMath::Max(0, AttribProps->GetSelectedAttributeIndex());
	return true;
}

void USkeletalMeshVertexAttributePaintTool::CommitToolMesh(UE::Geometry::FDynamicMesh3& InToolMesh)
{
	using namespace UE::Geometry;

	UInteractiveToolManager* ToolManager = GetToolManager();
	if (!ToolManager)
	{
		return;
	}

	ToolManager->BeginUndoTransaction(LOCTEXT("SkeletalMeshVertexAttributePaintTool_TransactionName", "Paint Maps"));

	// Force a full dyna-mesh update so weight-layer edits are converted back into the mesh description.
	constexpr bool bForceModifiedTopology = true;

	if (!IsolationSubmesh.IsSet())
	{
		// No isolation: ToolDynamicMesh is the unposed full mesh with painted weights — commit directly.
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, InToolMesh, bForceModifiedTopology);
	}
	else
	{
		// Copy painted weight values from submesh layers back to full-mesh layers by index.
		// Layer count and order are 1:1 because this tool never adds new layers.
		FDynamicMesh3& FullMesh = FullUnposedMesh.GetValue();
		const FDynamicMeshAttributeSet* InAttrs = InToolMesh.Attributes();
		FDynamicMeshAttributeSet* OutAttrs = FullMesh.Attributes();
		const int32 NumLayers = (InAttrs && OutAttrs) ? InAttrs->NumWeightLayers() : 0;
		check(!OutAttrs || OutAttrs->NumWeightLayers() == NumLayers);

		for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
		{
			const FDynamicMeshWeightAttribute* SourceLayer = InAttrs->GetWeightLayer(LayerIdx);
			FDynamicMeshWeightAttribute* TargetLayer = OutAttrs->GetWeightLayer(LayerIdx);

			ParallelFor(InToolMesh.MaxVertexID(), [this, &InToolMesh, &FullMesh, SourceLayer, TargetLayer](int32 SubVID)
			{
				if (!InToolMesh.IsVertex(SubVID))
				{
					return;
				}
				const int32 FullVID = IsolationSubmesh->MapVertexToBaseMesh(SubVID);
				if (FullVID != FDynamicMesh3::InvalidID && FullMesh.IsVertex(FullVID))
				{
					float Weight = 0.0f;
					SourceLayer->GetValue(SubVID, &Weight);
					TargetLayer->SetValue(FullVID, &Weight);
				}
			});
		}

		UE::ToolTarget::CommitDynamicMeshUpdate(Target, FullMesh, bForceModifiedTopology);
	}

	ToolManager->EndUndoTransaction();
}

void USkeletalMeshVertexAttributePaintTool::UpdatePreview(const TSet<int32>* TrianglesToUpdate, const TArray<int32>* VerticesToUpdate)
{
	Super::UpdatePreview(TrianglesToUpdate, VerticesToUpdate);

	const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
	const TMap<FName, float> MorphTargetWeights = GetMorphTargetWeights();
	PosePreviewMesh(ComponentSpaceTransforms, MorphTargetWeights, TrianglesToUpdate, VerticesToUpdate);

	if (TrianglesToUpdate)
	{
		Octree.ReinsertTriangles(*TrianglesToUpdate);
	}
	else
	{
		bRebuildOctree = true;
	}

	bMeshSelectorNeedsUpdate = true;
}

void USkeletalMeshVertexAttributePaintTool::Setup()
{
	EditorContext = GetToolManager()->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();

	AttribProps = NewObject<USkeletalMeshVertexAttributePaintToolProperties>(this);
	AttribProps->RestoreProperties(this);
	AddToolPropertySource(AttribProps);
	AttribProps->SetFlags(RF_Transactional);

	const FReferenceSkeleton& RefSkeleton = CastChecked<ISkeletonProvider>(GetTarget())->GetSkeleton();
	RefSkeleton.GetBoneAbsoluteTransforms(ComponentSpaceTransformsRefPose);

	ToggleBoneManipulation(true);

	PoseChangeDetector.GetNotifier().AddUObject(this, &USkeletalMeshVertexAttributePaintTool::HandlePoseChangeDetectorEvent);

	Super::Setup();

	SelectedAttributeWatcher.Initialize(
		[this]() { return AttribProps->GetSelectedAttributeIndex(); },
		[this](int32 NewAttributeIndex) { SetAttributeToPaint(NewAttributeIndex); },
		AttribProps->GetSelectedAttributeIndex());

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(GetSculptMeshComponent()));
}

void USkeletalMeshVertexAttributePaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	Super::Shutdown(ShutdownType);

	if (AttribProps && ShutdownType != EToolShutdownType::Cancel)
	{
		AttribProps->SaveProperties(this);
	}

	ToggleBoneManipulation(false);
}

void USkeletalMeshVertexAttributePaintTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (!InStroke())
	{
		SelectedAttributeWatcher.CheckAndUpdate();

		const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
		const TMap<FName, float> MorphTargetWeights = GetMorphTargetWeights();

		PoseChangeDetector.CheckPose(ComponentSpaceTransforms, MorphTargetWeights);

		if (bFastDeformPreviewMesh)
		{
			bFastDeformPreviewMesh = false;
			PosePreviewMesh(ComponentSpaceTransforms, MorphTargetWeights);
		}

		if (GetEditingMode() == EMeshVertexAttributePaintToolEditMode::Mesh && bMeshSelectorNeedsUpdate)
		{
			bMeshSelectorNeedsUpdate = false;
			MeshSelector->UpdateAfterMeshDeformation();
		}

		if (bRebuildOctree)
		{
			bRebuildOctree = false;
			RebuildOctree();
		}
	}
}

bool USkeletalMeshVertexAttributePaintTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

void USkeletalMeshVertexAttributePaintTool::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	OutHints.Add({ LOCTEXT("HintBrushSize", "Brush Size"), LOCTEXT("HintBrushSizeChord", "B + drag left/right") });
	OutHints.Add({ LOCTEXT("HintAttribValue", "Attribute Value"), LOCTEXT("HintAttribValueChord", "B + drag up/down") });
}

const TArray<FTransform>& USkeletalMeshVertexAttributePaintTool::GetComponentSpaceBoneTransforms()
{
	// Brush mirroring only suppresses pose while painting in brush mode; in mesh-selection mode
	// the mirror plane is irrelevant and the user expects pose to follow the editor context.
	if (IsBrushMirroringEnabled() && IsInBrushMode())
	{
		return ComponentSpaceTransformsRefPose;
	}

	if (EditorContext.IsValid())
	{
		return EditorContext->GetComponentSpaceBoneTransforms(GetTarget());
	}

	if (DefaultComponentSpaceBoneTransforms.IsEmpty())
	{
		ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(GetTarget());
		SkeletonProvider->GetSkeleton().GetBoneAbsoluteTransforms(DefaultComponentSpaceBoneTransforms);
	}

	return DefaultComponentSpaceBoneTransforms;
}

TMap<FName, float> USkeletalMeshVertexAttributePaintTool::GetMorphTargetWeights()
{
	if (IsBrushMirroringEnabled() && IsInBrushMode())
	{
		return {};
	}

	if (EditorContext.IsValid())
	{
		return EditorContext->GetMorphTargetWeights();
	}

	return {};
}

void USkeletalMeshVertexAttributePaintTool::ToggleBoneManipulation(bool bEnable)
{
	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(bEnable);
	}
}

void USkeletalMeshVertexAttributePaintTool::HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload)
{
	if (InStroke())
	{
		return;
	}

	using namespace SkeletalMeshToolsHelper;
	if (Payload.CurrentState == FPoseChangeDetector::PoseStoppedChanging)
	{
		bRebuildOctree = true;
	}
	else if (Payload.CurrentState == FPoseChangeDetector::PoseJustChanged ||
		Payload.CurrentState == FPoseChangeDetector::PoseChanged)
	{
		bFastDeformPreviewMesh = true;
		bMeshSelectorNeedsUpdate = true;
	}
}

void USkeletalMeshVertexAttributePaintTool::PosePreviewMesh(
	const TArray<FTransform>& ComponentSpaceTransforms,
	const TMap<FName, float>& MorphTargetWeights,
	const TSet<int32>* TrianglesToUpdate,
	const TArray<int32>* VerticesToUpdate)
{
	using namespace UE::Geometry;
	using namespace SkeletalMeshToolsHelper;

	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(ComponentSpaceTransformsRefPose, ComponentSpaceTransforms);

	FDynamicMesh3& SculptMesh = *GetSculptMesh();

	// GetPosedMesh operates on CommitMesh (full mesh), so when isolated we must hand it full-mesh vertex IDs.
	TArray<int32> FullMeshVertArrayForIsolation;
	if (IsolationSubmesh)
	{
		if (VerticesToUpdate)
		{
			FullMeshVertArrayForIsolation.Reserve(VerticesToUpdate->Num());
			for (int32 SubVID : *VerticesToUpdate)
			{
				FullMeshVertArrayForIsolation.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
			}
		}
		else
		{
			FullMeshVertArrayForIsolation.Reserve(IsolationSubmesh->GetSubmesh().VertexCount());
			for (int32 SubVID : IsolationSubmesh->GetSubmesh().VertexIndicesItr())
			{
				FullMeshVertArrayForIsolation.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
			}
		}
	}

	const TArray<int32>* FinalFullMeshVertArrayPtr = IsolationSubmesh ? &FullMeshVertArrayForIsolation : VerticesToUpdate;

	auto WriteFunc = [&](FVertInfo VertInfo, const FVector& PosedVertPos)
	{
		const int32 SculptVID = IsolationSubmesh ? IsolationSubmesh->MapVertexToSubmesh(VertInfo.VertID) : VertInfo.VertID;
		if (SculptVID != FDynamicMesh3::InvalidID)
		{
			SculptMesh.SetVertex(SculptVID, PosedVertPos);
		}
	};

	const TArray<int32>& VertArray = FinalFullMeshVertArrayPtr ? *FinalFullMeshVertArrayPtr : TArray<int32>{};
	if (IsolationSubmesh.IsSet())
	{
		GetPosedMesh(WriteFunc, FullUnposedMesh.GetValue(), BoneMatrices, NAME_None, MorphTargetWeights, VertArray);
	}
	else
	{
		// Without isolation, ToolDynamicMesh holds the unposed full mesh — read it directly.
		ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& UnposedSource)
		{
			GetPosedMesh(WriteFunc, UnposedSource, BoneMatrices, NAME_None, MorphTargetWeights, VertArray);
		});
	}

	if (!TrianglesToUpdate)
	{
		FMeshNormals::QuickRecomputeOverlayNormals(SculptMesh);
	}
	else
	{
		FMeshNormals::RecomputeOverlayTriNormals(SculptMesh, TrianglesToUpdate->Array());
	}

	constexpr bool bNormal = true;
	CastChecked<UDynamicMeshComponent>(GetSculptMeshComponent())->FastNotifyPositionsUpdated(bNormal);

	MeshElementsDisplay->NotifyMeshChanged();
}

#undef LOCTEXT_NAMESPACE
