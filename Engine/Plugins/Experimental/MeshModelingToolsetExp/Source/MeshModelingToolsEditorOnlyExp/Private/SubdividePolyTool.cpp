// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubdividePolyTool.h"
#include "ToolBuilderUtil.h"
#include "InteractiveToolManager.h"
#include "GroupTopology.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "ToolSetupUtil.h"
#include "Util/ColorConstants.h"
#include "DynamicMesh/MeshNormals.h"
#include "Components/DynamicMeshComponent.h"
#include "Drawing/PreviewGeometryActor.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubdividePolyTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USubdividePolyTool"

class SubdivPostProcessor : public IRenderMeshPostProcessor
{
public:

	SubdivPostProcessor(int InSubdivisionLevel,
						ESubdivisionScheme InSubdivisionScheme,
						ESubdivisionBoundaryScheme InBoundaryScheme,
						ESubdivisionOutputNormals InNormalComputationMethod,
						ESubdivisionOutputUVs InUVComputationMethod,
						bool bInNewPolyGroups,
						TFunction<bool(const FGroupTopology& GroupTopology, 
							int32 Vid, const FIndex2i& AttachedGroupEdgeEids)> ShouldAddExtraCornerAtVertIn) :
		SubdivisionLevel(InSubdivisionLevel),
		SubdivisionScheme(InSubdivisionScheme),
		BoundaryScheme(InBoundaryScheme),
		NormalComputationMethod(InNormalComputationMethod),
		UVComputationMethod(InUVComputationMethod),
		bNewPolyGroups(bInNewPolyGroups),
		ShouldAddExtraCornerAtVert(ShouldAddExtraCornerAtVertIn)
	{}

	int SubdivisionLevel = 3;
	ESubdivisionScheme SubdivisionScheme = ESubdivisionScheme::CatmullClark;
	ESubdivisionBoundaryScheme BoundaryScheme = ESubdivisionBoundaryScheme::SmoothCorners;
	ESubdivisionOutputNormals NormalComputationMethod = ESubdivisionOutputNormals::Generated;
	ESubdivisionOutputUVs UVComputationMethod = ESubdivisionOutputUVs::Interpolated;
	bool bNewPolyGroups = false;
	// Function passed to the group topology to add extra corners
	TFunction<bool(const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)> ShouldAddExtraCornerAtVert;

	void ProcessMesh(const FDynamicMesh3& Mesh, FDynamicMesh3& OutRenderMesh) final
	{
		if (Mesh.TriangleCount() == 0 || Mesh.VertexCount() == 0)
		{
			return;
		}

		if (SubdivisionLevel == 0)
		{
			return;
		}

		constexpr bool bBuildTopologyImmediately = false; // need to attach extra corner function first
		FGroupTopology Topo(&Mesh, bBuildTopologyImmediately);
		Topo.ShouldAddExtraCornerAtVert = ShouldAddExtraCornerAtVert;
		Topo.RebuildTopology();

		FSubdividePoly Subd(Topo, Mesh, SubdivisionLevel);
		Subd.SubdivisionScheme = SubdivisionScheme;
		Subd.BoundaryScheme = BoundaryScheme;
		Subd.NormalComputationMethod = NormalComputationMethod;
		Subd.UVComputationMethod = UVComputationMethod;
		Subd.bNewPolyGroups = bNewPolyGroups;

		ensure(Subd.ComputeTopologySubdivision());

		ensure(Subd.ComputeSubdividedMesh(OutRenderMesh));
	}
};


// Tool builder

USingleSelectionMeshEditingTool* USubdividePolyToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USubdividePolyTool>(SceneState.ToolManager);
}


bool USubdividePolyTool::CheckGroupTopology(FText& Message)
{
	Message = FText();

	if (!ensure(Topology.IsValid()))
	{
		return false;
	}

	FSubdividePoly TempSubD(*Topology, *OriginalMesh, 1);
	TempSubD.BoundaryScheme = Properties->BoundaryScheme; // will only matter here if we someday support NoBoundaryFaces
	FSubdividePoly::ETopologyCheckResult CheckResult = TempSubD.ValidateTopology();

	if (CheckResult == FSubdividePoly::ETopologyCheckResult::NoGroups)
	{
		Message = LOCTEXT("NoGroupsWarning",
						  "This object has no PolyGroups.\nTool can only use Loop subdivision scheme.\nUse the PolyGroups or Select Tool to assign PolyGroups.");
		return false;
	}

	if (CheckResult == FSubdividePoly::ETopologyCheckResult::InsufficientGroups)
	{
		Message = LOCTEXT("SingleGroupsWarning",
						  "This object has only one PolyGroup.\nTool can only use Loop subdivision scheme.\nUse the PolyGroups or Select Tool to assign PolyGroups.");
		return false;
	}

	if (CheckResult == FSubdividePoly::ETopologyCheckResult::UnboundedPolygroup)
	{
		Message = LOCTEXT("NoGroupBoundaryWarning",
			"Found a PolyGroup with no boundaries.\nTool can only use Loop subdivision scheme.\nUse the PolyGroups or Select Tool to assign PolyGroups.");
		return false;
	}

	if (CheckResult == FSubdividePoly::ETopologyCheckResult::MultiBoundaryPolygroup)
	{
		Message = LOCTEXT("MultipleGroupBoundaryWarning",
			"Found a PolyGroup with multiple boundaries, which is not supported.\nTool can only use Loop subdivision scheme.\nUse the PolyGroups or Select Tool to assign PolyGroups.");
		return false;
	}

	if (CheckResult == FSubdividePoly::ETopologyCheckResult::DegeneratePolygroup)
	{
		Message = LOCTEXT("DegenerateGroupPolygon",
			"One PolyGroup has fewer than three boundary edges.\nTool can only use Loop subdivision scheme.\n"
			"Use the PolyGroups or Select Tool to assign/fix PolyGroups, or use the \"Extra Corner\" settings to break up edges.");
		return false;
	}

	return true;
}


void USubdividePolyTool::CapSubdivisionLevel(ESubdivisionScheme Scheme, int DesiredLevel)
{
	// Stolen from UDisplaceMeshTool::ValidateSubdivisions
	constexpr int MaxFaces = 3000000;

	int NumOriginalFaces = MaxFaces;
	if (Scheme == ESubdivisionScheme::Loop)
	{
		NumOriginalFaces = OriginalMesh->TriangleCount();
	}
	else
	{
		NumOriginalFaces = Topology->Groups.Num();
	}
	int MaxLevel = (int)floor(log2(MaxFaces / (NumOriginalFaces+1)) / 2.0);

	CappedSubdivisionMessage = FText();
	if (DesiredLevel > MaxLevel)
	{
		CappedSubdivisionMessage = FText::Format(LOCTEXT("SubdivisionLevelTooHigh", "Subdivision level clamped: desired subdivision level ({0}) exceeds maximum level ({1}) for a mesh with this number of faces."),
										  FText::AsNumber(DesiredLevel),
										  FText::AsNumber(MaxLevel));
		

		Properties->SubdivisionLevel = MaxLevel;
		Properties->SilentUpdateWatched();		// Don't trigger this function again due to setting SubdivisionLevel above
	}
}

ESubdivisionScheme USubdividePolyTool::GetSubdivisionSchemeToUse()
{
	return Properties->bOverriddenSubdivisionScheme ? ESubdivisionScheme::Loop : Properties->SubdivisionScheme;
}

void USubdividePolyTool::UpdateDisplayedMessage()
{
	FText Message;
	if (Properties->bOverriddenSubdivisionScheme && Properties->SubdivisionScheme != ESubdivisionScheme::Loop)
	{
		Message = OverriddenSchemeMessage;
	}

	FText Delimiter = FText::FromString("\n");
	if (!CappedSubdivisionMessage.IsEmpty())
	{
		Message = FText::Join(Delimiter, Message, CappedSubdivisionMessage);
	}

	GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
}


void USubdividePolyTool::Setup()
{
	UInteractiveTool::Setup();
	SetToolDisplayName(LOCTEXT("ToolName", "Subdivide"));

	if (!Target)
	{
		return;
	}

	GetToolManager()->DisplayMessage(LOCTEXT("SubdividePolyToolMessage", "Set the subdivision level and hit Accept to create a new subdivided mesh"), EToolMessageLevel::UserNotification);

	Properties = NewObject<USubdividePolyToolProperties>(this, TEXT("Subdivide Mesh Tool Settings"));
	Properties->RestoreProperties(this);

	bool bWantVertexNormals = false;
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(bWantVertexNormals, false, false, false);
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

	Topology = MakeShared<FGroupTopology>(OriginalMesh.Get(), false);
	auto ShouldAddExtraCornerAtVert = [this](const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)
	{
		return Properties->bAddExtraCorners && GroupTopology.GetMesh()->IsBoundaryVertex(Vid) && FGroupTopology::IsEdgeAngleSharp(GroupTopology.GetMesh(), Vid, AttachedGroupEdgeEids, ExtraCornerDotProductThreshold);
	};
	Topology->ShouldAddExtraCornerAtVert = ShouldAddExtraCornerAtVert;

	auto UpdateExtraCornerThreshold = [this]() { ExtraCornerDotProductThreshold = FMathd::Cos(Properties->ExtraCornerAngleThresholdDegrees * FMathd::DegToRad); };
	UpdateExtraCornerThreshold();

	Topology->RebuildTopology();

	Properties->bOverriddenSubdivisionScheme = !CheckGroupTopology(OverriddenSchemeMessage);

	if (Properties->bOverriddenSubdivisionScheme)
	{
		Properties->SubdivisionScheme = ESubdivisionScheme::Loop;
	}

	AddToolPropertySource(Properties);
	SetToolPropertySourceEnabled(Properties, true);

	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	PreviewMesh = NewObject<UPreviewMesh>(this);
	if (PreviewMesh == nullptr)
	{
		return;
	}
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);

	PreviewMesh->SetTransform(TargetComponent->GetWorldTransform());
	PreviewMesh->UpdatePreview(OriginalMesh.Get());

	UDynamicMeshComponent* PreviewDynamicMeshComponent = (UDynamicMeshComponent*)PreviewMesh->GetRootComponent();
	if (PreviewDynamicMeshComponent == nullptr)
	{
		return;
	}

	if (Properties->SubdivisionLevel < 1)
	{
		Properties->SubdivisionLevel = 1;
	}
	
	CapSubdivisionLevel(GetSubdivisionSchemeToUse(), Properties->SubdivisionLevel);

	// Use the input mesh's material on the preview
	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		PreviewMesh->SetMaterial(k, MaterialSet.Materials[k]);
	}

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.8f, 0.75f, 0.0f), GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	// dynamic mesh configuration settings
	auto RebuildMeshPostProcessor = [this, ShouldAddExtraCornerAtVert]()
	{
		UDynamicMeshComponent* PreviewDynamicMeshComponent = (UDynamicMeshComponent*)PreviewMesh->GetRootComponent();
		PreviewDynamicMeshComponent->SetRenderMeshPostProcessor(MakeUnique<SubdivPostProcessor>(
			Properties->SubdivisionLevel,
			GetSubdivisionSchemeToUse(),
			Properties->BoundaryScheme,
			Properties->NormalComputationMethod,
			Properties->UVComputationMethod,
			Properties->bNewPolyGroups,
			ShouldAddExtraCornerAtVert));
		PreviewDynamicMeshComponent->NotifyMeshUpdated();
	};
	RebuildMeshPostProcessor();

	// Watch for property changes
	Properties->WatchProperty(Properties->SubdivisionLevel, [this, RebuildMeshPostProcessor](int NewSubdLevel)
	{
		CapSubdivisionLevel(GetSubdivisionSchemeToUse(), NewSubdLevel);
		UpdateDisplayedMessage();
		RebuildMeshPostProcessor();
	});

	Properties->WatchProperty(Properties->SubdivisionScheme, [this, RebuildMeshPostProcessor](ESubdivisionScheme NewScheme)
	{
		CapSubdivisionLevel(GetSubdivisionSchemeToUse(), Properties->SubdivisionLevel);
		UpdateDisplayedMessage();
		RebuildMeshPostProcessor();
		bPreviewGeometryNeedsUpdate = true;
	});

	Properties->WatchProperty(Properties->NormalComputationMethod, [this, RebuildMeshPostProcessor](ESubdivisionOutputNormals)
	{
		RebuildMeshPostProcessor();
	});
	Properties->WatchProperty(Properties->UVComputationMethod, [this, RebuildMeshPostProcessor](ESubdivisionOutputUVs)
	{
		RebuildMeshPostProcessor();
	});
	Properties->WatchProperty(Properties->bNewPolyGroups, [this, RebuildMeshPostProcessor](bool)
	{
		RebuildMeshPostProcessor();
	});
	Properties->WatchProperty(Properties->BoundaryScheme, [this, RebuildMeshPostProcessor](ESubdivisionBoundaryScheme)
	{
		RebuildMeshPostProcessor();
	});

	auto OnCornerChange = [this, UpdateExtraCornerThreshold, RebuildMeshPostProcessor]() {
		UpdateExtraCornerThreshold();
		Topology->RebuildTopology();

		Properties->bOverriddenSubdivisionScheme = !CheckGroupTopology(OverriddenSchemeMessage);
		UpdateDisplayedMessage();
		NotifyOfPropertyChangeByTool(Properties);

		bPreviewGeometryNeedsUpdate = true;
		RebuildMeshPostProcessor();
	};
	Properties->WatchProperty(Properties->bAddExtraCorners, [this, OnCornerChange](bool) {
		OnCornerChange();
	});
	Properties->WatchProperty(Properties->ExtraCornerAngleThresholdDegrees, [this, OnCornerChange](double) {
		OnCornerChange();
	});

	auto RenderGroupsChanged = [this](bool bNewRenderGroups)
	{
		if (bNewRenderGroups)
		{
			PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetVertexColorMaterial(GetToolManager()));
			PreviewMesh->SetTriangleColorFunction([](const FDynamicMesh3* Mesh, int TriangleID)
			{
				return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
			}, UPreviewMesh::ERenderUpdateMode::FullUpdate);
		}
		else
		{
			PreviewMesh->SetOverrideRenderMaterial(nullptr);
			PreviewMesh->SetTriangleColorFunction(nullptr, UPreviewMesh::ERenderUpdateMode::FullUpdate);
		}
	};

	Properties->WatchProperty(Properties->bRenderGroups, RenderGroupsChanged);

	// Render with polygroup colors
	RenderGroupsChanged(Properties->bRenderGroups);

	Properties->WatchProperty(Properties->bRenderCage, [this](bool bNewRenderCage)
	{
		bPreviewGeometryNeedsUpdate = true;
	});

	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(TargetComponent->GetOwnerActor()->GetWorld(), TargetComponent->GetWorldTransform());
	CreateOrUpdatePreviewGeometry();

	// regenerate preview geo if mesh changes due to undo/redo/etc
	PreviewDynamicMeshComponent->OnMeshChanged.AddLambda([this]() { bPreviewGeometryNeedsUpdate = true; });

	TargetComponent->SetOwnerVisibility(false);
	PreviewMesh->SetVisible(true);

	Properties->SilentUpdateWatched();
	UpdateDisplayedMessage();
}

void USubdividePolyTool::CreateOrUpdatePreviewGeometry()
{
	if (!Properties->bRenderCage)
	{
		PreviewGeometry->RemoveLineSet(TEXT("TopologyEdges"));
		PreviewGeometry->RemoveLineSet(TEXT("AllEdges"));
		return;
	}

	if (GetSubdivisionSchemeToUse() == ESubdivisionScheme::Loop)
	{
		int NumEdges = OriginalMesh->EdgeCount();

		PreviewGeometry->RemoveLineSet(TEXT("TopologyEdges"));

		PreviewGeometry->CreateOrUpdateLineSet(TEXT("AllEdges"),
											   NumEdges,
											   [this](int32 Index, TArray<FRenderableLine>& LinesOut)
		{
			FIndex2i EdgeVertices = OriginalMesh->GetEdgeV(Index);

			if (EdgeVertices[0] == FDynamicMesh3::InvalidID || EdgeVertices[1] == FDynamicMesh3::InvalidID)
			{
				return;
			}
			FVector A = (FVector)OriginalMesh->GetVertex(EdgeVertices[0]);
			FVector B = (FVector)OriginalMesh->GetVertex(EdgeVertices[1]);
			const float TopologyLineThickness = 4.0f;
			const FColor TopologyLineColor(255, 0, 0);
			LinesOut.Add(FRenderableLine(A, B, TopologyLineColor, TopologyLineThickness));
		});
	}
	else
	{
		int NumEdges = Topology->Edges.Num();

		PreviewGeometry->RemoveLineSet(TEXT("AllEdges"));

		PreviewGeometry->CreateOrUpdateLineSet(TEXT("TopologyEdges"),
											   NumEdges,
											   [this](int32 Index, TArray<FRenderableLine>& LinesOut)
		{
			const FGroupTopology::FGroupEdge& Edge = Topology->Edges[Index];
			FIndex2i EdgeCorners = Edge.EndpointCorners;

			if (EdgeCorners[0] == FDynamicMesh3::InvalidID || EdgeCorners[1] == FDynamicMesh3::InvalidID)
			{
				return;
			}

			FIndex2i EdgeVertices{ Topology->Corners[EdgeCorners[0]].VertexID,
									Topology->Corners[EdgeCorners[1]].VertexID };
			FVector A = (FVector)OriginalMesh->GetVertex(EdgeVertices[0]);
			FVector B = (FVector)OriginalMesh->GetVertex(EdgeVertices[1]);

			const float TopologyLineThickness = 4.0f;
			const FColor TopologyLineColor(255, 0, 0);
			LinesOut.Add(FRenderableLine(A, B, TopologyLineColor, TopologyLineThickness));
		});
	}
}


void USubdividePolyTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (Properties)
	{
		Properties->SaveProperties(this);
	}

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
	}
	
	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	if (PreviewMesh)
	{
		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("USubdividePolyTool", "Subdivide Mesh"));

			UDynamicMeshComponent* PreviewDynamicMeshComponent = (UDynamicMeshComponent*)PreviewMesh->GetRootComponent();
			FDynamicMesh3* DynamicMeshResult = PreviewDynamicMeshComponent->GetRenderMesh();

			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *DynamicMeshResult, true);

			GetToolManager()->EndUndoTransaction();
		}
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}
}

bool USubdividePolyTool::CanAccept() const
{
	return PreviewMesh != nullptr;
}


void USubdividePolyTool::OnTick(float DeltaTime)
{
	if (bPreviewGeometryNeedsUpdate)
	{
		CreateOrUpdatePreviewGeometry();
		bPreviewGeometryNeedsUpdate = false;
	}
}


#undef LOCTEXT_NAMESPACE

