// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoxelCSGMeshesTool.h"

#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "Selection/ToolSelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"



#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "CompositionOps/VoxelBooleanMeshesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoxelCSGMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UVoxelCSGMeshesTool"


void UVoxelCSGMeshesTool::SetupProperties()
{
	//Super::SetupProperties();
	//RemoveToolPropertySource(VoxProperties);

	UBaseCreateFromSelectedTool::SetupProperties();

	CSGProps = NewObject<UVoxelCSGMeshesToolProperties>(this);
	CSGProps->RestoreProperties(this);
	AddToolPropertySource(CSGProps);

	SetToolDisplayName(LOCTEXT("UVoxelCSGMeshesToolName", "Voxel Boolean"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Compute a CSG Boolean of the input meshes using voxelization techniques.UVs, sharp edges, and small / thin features will be lost.Increase Voxel Count to enhance accuracy."),
		EToolMessageLevel::UserNotification);

}


void UVoxelCSGMeshesTool::SaveProperties()
{
	//Super::SaveProperties();
	UBaseCreateFromSelectedTool::SaveProperties();
	CSGProps->SaveProperties(this);
}


void UVoxelCSGMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	Super::ConvertInputsAndSetPreviewMaterials(bSetPreviewMesh);

	if (HasOpenBoundariesInMeshInputs())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("WarnOpenEdges", "Open edges found: some artifacts will result."),
			EToolMessageLevel::UserWarning);
	}
}

TUniquePtr<FDynamicMeshOperator> UVoxelCSGMeshesTool::MakeNewOperator()
{
	TUniquePtr<FVoxelBooleanMeshesOp> CSGOp = MakeUnique<FVoxelBooleanMeshesOp>();
	CSGOp->Operation = (FVoxelBooleanMeshesOp::EBooleanOperation)(int)CSGProps->Operation;
	CSGOp->VoxelCount = CSGProps->VoxelCount;
	CSGOp->AdaptivityD = CSGProps->MeshAdaptivity;
	CSGOp->IsoSurfaceD = CSGProps->OffsetDistance;
	CSGOp->bAutoSimplify = CSGProps->bAutoSimplify;
	CSGOp->Transforms.SetNum(Targets.Num());
	CSGOp->Meshes.SetNum(Targets.Num());
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		CSGOp->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		CSGOp->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}
	return CSGOp;
}



FString UVoxelCSGMeshesTool::GetCreatedAssetName() const
{
	return TEXT("VoxelBoolean");
}

FText UVoxelCSGMeshesTool::GetActionName() const
{
	return LOCTEXT("VoxelCSGMeshes", "Voxel Boolean");
}



#undef LOCTEXT_NAMESPACE

