// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveToolBuilder.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "GameFramework/Actor.h"
#include "Changes/ValueWatcher.h"

#include "AddPivotActorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class UDragAlignmentMechanic;
class UCombinedTransformGizmo;
class UTransformProxy;


UCLASS(MinimalAPI)
class UPivotActorTransformProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = PivotLocation)
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = PivotLocation)
	FQuat Rotation = FQuat::Identity;
};


UCLASS(MinimalAPI)
class UAddPivotActorToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	
	UE_API void InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/** 
 * Given selected actors, creates an empty actor as the parent of those actors, at a location
 * specified using the gizmo. This is useful for creating a permanent alternate pivot to use in
 * animation.
 */
UCLASS(MinimalAPI)
class UAddPivotActorTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()
public:

	virtual void SetPivotRepositionMode(AActor* PivotActor) { ExistingPivotActor = PivotActor; }

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasAccept() const override { return true; }
	virtual bool HasCancel() const override { return true; }

	UE_API virtual void OnTick(float DeltaTime) override;

	// Uses the base class CanAccept

protected:

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UPivotActorTransformProperties> TransformProperties;

	TWeakObjectPtr<AActor> ExistingPivotActor = nullptr;

	TValueWatcher<FVector> GizmoPositionWatcher;
	TValueWatcher<FQuat> GizmoRotationWatcher;

	UE_API void UpdateGizmoFromProperties();
	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);

	FTransform ExistingPivotOriginalTransform;
};

#undef UE_API
