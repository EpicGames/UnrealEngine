// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "BakeMeshAttributeToolCommon.h"
#include "Baking/BakingTypes.h"
#include "Baking/RenderCaptureFunctions.h"

// Render Capture algorithm includes
#include "Scene/SceneCapturePhotoSet.h"

#include "BakeRenderCaptureTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class UTexture2D;

namespace UE
{
namespace Geometry
{

class FSceneCapturePhotoSet;

}
}

//
// Tool Result
//


UCLASS(MinimalAPI)
class UBakeRenderCaptureResults  : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> BaseColorMap = nullptr;

	/** World space normal map */
	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> NormalMap = nullptr;

	/** Packed Metallic/Roughness/Specular Map */
	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty, DisplayName = "Packed MRS Map"))
	TObjectPtr<UTexture2D> PackedMRSMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> MetallicMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> RoughnessMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> SpecularMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> EmissiveMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> OpacityMap = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Results, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> SubsurfaceColorMap = nullptr;

	/** Device depth is currently unused */
	UPROPERTY()
	TObjectPtr<UTexture2D> DeviceDepthMap = nullptr;

	/** Return the texture corresponding to the given CaptureType */
	UE_API const TObjectPtr<UTexture2D>& operator[](UE::Geometry::ERenderCaptureType CaptureType) const;

	UE_API bool IsEmpty() const;
};


//
// Tool Builder
//



UCLASS(MinimalAPI)
class UBakeRenderCaptureToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



//
// Tool Properties
//


UCLASS(MinimalAPI)
class URenderCaptureProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault, ClampMin = "1", UIMin= "1"), DisplayName="Render Capture Resolution")
	EBakeTextureResolution Resolution = EBakeTextureResolution::Resolution512;

	// Whether to generate a texture for the Base Color property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault))
	bool bBaseColorMap = false;

	// Whether to generate a texture for the World Normal property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault))
	bool bNormalMap = false;

	// Whether to generate a packed texture with Metallic, Roughness and Specular properties
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault), DisplayName="Packed MRS Map")
	bool bPackedMRSMap = false;

	// Whether to generate a texture for the Metallic property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault, EditCondition="bPackedMRSMap == false"))
	bool bMetallicMap = false;

	// Whether to generate a texture for the Roughness property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault, EditCondition="bPackedMRSMap == false"))
	bool bRoughnessMap = false;

	// Whether to generate a texture for the Specular property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault, EditCondition="bPackedMRSMap == false"))
	bool bSpecularMap = false;
	
	// Whether to generate a texture for the Emissive property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault))
	bool bEmissiveMap = false;

	// Whether to generate a texture for the Opacity property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault))
	bool bOpacityMap = false;

	// Whether to generate a texture for the SubsurfaceColor property
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, meta = (NoResetToDefault))
	bool bSubsurfaceColorMap = false;

	// Whether to use anti-aliasing in the render captures, this may introduce artefacts if pixels at different scene depths get blended
	UPROPERTY(Category = RenderCaptureOptions, EditAnywhere, AdvancedDisplay)
	bool bAntiAliasing = false;

	// Whether to generate a texture for the DeviceDepth.  This option is hidden from the user since its set only if the
	// Bake operation requires a depth map to remove occlusion artefacts, which is determined by ValidSampleDepthThreshold
	UPROPERTY()
	bool bDeviceDepthMap = false;

	// These are hidden in the UI right now, we might want to expose them if they turn out to be useful for very large
	// or very small objects (not tested yet) TODO Figure out if we want to expose these options
	
	UPROPERTY(meta = (ClampMin = "5.0", ClampMax = "160.0"))
	float CaptureFieldOfView = 30.0f;

	UPROPERTY(meta = (ClampMin = "0.001", ClampMax = "1000.0"))
	float NearPlaneDist = 1.0f;

	UE_API bool operator==(const URenderCaptureProperties& Other) const;
	UE_API bool operator!=(const URenderCaptureProperties& Other) const;
};

UCLASS(MinimalAPI)
class UBakeRenderCaptureToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	
	/** The map type to preview */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (DisplayName="Preview Output Type", EditCondition = "bEnableMapPreview", HideEditConditionToggle, TransientToolProperty, GetOptions = GetMapPreviewNamesFunc, NoResetToDefault))
	FString MapPreview = TEXT("");

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, Category = BakeOutput)
	EBakeTextureSamplesPerPixel SamplesPerPixel = EBakeTextureSamplesPerPixel::Sample1;

	/* Size of generated textures */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (ClampMin="1", UIMin="1"), DisplayName="Output Resolution")
	EBakeTextureResolution TextureSize = EBakeTextureResolution::Resolution512;

	/**
	 * This threshold is used to detect occlusion artefacts (e.g., silhouettes/blotches in the base color) so that texels
	 * exhibiting them can be cleaned up i.e., rejected and filled in using the values of nearby valid texels instead.
	 * 
	 * If the threshold is zero, the cleanup step is skipped.
	 * If the threshold is too large, texels with artefacts won't be detected and the cleanup step is effectively skipped.
	 * If the threshold is too small, texels without artefacts get detected and results will be bad because there won't
	 * be enough nearby valid texels from which to infill values.
	 * 
	 * A good starting point is to choose a threshold around the size of the distance (in world space) between the
	 * target and source meshes; if the target was generated by VoxWrap then the voxel size estimates this distance.
	 */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (ClampMin="0", UIMin="0"), DisplayName="Cleanup Threshold")
	float ValidSampleDepthThreshold = 0.f;

	//~ Hidden properties

	UPROPERTY(Transient, meta = (TransientToolProperty))
	bool bEnableMapPreview = false;

	UFUNCTION()
	const TArray<FString>& GetMapPreviewNamesFunc()
	{
		return MapPreviewNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> MapPreviewNamesList;
};


UCLASS(MinimalAPI)
class UBakeRenderCaptureInputToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty, NoResetToDefault))
	TObjectPtr<UStaticMesh> TargetStaticMesh = nullptr;

	/** UV channel to use for the target mesh */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (DisplayName = "Target Mesh UV Channel", GetOptions = GetTargetUVLayerNamesFunc, NoResetToDefault))
	FString TargetUVLayer;

	UFUNCTION()
	int32 GetTargetUVLayerIndex() const
	{
		return TargetUVLayerNamesList.IndexOfByKey(TargetUVLayer);
	}

	UFUNCTION()
	const TArray<FString>& GetTargetUVLayerNamesFunc() const
	{
		return TargetUVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> TargetUVLayerNamesList;
};

UCLASS(MinimalAPI)
class UBakeRenderCaptureVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/**
	 * If true  preview results by connecting them to corresponding material inputs
	 * If false connect the selected preview output as the Base Color input and use empty maps for other material inputs
	 */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bPreviewAsMaterial = false;

	/** Adjust the brightness of the Base Color material input; does not affect results stored in textures */
	UPROPERTY(EditAnywhere, Category = Preview, meta = (UIMin = "0.0", UIMax = "1.0"))
	float Brightness = 1.0f;

	/** Adjust the brightness of the Subsurface Color material input; does not affect results stored in textures */
	UPROPERTY(EditAnywhere, Category = Preview, meta = (UIMin = "0.0", UIMax = "1.0", DisplayName = "SS Brightness"))
	float SSBrightness = 1.0f;

	/** Adjust the brightness of the Emissive Color material input; does not affect results stored in textures */
	UPROPERTY(EditAnywhere, Category = Preview, meta = (UIMin = "0.0", UIMax = "1.0"))
	float EmissiveScale = 1.0f;
};


//
// Tool
//



UCLASS(MinimalAPI)
class UBakeRenderCaptureTool :
	public UMultiSelectionMeshEditingTool,
	public UE::Geometry::IGenericDataOperatorFactory<UE::Geometry::FMeshMapBaker>,
	public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

public:
	UBakeRenderCaptureTool() = default;

	// Begin UMultiSelectionMeshEditing < UMultiSelectionTool < UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	// End UMultiSelectionMeshEditing < UMultiSelectionTool < UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

protected:

	UPROPERTY()
	TArray<TObjectPtr<AActor>> Actors;

	UPROPERTY()
	TObjectPtr<UBakeRenderCaptureToolProperties> Settings;
	int32 MapPreviewWatcherIndex = -1;

	UPROPERTY()
	TObjectPtr<URenderCaptureProperties> RenderCaptureProperties;

	UPROPERTY()
	TObjectPtr<UBakeRenderCaptureInputToolProperties> InputMeshSettings;

	UPROPERTY()
	TObjectPtr<UBakeRenderCaptureVisualizationProperties> VisualizationProps;

	// The computed textures are displayed in the details panel and used in the preview material, they are written
	// out to assest on shutdown.
	UPROPERTY()
	TObjectPtr<UBakeRenderCaptureResults> ResultSettings;
	
protected:

	UE_API void UpdateResult();
	UE_API void UpdateVisualization();
	UE_API void InvalidateResults(UE::Geometry::FRenderCaptureTypeFlags ToInvalidate);
	UE_API void OnMapsUpdated(const TUniquePtr<UE::Geometry::FMeshMapBaker>& NewResult);

	/**
	 * Compute validity of the Target Mesh tangents. Only checks validity
	 * once and then caches the result for successive calls.
	 * 
	 * @return true if the TargetMesh tangents are valid
	 */
	UE_API bool ValidTargetMeshTangents();
	bool bCheckTargetMeshTangents = true;
	bool bValidTargetMeshTangents = false;

	/**
	 * Create texture and material assets from our result map of Texture2D
	 * @param SourceWorld the source world to define where the assets will be stored.
	 */
	UE_API void CreateAssets(UWorld* SourceWorld);

	TUniquePtr<TGenericDataBackgroundCompute<UE::Geometry::FMeshMapBaker>> BakeOp = nullptr;
	EBakeOpState BakeOpState = EBakeOpState::Evaluate;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> TargetMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> TargetMeshSpatial;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> TargetMeshUVCharts;
	TSharedPtr<UE::Geometry::FMeshTangentsd, ESPMode::ThreadSafe> TargetMeshTangents;

	// Empty maps are shown when nothing is computed
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyNormalMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyColorMapBlack;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyColorMapWhite;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyEmissiveMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyOpacityMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptySubsurfaceColorMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyPackedMRSMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyRoughnessMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyMetallicMap;
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptySpecularMap;

	float SecondsBeforeWorkingMaterial = 0.75;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> WorkingPreviewMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> ErrorPreviewMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterialRC;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterialPackedRC;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterialRC_Subsurface;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterialPackedRC_Subsurface;

	UE_API void InitializePreviewMaterials();

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	// Note: We need to compute this on the game thread because the implementation has checks for this
	TSharedPtr<UE::Geometry::FSceneCapturePhotoSet, ESPMode::ThreadSafe> SceneCapture;

	// These are used to determine if we need to re-bake results
	float ComputedValidDepthThreshold;
	EBakeTextureSamplesPerPixel ComputedSamplesPerPixel = EBakeTextureSamplesPerPixel::Sample1;
	EBakeTextureResolution ComputedTextureSize = EBakeTextureResolution::Resolution512;
	FString ComputedTargetUVLayer;

	TMap<int, FText> TargetUVLayerToError;

	//
	// Analytics
	//
	
	struct FBakeAnalytics
	{
		double TotalBakeDuration = 0.0;
		double WriteToImageDuration = 0.0;
		double WriteToGutterDuration = 0.0;
		int64 NumSamplePixels = 0;
		int64 NumGutterPixels = 0;

		struct FMeshSettings
		{
			int32 NumTargetMeshTris = 0;
			int32 NumDetailMesh = 0;
			int64 NumDetailMeshTris = 0;
		};
		FMeshSettings MeshSettings;
	};

	FBakeAnalytics BakeAnalytics;

	FString GetAnalyticsEventName() const { return TEXT("BakeRC"); }
	UE_API void GatherAnalytics(FBakeAnalytics::FMeshSettings& Data);
	UE_API void GatherAnalytics(const UE::Geometry::FMeshMapBaker& Result);
	UE_API void RecordAnalytics() const;
};

#undef UE_API
