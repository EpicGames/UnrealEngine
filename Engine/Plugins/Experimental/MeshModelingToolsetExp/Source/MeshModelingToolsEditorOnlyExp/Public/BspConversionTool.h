// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UObject/ObjectMacros.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "ToolContextInterfaces.h"
#include "MultiSelectionTool.h"
#include "PreviewMesh.h"
#include "Properties/MeshMaterialProperties.h"

#include "BspConversionTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class ABrush;

/**
 * Builder for UBspConversionTool.
 */
UCLASS(MinimalAPI)
class UBspConversionToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


UENUM()
enum class EBspConversionMode : uint8
{
	/** First converts the brushes to static meshes, then performs mesh boolean operations. */
	ConvertFirst = 0 UMETA(DisplayName = "Convert, then Combine"),

	/** First combines brushes using brush CSG operations, then converts result to static mesh (legacy path). */
	CombineFirst = 1 UMETA(DisplayName = "Combine, then Convert"),
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UBspConversionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Options)
	EBspConversionMode ConversionMode = EBspConversionMode::ConvertFirst;

	/** Whether to consider BSP volumes to be valid conversion targets. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Options)
	bool bIncludeVolumes = false;
	
	/** Whether to remove any selected BSP volumes after using them to create a static mesh. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Options, meta = (EditCondition = "bIncludeVolumes"))
	bool bRemoveConvertedVolumes = false;

	/** Whether subtractive brushes have to be explicitly selected to be part of the conversion. If false, all
	 subtractive brushes in the level will be used. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Options)
	bool bExplicitSubtractiveBrushSelection = true;

	/** Whether subtractive brushes used in a conversion should be removed. Only acts on explicitly selected
	 subtractive brushes. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Options)
	bool bRemoveConvertedSubtractiveBrushes = true;

	/** Caches individual brush conversions in "convert then combine" mode during a single invocation of 
	 the tool. Only useful if changing selections or properties after starting the tool. Cleared on tool shutdown. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Options, AdvancedDisplay, meta = (EditCondition = "ConversionMode == EBspConversionMode::ConvertFirst"))
	bool bCacheBrushes = true;

	/** Determines whether a dynamic preview is shown. Note that this introduces non-background computations 
	at each event that changes the result, rather than only performing a computation on Accept. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = PreviewOptions, AdvancedDisplay)
	bool bShowPreview = true;
};

UENUM()
enum class EBspConversionToolAction
{
	NoAction,

	SelectAllValidBrushes,
	DeselectVolumes,
	DeselectNonValid
};

UCLASS(MinimalAPI)
class UBspConversionToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UBspConversionTool> ParentTool;

	void Initialize(UBspConversionTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EBspConversionToolAction Action);

	/** Select all brushes that satisfy the current settings. */
	UFUNCTION(CallInEditor, Category = SelectionOperations, Meta = (DisplayName = "Select All Valid Brushes", DisplayPriority = 1))
	void SelectAllValidBrushes() { PostAction(EBspConversionToolAction::SelectAllValidBrushes); }

	/** Deselect any currently selected volume brushes. */
	UFUNCTION(CallInEditor, Category = SelectionOperations, Meta = (DisplayName = "Deselect Volumes", DisplayPriority = 2))
	void DeselectVolumes() { PostAction(EBspConversionToolAction::DeselectVolumes); }

	/** Deselect any currently selected brushes that are not valid targets given current settings. */
	UFUNCTION(CallInEditor, Category = SelectionOperations, Meta = (DisplayName = "Deselect Non-Valid", DisplayPriority = 3))
	void DeselectNonValid() { PostAction(EBspConversionToolAction::DeselectNonValid); }
};

/**
 * Converts BSP brushes to static meshes.
 *
 * Known limitations:
 * - Preview does not respond to property changes in the brush detail panel while the tool is running. User would need
 *   to create some event that does change the preview (such as tool property change, or selection change).
 * - BSP brushes with non-manifold geometry (specifically, the stair brushes) cannot be used with the "Convert, then combine"
 *   path because boolean operations do not allow them. The user gets properly notified of this if it comes up.
 */
UCLASS(MinimalAPI)
class UBspConversionTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	UE_API UBspConversionTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual void SetWorld(UWorld* World) { this->TargetWorld = World; }

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void RequestAction(EBspConversionToolAction ActionType);

	UPROPERTY()
	TObjectPtr<UBspConversionToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UBspConversionToolActionPropertySet> ToolActions = nullptr;

protected:
	bool bCanAccept = false;

	UWorld* TargetWorld;
	EBspConversionToolAction PendingAction;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	TArray<ABrush*> BrushesToConvert;
	TArray<ABrush*> BrushesToDelete;
	ABrush* BrushForPivot = nullptr;

	TMap<ABrush*, TPair<TSharedPtr<const UE::Geometry::FDynamicMesh3>, 
		TSharedPtr<const TArray<UMaterialInterface*>>>> CachedBrushes;

	UE_API bool IsValidConversionTarget(const ABrush* Brush) const;
	UE_API bool AtLeastOneValidConversionTarget() const;
	UE_API bool CompareAndUpdateConversionTargets();

	UE_API void OnEditorSelectionChanged(UObject* NewSelection);
	UE_API void OnEditorLevelActorListChanged();
	UE_API void OnEditorActorMoved(AActor* InActor);

    UE_API void ApplyAction(EBspConversionToolAction ActionType);
	UE_API bool ComputeAndUpdatePreviewMesh(FText* ErrorMessage = nullptr);
	UE_API bool ConvertThenCombine(FText* ErrorMessage = nullptr);
	UE_API bool CombineThenConvert(FText* ErrorMessage = nullptr);
};

#undef UE_API
