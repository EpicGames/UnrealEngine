// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "InteractiveToolChange.h"
#include "ToolContextInterfaces.h" // FViewCameraState
#include "TransactionUtil.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "DrawSplineTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class AActor;
class APreviewGeometryActor;
class UConstructionPlaneMechanic;
class USingleClickOrDragInputBehavior;
class USplineComponent;
class UWorld;

UENUM()
enum class EDrawSplineDrawMode : uint8
{
	// Click to place a point and then drag to set its tangent. Clicking without
	// dragging will create sharp corners.
	TangentDrag,
	// Click and drag new points, with the tangent set automatically
	ClickAutoTangent,
	// Drag to place multiple points, with spacing controlled by Min Point Spacing
	FreeDraw,
};

UENUM()
enum class ESplineOffsetMethod : uint8
{
	// Spline points will be offset along the normal direction of the clicked surface
	HitNormal,
	// Spline points will be offset along a manually-chosen direction
	Custom
};

UENUM()
enum class EDrawSplineOutputMode : uint8
{
	// Create a new empty actor with the spline inside it 
	EmptyActor,
	// Attach the spline to an existing actor, or replace a spline inside that
	// actor if Existing Spline Index To Replace is valid.
	ExistingActor,
	// Create the blueprint specified by Blueprint To Create, and either attach
	// the spline to that, or replace an existing spline if Existing Spline Index 
	// To Replace is valid.
	CreateBlueprint,

	//~ The original implementation of this option was too unwieldy, but there is some way to
	//~ implement it using FComponentReference to make a nice spline component picker...
	//SelectedComponent,
};

UENUM()
enum class EDrawSplineUpVectorMode : uint8
{
	// Pick the first up vector based on the hit normal, and then align subsequent
	// up vectors with the previous ones.
	AlignToPrevious,
	// Base the up vector off the hit normal.
	UseHitNormal,
};

UCLASS(MinimalAPI)
class UDrawSplineToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	//~ This is TransientToolProperty because while it's useful to toggle after drawing a spline,
	//~ it's confusing when you first start the tool and are drawing a spline.
	/** 
	 * Determines whether the created spline is a loop. This can be toggled using "Closed Loop" in
	 * the detail panel after spline creation.
	 */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (TransientToolProperty))
	bool bLoop = false;

	/**
	 * Determines how the resulting spline is emitted on tool accept.
	 */
	UPROPERTY(EditAnywhere, Category = Spline)
	EDrawSplineOutputMode OutputMode = EDrawSplineOutputMode::EmptyActor;

	/**
	 * Actor to attach to when Output Mode is "Existing Actor"
	 */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (TransientToolProperty, 
		EditCondition = "OutputMode == EDrawSplineOutputMode::ExistingActor", EditConditionHides))
	TWeakObjectPtr<AActor> TargetActor = nullptr;

	/**
	 * Blueprint to create when Output Mode is "Create Blueprint"
	 */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (
		EditCondition = "OutputMode == EDrawSplineOutputMode::CreateBlueprint", EditConditionHides))
	TWeakObjectPtr<UBlueprint> BlueprintToCreate;

	/**
	 * When attaching to an existing actor or creating a blueprint, controls whether the drawn
	 * spline replaces one of the target's existing components or gets added as a new one (if
	 * the index is out of bounds).
	 */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (ClampMin = -1, UIMax = 10, EditConditionHides,
		EditCondition = "OutputMode == EDrawSplineOutputMode::ExistingActor || OutputMode == EDrawSplineOutputMode::CreateBlueprint"))
	int32 ExistingSplineIndexToReplace = 0;

	/**
	 * How the spline is drawn in the tool.
	 */
	UPROPERTY(EditAnywhere, Category = Spline)
	EDrawSplineDrawMode DrawMode = EDrawSplineDrawMode::TangentDrag;
	
	/** Point spacing when Draw Mode is "Free Draw" */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (ClampMin = 0, UIMax = 1000,
		EditCondition = "DrawMode == EDrawSplineDrawMode::FreeDraw", EditConditionHides))
	double MinPointSpacing = 200;

	/** How far to offset spline points from the clicked surface, along the surface normal */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (UIMin = 0, UIMax = 100))
	double ClickOffset = 0;

	/** How to choose the direction to offset points from the clicked surface */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (EditCondition = "ClickOffset > 0", EditConditionHides))
	ESplineOffsetMethod OffsetMethod = ESplineOffsetMethod::HitNormal;

	/** Manually-specified click offset direction. Note: Will be normalized. If it is a zero vector, a default Up vector will be used instead. */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (EditCondition = "ClickOffset > 0 && OffsetMethod == ESplineOffsetMethod::Custom", EditConditionHides))
	FVector OffsetDirection = FVector::UpVector;

	/**
	 * When nonzero, allows a visualization of the rotation of the spline. Can be controlled
	 * in the detail panel after creation via "Scale Visualization Width" property.
	 */
	UPROPERTY(EditAnywhere, Category = Spline, AdvancedDisplay, meta = (ClampMin = 0, UIMax = 100))
	double FrameVisualizationWidth = 0;

	/**
	 * How the spline rotation is set. It is suggested to use a nonzero FrameVisualizationWidth to
	 * see the effects.
	 */
	UPROPERTY(EditAnywhere, Category = Spline, AdvancedDisplay)
	EDrawSplineUpVectorMode UpVectorMode = EDrawSplineUpVectorMode::UseHitNormal;

	/**
	 * When modifying existing actors, whether the result should be previewed using a copy
	 * of that actor (rather than just drawing the spline by itself). Could be toggled off
	 * if something about duplicating the given actor is problematic.
	 */
	UPROPERTY(EditAnywhere, Category = Spline, AdvancedDisplay, meta=(
		EditCondition = "OutputMode != EDrawSplineOutputMode::EmptyActor"))
	bool bPreviewUsingActorCopy = true;

	/** Whether to place spline points on the surface of objects in the world */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (DisplayName = "World Objects"))
	bool bHitWorld = true;

	/** Whether to place spline points on a custom, user-adjustable plane */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (DisplayName = "Custom Plane"))
	bool bHitCustomPlane = false;

	/** Whether to place spline points on a plane through the origin aligned with the Z axis in perspective views, or facing the camera in othographic views */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (DisplayName = "Ground Planes"))
	bool bHitGroundPlanes = true;

	/** Whether to fit the spline to the selected surface. The spline will be subdivided and projected to reduce the distance between the spline and the surface */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (EditCondition = bMeshSelected, HideEditConditionToggle))
	bool bFitToSurface = false;

	/** The maximum number of times the spline will be subdivided in order to fit the surface */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (EditCondition = "bFitToSurface && bMeshSelected", ClampMin = 0, UIMax = 100))
	int32 MaxNewPoints = 100;

	/** The threshold for spline subdivision. New points are iteratively added where the distance from the spline to the surface is greater than this value */
	UPROPERTY(EditAnywhere, Category = RaycastTargets, meta = (EditCondition = "bFitToSurface && bMeshSelected", ClampMin = 0, UIMax = 100))
	double DistanceThreshold = 1.0;

	/**
	 * If modifying a blueprint actor, whether to run the construction script while dragging
	 * or only at the end of a drag. Can be toggled off for expensive construction scripts.
	 */
	UPROPERTY(EditAnywhere, Category = Spline, AdvancedDisplay, meta = (
		EditCondition = "OutputMode != EDrawSplineOutputMode::EmptyActor"))
	bool bRerunConstructionScriptOnDrag = true;

	// Hidden property set by the tool
	UPROPERTY()
	bool bMeshSelected = false;
};

UCLASS(MinimalAPI)
class UDrawSplineTool : public USingleSelectionMeshEditingTool
	, public IClickBehaviorTarget
	, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:

	UE_API virtual void SetSelectedActor(AActor* Actor);

	UE_API virtual void SetWorld(UWorld* World);
	virtual UWorld* GetTargetWorld() { return TargetWorld.Get(); }

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IClickBehaviorTarget
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos);
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos);

	// IClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos);
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos);
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos);
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos);
	UE_API virtual void OnTerminateDragSequence();
private:

	UPROPERTY()
	TObjectPtr<UDrawSplineToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	TWeakObjectPtr<UWorld> TargetWorld = nullptr;

	// This is only used to initialize TargetActor in the settings object
	TWeakObjectPtr<AActor> StartupSelectedActor = nullptr;

	// The preview actor is either a APreviewGeometryActor with a spline, or a duplicate of 
	// some target blueprint actor so that we can see the effects of the drawn spline immediately.
	UPROPERTY()
	TObjectPtr<AActor> PreviewActor = nullptr;

	// Used for recapturing the spline when rerunning construction scripts
	int32 SplineRecaptureIndex = 0;

	// This is the spline we add points to. It points to a component nested somewhere under 
	// PreviewRootActor.
	TWeakObjectPtr<USplineComponent> WorkingSpline = nullptr;

	bool bDrawTangentForLastPoint = false;
	bool bFreeDrawPlacedPreviewPoint = false;
	int32 FreeDrawStrokeStartIndex = 0;

	void RefitToSurface();

	UE_API bool Raycast(const FRay& WorldRay, FVector3d& HitLocationOut, FVector3d& HitNormalOut, double& HitTOut);
	UE_API void AddSplinePoint(const FVector3d& HitLocation, const FVector3d& UpVector);
	UE_API FVector3d GetUpVectorToUse(const FVector3d& HitLocation, const FVector3d& HitNormal, int32 NumSplinePointsBeforehand);

	UE_API void TransitionOutputMode();
	UE_API void GenerateAsset();
	
	// Used to restore visibility of previous actor when switching to a different one, and to avoid switching
	//  target actors if the new value is invalid (e.g., user clicked the preview actor with the actor picker)
	UPROPERTY()
	TObjectPtr<AActor> PreviousTargetActor = nullptr;

	int32 TargetActorWatcherID = -1;
	bool PreviousTargetActorVisibility = true;
	// Used to restore visibility of previous spline when switching to a different one
	TWeakObjectPtr<USplineComponent> HiddenSpline = nullptr;
	bool bPreviousSplineVisibility = true;

	bool bNeedToRerunConstructionScript = false;

	FViewCameraState CameraState;

	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> ProjectionTarget;
	TUniquePtr<UE::Geometry::FDynamicMesh3> ProjectionTargetMesh;
	FTransform MeshTargetTransform;
	TWeakObjectPtr<USplineComponent> ProjectionSpline = nullptr;


private:
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;

public:
	// Helper class for making undo/redo transactions, to avoid friending all the variations.
	class FSplineChange : public FToolCommandChange
	{
	public:
		// These pass the working spline to the overloads below
		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;

	protected:
		virtual void Apply(USplineComponent& Spline) = 0;
		virtual void Revert(USplineComponent& Spline) = 0;
	};
};



UCLASS(MinimalAPI, Transient)
class UDrawSplineToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual void InitializeNewTool(USingleSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const override;
};

#undef UE_API
