// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshSelectionTool.h"
#include "EditMeshMaterialsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API


/**
 *
 */
UCLASS(MinimalAPI)
class UEditMeshMaterialsToolBuilder : public UMeshSelectionToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};






UCLASS(MinimalAPI)
class UEditMeshMaterialsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** */
	UPROPERTY(EditAnywhere, Category = Materials, meta = (TransientToolProperty, DisplayName = "Active Material", GetOptions = GetMaterialNamesFunc, NoResetToDefault))
	FString ActiveMaterial;

	UFUNCTION()
	const TArray<FString>& GetMaterialNamesFunc() { return MaterialNamesList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> MaterialNamesList;

	UE_API void UpdateFromMaterialsList();
	UE_API int32 GetSelectedMaterialIndex() const;

	UPROPERTY(EditAnywhere, Category=Materials)
	TArray<TObjectPtr<UMaterialInterface>> Materials;
};





UENUM()
enum class EEditMeshMaterialsToolActions
{
	NoAction,
	AssignMaterial
};



UCLASS(MinimalAPI)
class UEditMeshMaterialsEditActions : public UMeshSelectionToolActionPropertySet
{
	GENERATED_BODY()

public:
	UFUNCTION(CallInEditor, Category = MaterialEdits, meta = (DisplayName = "Assign Active Material", DisplayPriority = 1))
	void AssignActiveMaterial()
	{
		PostMaterialAction(EEditMeshMaterialsToolActions::AssignMaterial);
	}

	UE_API void PostMaterialAction(EEditMeshMaterialsToolActions Action);
};






/**
 *
 */
UCLASS(MinimalAPI)
class UEditMeshMaterialsTool : public UMeshSelectionTool
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	UE_API virtual bool CanAccept() const override;

	UE_API void RequestMaterialAction(EEditMeshMaterialsToolActions ActionType);

protected:
	UPROPERTY()
	TObjectPtr<UEditMeshMaterialsToolProperties> MaterialProps;

	UE_API virtual UMeshSelectionToolActionPropertySet* CreateEditActions() override;
	UE_API virtual void AddSubclassPropertySets() override;

	bool bHavePendingSubAction = false;
	EEditMeshMaterialsToolActions PendingSubAction = EEditMeshMaterialsToolActions::NoAction;

	UE_API void ApplyMaterialAction(EEditMeshMaterialsToolActions ActionType);
	UE_API void AssignMaterialToSelectedTriangles();

	TArray<UMaterialInterface*> CurrentMaterials;
	UE_API void OnMaterialSetChanged();

	struct FMaterialSetKey
	{
		TArray<void*> Values;
		bool operator!=(const FMaterialSetKey& Key2) const;
	};
	UE_API FMaterialSetKey GetMaterialKey();

	FMaterialSetKey InitialMaterialKey;
	bool bHaveModifiedMaterials = false;
	bool bShowingMaterialSetError = false;
	bool bShowingNotEnoughMaterialsError = false;

	UE_API virtual void ApplyShutdownAction(EToolShutdownType ShutdownType) override;

	UE_API void ExternalUpdateMaterialSet(const TArray<UMaterialInterface*>& NewMaterialSet);
	friend class FEditMeshMaterials_MaterialSetChange;

private:

	// @return the max material ID used in the PreviewMesh
	UE_API int32 FindMaxActiveMaterialID() const;

	// Clamp material IDs to be less than the number of materials
	UE_API bool FixInvalidMaterialIDs();

	// Update error messages regarding whether we have enough materials on the current mesh
	UE_API void UpdateMaterialSetErrors();

	int32 MaterialSetWatchIndex = -1;
};




/**
 */
class FEditMeshMaterials_MaterialSetChange : public FToolCommandChange
{
public:
	TArray<UMaterialInterface*> MaterialsBefore;
	TArray<UMaterialInterface*> MaterialsAfter;

	/** Makes the change to the object */
	UE_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	UE_API virtual void Revert(UObject* Object) override;

	/** Describes this change (for debugging) */
	UE_API virtual FString ToString() const override;
};

#undef UE_API
