// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "FrameTypes.h"
#include "BoxTypes.h"
#include "TransformTypes.h"
#include "HarvestInstancesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class UStaticMesh;
class UStaticMeshComponent;
class UInstancedStaticMeshComponent;
class UMaterialInterface;


/**
 *
 */
UCLASS(MinimalAPI)
class UHarvestInstancesToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const override;
	
protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
 * Settings for the Pattern Tool
 */
UCLASS(MinimalAPI)
class UHarvestInstancesToolSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Display the bounding box of each mesh that will be moved to instances. Bounding box colors indicate instance groupings. */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bDrawBounds = true;
};



UENUM()
enum class EHarvestInstancesToolOutputType
{
	HISMC UMETA(DisplayName="Hierarchical (HISM)"),
	ISMC UMETA(DisplayName="Non-Hierarchical (ISM)")
};



/**
 * Output Settings for the Pattern Tool
 */
UCLASS(MinimalAPI)
class UHarvestInstancesTool_OutputSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Which type of Instanced Static Mesh Component to Create */
	UPROPERTY(EditAnywhere, Category = Output)
	EHarvestInstancesToolOutputType ComponentType = EHarvestInstancesToolOutputType::HISMC;

	/** Group all output Instanced Components under a single Actor. By default, multiple Actors will be created. */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition="bHaveSingleInstanceSet == false", HideEditConditionToggle) )
	bool bSingleActor = false;

	/** Base Name to use for the emitted single Actor */
	UPROPERTY(EditAnywhere, Category = Output, meta=(TransientToolProperty, DisplayName="New Actor Name", EditCondition = "bHaveSingleInstanceSet || bSingleActor == true"))
	FString ActorName = TEXT("Instances");

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Output, meta = (EditCondition = "bSingleActor == true && bHaveLonerInstances == true", HideEditConditionToggle))
	bool bIncludeSingleInstances = false;

	/** Delete Actors that have had their Components harvested */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition = "bCanDeleteInputs == true", HideEditConditionToggle))
	bool bDeleteInputs = true;

	UPROPERTY(meta=(TransientToolProperty))
	bool bHaveSingleInstanceSet = false;

	UPROPERTY(meta = (TransientToolProperty))
	bool bHaveLonerInstances = false;

	UPROPERTY(meta = (TransientToolProperty))
	bool bCanDeleteInputs = true;
};



/**
 * UHarvestInstancesTool 
 */
UCLASS(MinimalAPI)
class UHarvestInstancesTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()

public:
	UE_API UHarvestInstancesTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

public:
	UPROPERTY()
	TObjectPtr<UHarvestInstancesToolSettings> Settings;
	
	UPROPERTY()
	TObjectPtr<UHarvestInstancesTool_OutputSettings> OutputSettings;

protected:

	UE_API void OnParametersUpdated();


	struct FSourceMesh
	{
		TArray<UStaticMeshComponent*> SourceComponents;
		UStaticMesh* Mesh;
		TArray<UMaterialInterface*> InstanceMaterials;
		TArray<FTransform> InstanceTransformsWorld;
	};
	TArray<FSourceMesh> SourceMeshes;

	struct FInstanceSet
	{
		TArray<UStaticMeshComponent*> SourceComponents;
		TArray<AActor*> SourceActors;

		UStaticMesh* StaticMesh = nullptr;
		FBox Bounds;
		TArray<UMaterialInterface*> InstanceMaterials;

		TArray<FTransform> WorldInstanceTransforms;
	};
	TArray<FInstanceSet> InstanceSets;

	TArray<AActor*> SourceActors;
	TArray<UPrimitiveComponent*> SourceComponents;

	bool bHaveNonUniformScaleElements = false;

protected:

	UE_API void InitializeInstanceSets();
	UE_API void UpdateInstanceSets();

	UE_API void EmitResults();

	UE_API void UpdateWarningMessages();
};

#undef UE_API
