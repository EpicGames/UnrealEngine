// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MultiSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Changes/TransformChange.h"
#include "FrameTypes.h"
#include "BoxTypes.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "Mechanics/RectangleMarqueeMechanic.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "ISMEditorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class UBaseAxisTranslationGizmo;
class UAxisAngleGizmo;
class UDragAlignmentMechanic;
class ULocalDoubleClickInputBehavior;
class UTransformGizmo;
class UTransformProxy;
class UPreviewGeometry;
PREDECLARE_GEOMETRY(class FMeshSceneAdapter);

/**
 *
 */
UCLASS(MinimalAPI)
class UISMEditorToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



/** Mesh Transform modes */
UENUM()
enum class EISMEditorTransformMode : uint8
{
	/** Single Gizmo for all Instances */
	SharedGizmo = 0 UMETA(DisplayName = "Shared Gizmo"),

	/** Single Gizmo for all Instances, Rotations applied per-Instance */
	SharedGizmoLocal = 1 UMETA(DisplayName = "Shared Gizmo (Local)"),

	/** Separate Gizmo for each Instance */
	PerObjectGizmo = 2 UMETA(DisplayName = "Multi-Gizmo"),

	LastValue UMETA(Hidden)
};


/**
 * Standard properties of the Transform Meshes operation
 */
UCLASS(MinimalAPI)
class UISMEditorToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Transformation Mode controls the overall behavior of the Gizmos in the Tool */
	UPROPERTY(EditAnywhere, Category = Options)
	EISMEditorTransformMode TransformMode = EISMEditorTransformMode::SharedGizmo;

	/** When true, the Gizmo can be moved independently without affecting objects. This allows the Gizmo to be repositioned before transforming. */
	UPROPERTY(EditAnywhere, Category = Pivot, meta = (TransientToolProperty, EditCondition = "TransformMode != EISMEditorTransformMode::PerObjectGizmo") )
	bool bSetPivotMode = false;

	/** Show a highlight around all selectable instances */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowSelectable = true;

	/** Show a highlight around the selected instances */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowSelected = true;

	/** Hide the Selectable and Selected highlights when dragging with the Gizmos */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bHideWhenDragging = true;

	/** List of selected Component/Instance values */
	UPROPERTY(VisibleAnywhere, Category = Selection)
	TArray<FString> SelectedInstances;
};


USTRUCT()
struct FISMEditorTarget
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;
};



UENUM()
enum class EISMEditorToolActions
{
	NoAction,
	ClearSelection,
	ContractLast,
	ExpandLast,
	ExpandAll,
	Delete,
	Duplicate,
	DuplicateToNew,
	ExtractToNew,
	Replace
};


UCLASS(MinimalAPI)
class UISMEditorToolActionPropertySetBase : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	TWeakObjectPtr<UISMEditorTool> ParentTool;

	void Initialize(UISMEditorTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EISMEditorToolActions Action);
};



UCLASS(MinimalAPI)
class UISMEditorToolActionPropertySet : public UISMEditorToolActionPropertySetBase
{
	GENERATED_BODY()
public:
	/** Clear the active instance selection */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 0))
	void ClearSelection() { PostAction(EISMEditorToolActions::ClearSelection); }

	/**
	 * Take the last selected element and contract the selection to exclude any instances of that static mesh and materials.
	 */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 1))
	void ContractLast() { PostAction(EISMEditorToolActions::ContractLast); }

	/**
	 * Take the last selected element and expand the selection to include any other instances of the same static mesh and materials.
	 */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 2))
	void ExpandLast() { PostAction(EISMEditorToolActions::ExpandLast); }

	/**
	 * Expand the selection to include any other instances of the same static mesh(es) and materials as selected.
	 */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 3))
	void ExpandAll() { PostAction(EISMEditorToolActions::ExpandAll); }

	/** Delete the selected instances */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 4))
	void Delete() { PostAction(EISMEditorToolActions::Delete); }

	/** Duplicate the selected instances */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 5))
	void Duplicate() { PostAction(EISMEditorToolActions::Duplicate); }
};



UCLASS(MinimalAPI)
class UISMEditorToolReplacePropertySet : public UISMEditorToolActionPropertySetBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Replace)
	TObjectPtr<UStaticMesh> ReplaceWith = nullptr;

	/** Clear the active instance selection */
	UFUNCTION(CallInEditor, Category = Replace, meta = (DisplayPriority = 0))
	void Replace() { PostAction(EISMEditorToolActions::Replace); }
};

UCLASS(MinimalAPI)
class UISMEditorToolExtractPropertySet : public UISMEditorToolActionPropertySetBase
{
	GENERATED_BODY()
public:
	/** Name to use when duplicating or extracting to a new actor. */
	UPROPERTY(EditAnywhere, Category = Extract)
	FString ActorName;

	/** When true, the components created when extracting to a new actor are HISM's instead of ISM's. */
	UPROPERTY(EditAnywhere, Category = Extract)
	bool bCreateHISMs = false;

	/** Duplicate the selected instances into a new actor. */
	UFUNCTION(CallInEditor, Category = Extract)
	void DuplicateToNew() { PostAction(EISMEditorToolActions::DuplicateToNew); }

	/** Duplicate the selected instances into a new actor and delete them from the current actor(s). */
	UFUNCTION(CallInEditor, Category = Extract)
	void ExtractToNew() { PostAction(EISMEditorToolActions::ExtractToNew); }
};



/**
 *
 */
UCLASS(MinimalAPI)
class UISMEditorTool : public UInteractiveTool, public IInteractiveToolCameraFocusAPI, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void SetTargets(TArray<UInstancedStaticMeshComponent*> Components);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }


	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	UE_API virtual void RequestAction(EISMEditorToolActions ActionType);

	UE_API virtual void NotifySceneModified();


	//
	// Marquee Support
	//
public:
	UPROPERTY()
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior;

	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> RectangleMarqueeMechanic;

protected:
	UE_API void OnMarqueeRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

protected:
	UPROPERTY()
	TObjectPtr<UISMEditorToolProperties> TransformProps;

	UPROPERTY()
	TObjectPtr<UISMEditorToolActionPropertySet> ToolActions;

	UPROPERTY()
	TObjectPtr<UISMEditorToolReplacePropertySet> ReplaceAction;

	UPROPERTY()
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> TargetComponents;

	UPROPERTY()
	TArray<FISMEditorTarget> ActiveGizmos;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry;

private:
	UPROPERTY()
	TObjectPtr<UISMEditorToolExtractPropertySet> ExtractActions;

	UPROPERTY()
	TObjectPtr<ULocalDoubleClickInputBehavior> DoubleClickBehavior;
protected:

	TSharedPtr<UE::Geometry::FMeshSceneAdapter> MeshScene;
	TArray<UE::Geometry::FAxisAlignedBox3d> AllMeshBoundingBoxes;
	bool bMeshSceneDirty = false;
	UE_API void RebuildMeshScene();

	struct FSelectedInstance
	{
		UInstancedStaticMeshComponent* Component;
		int32 Index;
		UE::Geometry::FAxisAlignedBox3d WorldBounds;
		bool operator==(const FSelectedInstance& Other) const { return Component == Other.Component && Index == Other.Index; }
	};
	TArray<FSelectedInstance> CurrentSelection;
	UE_API void OnSelectionUpdated();
	UE_API void UpdateSelectionInternal(const TArray<FSelectedInstance>& NewSelection, bool bEmitChange);
	UE_API void UpdateSelectionFromUndoRedo(const TArray<FSelectedInstance>& NewSelection);

	bool bInActiveDrag = false;
	bool bShiftModifier = false;
	bool bCtrlModifier = false;
	UE_API void OnUpdateModifierState(int ModifierID, bool bIsOn);

	EISMEditorTransformMode CurTransformMode;
	UE_API void UpdateTransformMode(EISMEditorTransformMode NewMode);
	UE_API void UpdateSetPivotModes(bool bEnableSetPivot);
	UE_API void OnTransformCompleted();

	UE_API void SetActiveGizmos_Single(bool bLocalRotations);
	UE_API void SetActiveGizmos_PerObject();
	UE_API void ResetActiveGizmos();

	EISMEditorToolActions PendingAction = EISMEditorToolActions::NoAction;
	UE_API virtual void OnClearSelection();
	UE_API virtual void OnDeleteSelection();
	UE_API virtual void OnDuplicateSelection();
	UE_API virtual void OnReplaceSelection();
private:
	void OnContractLastSelection();
	void OnExpandLastSelection();
	void OnExpandAllSelection();
	void OnDuplicateToNew();
	void OnExtractToNew();

	void ExpandSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify);
	void ContractSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify);
	void ToggleSelectionBy(const FSelectedInstance& SelectedInstance, TArray<FSelectedInstance>& SelectionToModify);
	void UpdateSelectionInternal(const TArray<FSelectedInstance>& NewSelection, bool bEmitChange, const FText& TransactionLabel);

	void DuplicateToNew(bool bDeleteFromCurrent);

	void OnDoubleClicked(const FInputDeviceRay& ClickPos);

	// Helpers that use the modifiers to determine what to do with selection
	bool ShouldAddToSelection() { return bShiftModifier && !bCtrlModifier; }
	bool ShouldToggleSelection() { return bShiftModifier && bCtrlModifier; }
	bool ShouldSubtractSelection() { return !bShiftModifier && bCtrlModifier; }
	bool ModifyingSelection() { return bShiftModifier || bCtrlModifier; }
protected:
	UE_API void UpdatePreviewGeometry();


	UE_API virtual void InternalNotifySceneModified(const TArray<UInstancedStaticMeshComponent*>& ComponentList, bool bAddToScene);

	friend class FISMEditorSelectionChange;
	friend class FISMEditorSceneChange;
};



class FISMEditorSelectionChange : public FToolCommandChange
{
public:
	TArray<UISMEditorTool::FSelectedInstance> OldSelection;
	TArray<UISMEditorTool::FSelectedInstance> NewSelection;
	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;
};



class FISMEditorSceneChange : public FToolCommandChange
{
public:
	TArray<UInstancedStaticMeshComponent*> Components;
	bool bAdded;
	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;
};

#undef UE_API
