// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseVoxelTool.h"
#include "CompositionOps/VoxelMergeMeshesOp.h"
#include "MergeMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API




/**
 * Standard properties of the Merge Meshes operation
 */
UCLASS(MinimalAPI)
class UMergeMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** The size of the geometry bounding box major axis measured in voxels.*/
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "8", UIMax = "1024", ClampMin = "8", ClampMax = "1024"))
	int32 VoxelCount = 128;

	/** Remeshing adaptivity, prior to optional simplification */
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float MeshAdaptivity = 0.001f;

	/** Offset when remeshing, note large offsets with high voxels counts will be slow.  Hidden because this duplicates functionality of the voxel offset tool */
	UPROPERTY()
	float OffsetDistance = 0;

	/** Automatically simplify the result of voxel-based merge.*/
	UPROPERTY(EditAnywhere, Category = VoxelSettings)
	bool bAutoSimplify = false;
};

UCLASS(MinimalAPI)
class UMergeMeshesTool : public UBaseVoxelTool
{
	GENERATED_BODY()

public:

	UMergeMeshesTool() {}

protected:

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UMergeMeshesToolProperties> MergeProps;

};




UCLASS(MinimalAPI)
class UMergeMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:

	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UMergeMeshesTool>(SceneState.ToolManager);
	}

	virtual int32 MinComponentsSupported() const override { return 1; }
};


#undef UE_API
