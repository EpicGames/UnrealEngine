// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshVertexAttributePaintToolBase.h"

#include "MeshAttributePaintToolV2.generated.h"


#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class IToolsContextTransactionsAPI;
class UMeshElementsVisualizer;
class UMeshVertexAttributePaintToolSmoothBrushOpProps;
class UMeshVertexAttributePaintToolPaintBrushOpProps;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UMeshAttributePaintToolV2Builder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()
private:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;

};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UMeshAttributePaintToolV2Properties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Selected Attribute", GetOptions = GetAttributeNames,  NoResetToDefault))
	FString Attribute;

	UFUNCTION()
	const TArray<FString>& GetAttributeNames() { return Attributes; };

	TArray<FString> Attributes;

public:
	/**
	* Initialize the internal array of attribute names
	* @param bInitialize if set, selected Attribute will be reset to the first attribute or empty if there are none.
	*/
	UE_API void Initialize(const TArray<FName>& AttributeNames, bool bInitialize = false);

	/**
	 * @return selected attribute index, or -1 if invalid selection
	 */
	UE_API int32 GetSelectedAttributeIndex();
};

/**
 * Editor tool to paint a vertex attribute as vertex colors
 */
UCLASS(MinimalAPI)
class UMeshAttributePaintToolV2 : public UMeshVertexAttributePaintToolBase
{
	GENERATED_BODY()
public:
	UE_API virtual bool SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex) override;
	UE_API virtual void CommitToolMesh(FDynamicMesh3& InToolMesh) override;

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnTick(float DeltaTime) override;
	
	UPROPERTY()
	TObjectPtr<UMeshAttributePaintToolV2Properties> AttribProps;

	TValueWatcher<int32> SelectedAttributeWatcher;
};




#undef UE_API

