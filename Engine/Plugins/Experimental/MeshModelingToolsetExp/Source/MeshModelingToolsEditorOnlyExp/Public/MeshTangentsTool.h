// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PreviewMesh.h"
#include "Drawing/PreviewGeometryActor.h"
#include "MeshOpPreviewHelpers.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTangents.h"
#include "ParameterizationOps/CalculateTangentsOp.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "Selections/GeometrySelection.h"
#include "Properties/MeshUVChannelProperties.h"
#include "MeshTangentsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API


// Forward declarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;


/**
 *
 */
UCLASS(MinimalAPI)
class UMeshTangentsToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(USingleSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



UCLASS(MinimalAPI)
class UMeshTangentsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Method used for calculating the tangents */
	UPROPERTY(EditAnywhere, Category = TangentsCalculation)
	EMeshTangentsType CalculationMethod = EMeshTangentsType::FastMikkTSpace;

	/** Display the mesh tangents */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowTangents = true;

	/** Display the mesh normals */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowNormals = false;

	/** Length of lines used for displaying tangents and/or normals */
	UPROPERTY(EditAnywhere, Category = Display, meta = (UIMin = "0.01", UIMax = "25.0", ClampMin = "0.01", ClampMax = "10000000.0"))
	float LineLength = 2.0f;

	/** Thickness of lines used for displaying tangents and/or normals */
	UPROPERTY(EditAnywhere, Category = Display, meta = (UIMin = "0", UIMax = "25.0", ClampMin = "0", ClampMax = "1000.0"))
	float LineThickness = 1.0f;

	/** Display tangents and/or normals for degenerate triangles */
	UPROPERTY(EditAnywhere, Category = Display, AdvancedDisplay)
	bool bShowDegenerates = false;

	/** Display difference between FastMikkTSpace tangents and MikkTSpace tangents.
	 * This is only available if the FastMikkTSpace Calculation Method is selected. */
	UPROPERTY(EditAnywhere, Category = Display, AdvancedDisplay, meta = (DisplayName = "Compare with MikkT",
		EditCondition = "CalculationMethod == EMeshTangentsType::FastMikkTSpace"))
	bool bCompareWithMikkt = false;

	/** Minimum angle difference in degrees for a FastMikkTSpace tangent to be considered different to a MikkTSpace tangent.
	 * This is only available if a Compare with MikkT is enabled and the FastMikkTSpace Calculation Method is selected. */
	UPROPERTY(EditAnywhere, Category = Display, AdvancedDisplay, meta = (DisplayName = "Compare Threshold", UIMin = "0.5", UIMax = "90.0",
		EditCondition = "CalculationMethod == EMeshTangentsType::FastMikkTSpace && bCompareWithMikkt"))
	float CompareWithMikktThreshold = 5.0f;

};





/**
 * Simple Mesh Simplifying Tool
 */
UCLASS(MinimalAPI)
class UMeshTangentsTool : public USingleSelectionMeshEditingTool, public UE::Geometry::IGenericDataOperatorFactory<UE::Geometry::FMeshTangentsd>, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API UMeshTangentsTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// input selection support
	UE_API void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

	// IGenericDataOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshTangentsd>> MakeNewOperator() override;

protected:
	UPROPERTY()
	TObjectPtr<UMeshTangentsToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> DefaultMaterial = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;

	TUniquePtr<TGenericDataBackgroundCompute<UE::Geometry::FMeshTangentsd>> Compute = nullptr;

	TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> InputMeshDescription;
	TSharedPtr<UE::Geometry::FMeshTangentsf, ESPMode::ThreadSafe> InitialTangents;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

	bool bThicknessDirty = false;
	bool bLengthDirty = false;
	bool bVisibilityChanged = false;

	UE_API void OnTangentsUpdated(const TUniquePtr<UE::Geometry::FMeshTangentsd>& NewResult);
	UE_API void CopyToOverlays(const UE::Geometry::FMeshTangentsd& Tangents, FDynamicMesh3& Mesh);
	UE_API void UpdateVisualization(bool bThicknessChanged, bool bLengthChanged);
	UE_API TSet<int32> ComputeDegenerateTris() const;
	UE_API void ComputeMikkTDeviations(const TSet<int32>& DegenerateTris);

	struct FMikktDeviation
	{
		float MaxAngleDeg;
		int32 TriangleID;
		int32 TriVertIndex;
		FVector3f VertexPos;
		FVector3f MikktTangent, MikktBitangent;
		FVector3f OtherTangent, OtherBitangent;
	};
	TArray<FMikktDeviation> Deviations;

	//
	// Selection
	//

	UPROPERTY()
	TObjectPtr<UGeometrySelectionVisualizationProperties> GeometrySelectionVizProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> GeometrySelectionViz = nullptr;

	// The geometry selection that the user started the tool with.
	// If the selection is empty we operate on the whole mesh, otherwise only edit the overlay elements implied by the selection.
	UE::Geometry::FGeometrySelection InputGeometrySelection;

	// If the user starts the tool with an edge selection we convert it to a vertex selection with triangle topology
	// and store it here, we do this since we expect users to want vertex and edge selections to behave similarly.
	UE::Geometry::FGeometrySelection TriangleVertexGeometrySelection;

	// These are indices into InputMesh.
	// If both are non-empty we edit the corresponding elements in the overlay, otherwise operate on the whole overlay
	TSet<int> EditTriangles;
	TSet<int> EditVertices;

private:
	bool bHasDisplayedNoAttributeError = false;
};

#undef UE_API
