// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "ConversionUtils/DynamicMeshToVolume.h"
#include "Drawing/LineSetComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/BlockingVolume.h"
#include "GameFramework/Volume.h"
#include "InteractiveToolBuilder.h"
#include "PreviewMesh.h"
#include "PropertySets/OnAcceptProperties.h"
#include "SingleSelectionTool.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "MeshToVolumeTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

/**
 *
 */
UCLASS(MinimalAPI)
class UMeshToVolumeToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState & SceneState) const override;
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState & SceneState) const override;
};




UENUM()
enum class EMeshToVolumeMode
{
	/** Create a separate Volume Face for each Triangle */
	TriangulatePolygons,
	/** Create Volume Faces based on Planar Polygons */
	MinimalPolygons
};



UCLASS(MinimalAPI)
class UMeshToVolumeToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Method for converting the input mesh to a set of Planar Polygonal Faces in the output Volume. */
	UPROPERTY(EditAnywhere, Category = ConversionOptions)
	EMeshToVolumeMode ConversionMode = EMeshToVolumeMode::MinimalPolygons;

	/** 
	 * When true, adjacent coplanar groups will not be merged together into single faces. Not relevant if 
	 * conversion mode is set to emit all triangles separately.
	 */
	UPROPERTY(EditAnywhere, Category = ConversionOptions, meta = (EditCondition = "ConversionMode != EMeshToVolumeMode::TriangulatePolygons"))
	bool bPreserveGroupBoundaries = false;

	/**
	 * Determines whether mesh gets auto simplified when its triangle count is too high.
	 */
	UPROPERTY(EditAnywhere, Category = ConversionOptions, AdvancedDisplay)
	bool bAutoSimplify = true;

	/**
	 * Target triangle count for auto simplification when Auto Simplify is true.
	 */
	UPROPERTY(EditAnywhere, Category = ConversionOptions, AdvancedDisplay, meta = (EditCondition = "bAutoSimplify"))
	int32 SimplifyMaxTriangles = 250;

	/** Type of new Volume to create on Accept */
	UPROPERTY(EditAnywhere, Category = NewVolume, meta = (EditCondition = "TargetVolume == nullptr") )
	TSubclassOf<class AVolume> NewVolumeType = ABlockingVolume::StaticClass();

	/** If set, the target Volume will be updated, rather than creating a new Volume. */
	UPROPERTY(EditAnywhere, Category = UpdateExisting)
	TSoftObjectPtr<AVolume> TargetVolume;

};


using FDynamicMeshFaceArray = TArray<UE::Conversion::FDynamicMeshFace>;

/**
 * Converts a mesh to a volume.
 *
 * Note: If ConversionUtils/DynamicMeshToVolume is rewritten to be safe for runtime, this
 * tool can be moved out of the editor-only section and put with VolumeToMeshTool.
 */
UCLASS(MinimalAPI)
class UMeshToVolumeTool : public USingleSelectionMeshEditingTool, public UE::Geometry::IGenericDataOperatorFactory<FDynamicMeshFaceArray>
{
	GENERATED_BODY()

public:
	UE_API UMeshToVolumeTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// IGenericDataOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<FDynamicMeshFaceArray>> MakeNewOperator() override;

protected:
	UPROPERTY()
	TObjectPtr<UMeshToVolumeToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UOnAcceptHandleSourcesPropertiesSingle> HandleSourcesProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> VolumeEdgesSet;

protected:
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
	
	TUniquePtr<TGenericDataBackgroundCompute<FDynamicMeshFaceArray>> Compute = nullptr;

	UE_API void UpdateLineSet(FDynamicMeshFaceArray& Faces);

};

#undef UE_API
