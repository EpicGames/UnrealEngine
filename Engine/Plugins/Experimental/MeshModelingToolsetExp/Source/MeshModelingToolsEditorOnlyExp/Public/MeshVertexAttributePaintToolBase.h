// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "PropertySets/WeightMapSetProperties.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMesh/MeshNormals.h"
#include "TransformTypes.h"
#include "ToolDataVisualizer.h"
#include "GroupTopology.h"
#include "MeshVertexPaintTool.h"
#include "Curves/LinearColorRamp.h"
#include "ToolMeshSelector.h"

#include "Framework/Commands/InputChord.h"

#include "MeshVertexAttributePaintToolBase.generated.h"


#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class IToolsContextTransactionsAPI;
class UMeshElementsVisualizer;
class UMeshVertexAttributePaintToolSmoothBrushOpProps;
class UMeshVertexAttributePaintToolPaintBrushOpProps;
class UMeshVertexAttributePaintToolBase;




namespace UE::MeshVertexAttributePaintToolBase::Private
{
	class FMeshChange;
	class FStrokeAccumulator;

	// Selects how a transaction's stamps commit:
	//   GatedByMaxFalloff — each vertex gets at most one stamp's effect (max-falloff wins).
	//                       Reads from the pre-transaction snapshot.
	//   Accumulate        — stamps compound
	enum class EStampAccumulatorMode : uint8
	{
		GatedByMaxFalloff,
		Accumulate
	};
}

namespace UE::MeshVertexAttributePaintToolBase
{
	class FVertexAttributePaintToolDetailCustomization;
}

UENUM()
enum class EMeshVertexAttributePaintToolEditMode : uint8
{
	Brush,
	Mesh,
};

UENUM()
enum class EMeshVertexAttributePaintToolEditOperation : uint8
{
	Add,
	Replace,
	Multiply,
	Invert,
	Relax,
};

UENUM()
enum class EMeshVertexAttributePaintToolVisibilityType : uint8
{
	None,
	Unoccluded,
};

/** Value Query mode  */
UENUM()
enum class EMeshVertexAttributePaintToolValueQueryType : uint8
{
	/** Value is interpolated from triangle vertices */
	Interpolated UMETA(DisplayName = "Interpolated"),

	/** Return the value at the closest vertex of the triangle under the mouse cursor */
	NearestVertexFast UMETA(DisplayName = "Nearest Vertex (Fast)"),

	/** Return the value of the nearest vertex inside the brush radius, even if the vertex is not on the triangle under the mouse cursor */
	NearestVertexAccurate UMETA(DisplayName = "Nearest Vertex (Accurate)")
};

UENUM()
enum class EMeshVertexAttributePaintToolColorMode : uint8
{
	Greyscale,
	Ramp,
	FullMaterial,
};

// mirror direction mode
UENUM()
enum class EMeshVertexAttributePaintToolMirrorDirection : uint8
{
	PositiveToNegative,
	NegativeToPositive,
};

DECLARE_STATS_GROUP(TEXT("MeshVertexAttributePaintTool"), STATGROUP_VertexAttributePaintTool, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("MeshVertexAttributePaintTool_UpdateROI"), VertexAttributePaintTool_UpdateROI, STATGROUP_VertexAttributePaintTool);
DECLARE_CYCLE_STAT(TEXT("MeshVertexAttributePaintTool_ApplyStamp"), VertexAttributePaintTool_ApplyStamp, STATGROUP_VertexAttributePaintTool);
DECLARE_CYCLE_STAT(TEXT("MeshVertexAttributePaintTool_Tick"), VertexAttributePaintTool_Tick, STATGROUP_VertexAttributePaintTool);
DECLARE_CYCLE_STAT(TEXT("MeshVertexAttributePaintTool_Tick_ApplyStampBlock"), VertexAttributePaintTool_Tick_ApplyStampBlock, STATGROUP_VertexAttributePaintTool);
DECLARE_CYCLE_STAT(TEXT("MeshVertexAttributePaintTool_Tick_UpdateMeshBlock"), VertexAttributePaintTool_Tick_UpdateMeshBlock, STATGROUP_VertexAttributePaintTool);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolBrushProperties
{
	GENERATED_BODY()

	UPROPERTY()
	EMeshVertexAttributePaintToolEditOperation BrushMode = EMeshVertexAttributePaintToolEditOperation::Replace;

	/** Value to paint on the mesh (dependent on the BrushMode) */
	UPROPERTY()
	float AttributeValue = 1.0;

	/** Area Mode specifies the shape of the brush and which triangles will be included relative to the cursor */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (DisplayName = "Brush Area Mode"))
	EMeshVertexPaintBrushAreaType BrushAreaMode = EMeshVertexPaintBrushAreaType::Connected;

	/** The Region affected by the current operation will be bounded by edge angles larger than this threshold */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (UIMin = "0.0", ClampMin = "0.0", UIMax = "180.0", ClampMax = "180.0"))
	float AngleThreshold = 180.0f;

	/** The Region affected by the current operation will be bounded by UV borders/seams */
	UPROPERTY(EditAnywhere, Category = Filters)
	bool bUVSeams = false;

	/** The Region affected by the current operation will be bounded by Hard Normal edges/seams */
	UPROPERTY(EditAnywhere, Category = Filters)
	bool bNormalSeams = false;

	/** Control which triangles can be affected by the current operation based on visibility. Applied after all other filters. */
	UPROPERTY(EditAnywhere, Category = Filters)
	EMeshVertexAttributePaintToolVisibilityType VisibilityFilter = EMeshVertexAttributePaintToolVisibilityType::None;

	/** The weight value at the brush indicator */
	UPROPERTY(VisibleAnywhere, Transient, Category = Query, meta = (NoResetToDefault))
	double ValueAtBrush = 0;

	/** method used for querying the value at brush */
	UPROPERTY(EditAnywhere, Category = Filters)
	EMeshVertexAttributePaintToolValueQueryType ValueQueryType = EMeshVertexAttributePaintToolValueQueryType::NearestVertexFast;
};

USTRUCT()
struct FMeshVertexAttributePaintToolBrushConfig
{
	GENERATED_BODY()

public:
	/** Adaptive size of brush - Relative to the model size */
	UPROPERTY(Config)
	float BrushSize = 0.25f;

	/** Value to paint on the mesh (dependent on the BrushMode) */
	UPROPERTY(Config)
	float Value = 1.0;

	UPROPERTY(Config)
	float Falloff = 0.0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolRelaxBrushAdvancedConfig
{
	GENERATED_BODY()

	UPROPERTY(Config)
	bool bAccumulate = true;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolGradientProperties
{
	GENERATED_BODY()

	// Mesh selection mode (vertex, edge face)
	/** The Gradient upper limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	double GradientHighValue = 1.0;

	/** The Gradient lower limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	double GradientLowValue = 0.0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolSelectionProperties
{
	GENERATED_BODY()

	// Mesh selection mode (vertex, edge face)
	UPROPERTY()
	EComponentSelectionMode ComponentSelectionMode = EComponentSelectionMode::Vertices;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolDisplayProperties
{
	GENERATED_BODY()

	// Vertex color display mode (Greyscale, Ramp, ... )
	UPROPERTY()
	EMeshVertexAttributePaintToolColorMode ColorMode = EMeshVertexAttributePaintToolColorMode::Greyscale;

	// Color ramp to use when the color mode is set to "Ramp"
	UPROPERTY()
	FLinearColorRamp ColorRamp;

	UPROPERTY()
	TArray<FLinearColorRamp> ColorRampPresets;

	// Used by UI customization 
	FLinearColorRamp GreyScaleColorRamp;
	FLinearColorRamp WhiteColorRamp;

	UPROPERTY()
	bool bShowValues = false;

	UPROPERTY()
	bool bShowValuesOnlySelected = false;

	UPROPERTY()
	float MinValue = 0;

	UPROPERTY()
	float MaxValue = 100;

	/** Should mesh boundary edges be shown */
	UPROPERTY()
	bool bShowBorders = true;

	/** Should mesh uv seam edges be shown */
	UPROPERTY()
	bool bShowUVSeams = true;

	/** Should mesh normal seam edges be shown */
	UPROPERTY()
	bool bShowNormalSeams = false;

	/** Should mesh tangent seam edges be shown */
	UPROPERTY()
	bool bShowTangentSeams = true;

	/** Should mesh color seam edges be shown */
	UPROPERTY()
	bool bShowColorSeams = true;

	/** Multiplier on edge thicknesses */
	UPROPERTY()
	float ThicknessScale = 0.5f;

	/** Color of mesh wireframe */
	UPROPERTY()
	FColor WireframeColor = FColor(128, 128, 128);

	/** Color of mesh boundary edges */
	UPROPERTY()
	FColor BoundaryEdgeColor = FColor(164, 73, 164);

	/** Color of mesh UV seam edges */
	UPROPERTY()
	FColor UVSeamColor = FColor(240, 160, 15);

	/** Color of mesh normal seam edges */
	UPROPERTY()
	FColor NormalSeamColor = FColor(128, 128, 240);

	/** Color of mesh tangent seam edges */
	UPROPERTY()
	FColor TangentSeamColor = FColor(64, 240, 240);

	/** Color of mesh color seam edges */
	UPROPERTY()
	FColor ColorSeamColor = FColor(46, 204, 113);

	/** Show polygroup boundary edges */
	UPROPERTY()
	bool bShowGroupBoundaries = true;

	/** Color of polygroup boundary edges */
	UPROPERTY()
	FColor GroupBoundaryColor = FColor(220, 100, 220);

	/** Depth bias used to slightly shift depth of lines */
	UPROPERTY()
	float DepthBias = 0.2f;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FMeshVertexAttributePaintToolMirrorProperties
{
	GENERATED_BODY()

	/* whether the mirror is enabled in brush mode */
	UPROPERTY()
	bool bEnableBrushMirroring = false;

	/* whether the mirror plane is visible or not */
	UPROPERTY()
	bool bMirrorPlaneWidgetVisible = true;

	/* if true, the mirror planes will be centered on the bounds of the object */
	UPROPERTY()
	bool bObjectSpace = true;

	/* if true, the mirror planes will be hidden when actively painting */
	UPROPERTY()
	bool bHideOnBrushStroke = false;

	UPROPERTY()
	TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X;

	UPROPERTY()
	EMeshVertexAttributePaintToolMirrorDirection MirrorDirection = EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, config = EditorSettings)
class UMeshVertexAttributePaintToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

	UE_API UMeshVertexAttributePaintToolProperties();

public:

	// Edit mode (brush,mesh...)
	UPROPERTY()
	EMeshVertexAttributePaintToolEditMode EditingMode = EMeshVertexAttributePaintToolEditMode::Brush;
	
	UPROPERTY(config)
	FMeshVertexAttributePaintToolBrushProperties BrushProperties;

	UPROPERTY()
	FMeshVertexAttributePaintToolGradientProperties GradientProperties;

	UPROPERTY()
	FMeshVertexAttributePaintToolSelectionProperties SelectionProperties;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolDisplayProperties DisplayProperties;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolMirrorProperties MirrorProperties;

	const FMeshVertexAttributePaintToolBrushConfig& GetBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode) const;
	void SetBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode, const FMeshVertexAttributePaintToolBrushConfig& BrushConfig);

	// When true (default), the brush radius is shared across all brush modes — every per-mode
	// SetBrushRadius / SaveBrushConfig propagates the new size to every BrushConfig* via
	// SetSharedBrushSize. When false, each mode remembers its own radius and a mode switch
	// (via LoadBrushConfig) pulls that mode's saved size. The UI toggle next to the radius
	// slider drives this and snap-canonicalises all configs to the current size on re-enable.
	UPROPERTY(config)
	bool bSyncBrushRadiusAcrossModes = true;

	void SetSharedBrushSize(float NewBrushSize)
	{
		BrushConfigAdd.BrushSize      = NewBrushSize;
		BrushConfigReplace.BrushSize  = NewBrushSize;
		BrushConfigMultiply.BrushSize = NewBrushSize;
		BrushConfigRelax.BrushSize    = NewBrushSize;
	}

	UPROPERTY(config)
	FMeshVertexAttributePaintToolBrushConfig BrushConfigAdd;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolBrushConfig BrushConfigReplace;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolBrushConfig BrushConfigMultiply;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolBrushConfig BrushConfigRelax;

	UPROPERTY(config)
	FMeshVertexAttributePaintToolRelaxBrushAdvancedConfig RelaxBrushAdvancedConfig;

	UPROPERTY(config)
	float SelectionAddStrength = 1.0f;

	UPROPERTY(config)
	float SelectionReplaceValue = 1.0f;

	UPROPERTY(config)
	float SelectionRelaxStrength = 0.5f;

	UPROPERTY(config)
	float SelectionPruneThreshold = 0.01f;

	UE_API void ResetColorRamp();

private:
	void CreateDefaultColorRamp();

};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FMeshVertexAttributePaintToolData
{
public:
	void Setup(UDynamicMesh* InToolDynamicMesh, int32 InInitialAttributeIndex);
	bool SetAttributeToEdit(int32 InNewAttributeIndex);


	void BeginChange();
	TUniquePtr<UE::MeshVertexAttributePaintToolBase::Private::FMeshChange> EndChange();
	void CancelChange();

	bool IsValid() const;

	UE_API float GetValue(int32 VertexIdx) const;
	float GetAverageValue(const TArray<int32>& Vertices) const;

	void SetValue(int32 VertexIdx, float Value);

	// Bulk-read every vertex's current value into OutSnapshot, sized to MaxVertexID.
	// Holes (invalid vertex IDs) are written as 0. Performed in a single ProcessMesh
	// call so the cost is one mesh lock plus one linear pass over the weight layer.
	UE_API void SnapshotAllValues(TArray<float>& OutSnapshot) const;

	// Sync PreChangeWeights to the current mesh attribute values (the "current weights").
	// Called at every transaction boundary (EndChange/CancelChange/AfterModify) and at
	// attribute swap so the next transaction's stamps read against a baseline matching
	// the current mesh state.
	UE_API void SyncPreChangeWeightsToCurrentWeights();

	// Stable baseline for gated-mode stamp reads. Frozen across an entire transaction
	// so multi-call sequences (e.g. slider drag) read the same pre-transaction value
	// for a given vertex. Returns 0 if invalid or vertex index is out of range.
	UE_API float GetPreChangeValue(int32 VertexIdx) const;

	int32 GetNumVerts() const { return PreChangeWeights.Num(); }

private:

	TUniquePtr<UE::Geometry::FDynamicMeshChangeTracker> ActiveWeightEditChangeTracker;

	TWeakObjectPtr<UDynamicMesh> ToolDynamicMesh;

	int32 AttributeIndex = INDEX_NONE;

	TArray<float> PreChangeWeights;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Editor tool to paint a vertex attribute as vertex colors
 */
UCLASS(MinimalAPI)
class UMeshVertexAttributePaintToolBase : public UMeshSculptToolBase
{
	GENERATED_BODY()
public:
	UE_API UMeshVertexAttributePaintToolBase();
	UE_API virtual ~UMeshVertexAttributePaintToolBase();

protected:
	UE_API virtual bool SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex);
	UE_API virtual void CommitToolMesh(FDynamicMesh3& InToolMesh);
	UE_API virtual void UpdatePreview(const TSet<int32>* TrianglesToUpdate = nullptr, const TArray<int32>* VerticesToUpdate = nullptr);

	UE_API bool SetAttributeToPaint(int32 InNewAttributeIndex);
	UE_API void RebuildOctree();
	
public:
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& Ray)  override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	UE_API void SetEditingMode(EMeshVertexAttributePaintToolEditMode Editmode);
	UE_API EMeshVertexAttributePaintToolEditMode GetEditingMode() const;

	UE_API void SetBrushMode(EMeshVertexAttributePaintToolEditOperation BrushMode);
	UE_API bool IsInBrushMode() const;

	UE_API void SetBrushAreaMode(EMeshVertexPaintBrushAreaType BrushAreaMode);
	UE_API bool IsVolumetricBrush() const;

	UE_API virtual void SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior) override;
	

	//~ UObject interface

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:
	/** properties of the tool */
	UPROPERTY()
	TObjectPtr<UMeshVertexAttributePaintToolProperties> ToolProperties;

	UPROPERTY()
	TObjectPtr<UMeshVertexAttributePaintToolPaintBrushOpProps> PaintBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UMeshVertexAttributePaintToolSmoothBrushOpProps> SmoothBrushOpProperties;

	UE_API bool HaveVisibilityFilter() const;
	UE_API void ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices);
	UE_API void ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer);

	using UMeshSculptToolBase::GetCurrentBrushRadius;
	UE_API float GetBrushAdaptiveSize() const;
	UE_API float GetBrushMinRadius() const;
	UE_API float GetBrushMaxRadius() const;
	UE_API void SetBrushRadius(float NewRadius);
	UE_API void SetBrushAdaptiveSize(float AdaptiveSize);

	UE_API float GetBrushFalloff() const;
	UE_API void SetBrushFalloff(float NewFalloff);

	UE_API void SetColorMode(EMeshVertexAttributePaintToolColorMode NewColorMode);

	UE_API bool HasSelection() const;
	UE_API void GrowSelection() const;
	UE_API void ShrinkSelection() const;
	UE_API void InvertSelection() const;
	UE_API void FloodSelection() const;
	UE_API void SelectBorder() const;
	UE_API void SetComponentSelectionMode(EComponentSelectionMode NewMode);

	UE_API void CopyAverageFromSelectionToClipboard();
	UE_API void PasteValueToSelectionFromClipboard();
	UE_API void PruneSelection(float Threshold);

	// selection operations
	UE_API void ApplyValueToSelection(EMeshVertexAttributePaintToolEditOperation Operation, float InValue, bool bWithTransaction = true);
	UE_API void MirrorValues();

	UE_API const TArray<int32>& GetSelectedVertices() const;

	UE_API virtual void SetFocusInViewport() const;

public:
	static UE_API FInputChord GreyscaleDisplayInputChord;
	static UE_API FInputChord ColorMapDisplayInputChord;
	static UE_API FInputChord MaterialColorDisplayInputChord;

	static UE_API FInputChord VerticesSelectionModeInputChord;
	static UE_API FInputChord EdgesSelectionModeInputChord;
	static UE_API FInputChord FacesSelectionModeInputChord;

	static UE_API FInputChord GrowSelectionInputChord;
	static UE_API FInputChord ShrinkSelectionInputChord;
	static UE_API FInputChord FloodSelectionInputChord;

	static UE_API FInputChord InvertMirrorDirectionInputChord;
	static UE_API FInputChord MirrorValuesInputChord;

	static UE_API FInputChord CopyAverageValueInputChord;
	static UE_API FInputChord PasteAverageValueInputChord;

	static UE_API FInputChord ToggleValuePickerInputChord;

protected:
	UPROPERTY()
	TObjectPtr<UDynamicMesh> ToolDynamicMesh;
	
	// UMeshSculptToolBase API
	UE_API virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() override;
	virtual FDynamicMesh3* GetBaseMesh()  override { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const  override { check(false); return nullptr; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual void UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay) override;

	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnEndStroke() override;
	UE_API virtual void OnCancelStroke() override;

	UE_API void OnColorRampChanged(TArray<FRichCurve*> Curves);

	void SaveBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode);
	void LoadBrushConfig(EMeshVertexAttributePaintToolEditOperation BrushMode);

	virtual bool SharesBrushPropertiesChanges() const override { return false; }
	UE_API virtual void InitializeIndicator() override;
	// end UMeshSculptToolBase API

	UE_API void EnableValuePicker(bool bEnable);
	UE_API bool IsValuePickerEnabled() const;

	UE_API void EnableBrushMirroring(bool bEnable);
	UE_API bool IsBrushMirroringEnabled() const;

	UE_API void SetMirrorPlaneWidgetVisible(bool bVisible);
	UE_API bool IsMirrorPlaneWidgetVisible();

	bool IsBrushMirroringInObjectSpace() const;

	UE_API void SetMirrorAxis(EAxis::Type InAxis);
	UE_API EAxis::Type GetMirrorAxis() const;

	void InitializeMirrorIndicator();
	void UpdateMirrorIndicator();
	void UpdateMirrorPlaneMesh();
	UE::Geometry::FFrame3d ComputeMirroredFrame(const UE::Geometry::FFrame3d& InLocalSpaceFrame) const;
	void OnMirrorCommand();

	UE_API void SetMeshSelectionTool(EMeshSelectorTool SelectorTool);
	UE_API EMeshSelectorTool GetMeshSelectionTool() const;

protected:

	// 
	// Gradient Support
	//
	FToolDataVisualizer GradientSelectionRenderer;

	UE::Geometry::FGroupTopologySelection LowValueGradientVertexSelection;
	UE::Geometry::FGroupTopologySelection HighValueGradientVertexSelection;

	UE_API void OnSelectionModified();

	//
	// Internals
	//

protected:

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;
	UE::Geometry::FAxisAlignedBox3d MeshLocalBounds;

	UPROPERTY()
	TObjectPtr<UToolMeshSelector> MeshSelector = nullptr;

	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> MirrorBrushIndicator = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> MirrorBrushIndicatorMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> MirrorPlaneMesh = nullptr;

	FSculptBrushStamp MirrorCurrentStamp;
	int32 MirrorBrushTriangleID = INDEX_NONE;

	UE_API float GetCurrentWeightValueUnderBrush() const;
	FVector3d CurrentBaryCentricCoords;
	UE_API int32 GetBrushNearestVertex() const;
	UE_API int32 GetBrushNearestVertexAccurate() const;
	UE_API float GetAverageWeightValueFromSelection() const;

	// Always applies a stamp on mouse press
	bool bFirstStampPending = false;
	TArray<int> NormalsBuffer;

	TArray<int> TempROIBuffer;
	TArray<int> VertexROI;
	TArray<bool> VisibilityFilterBuffer;
	TSet<int> VertexSetBuffer;
	TSet<int> TriangleROI;
	UE_API void UpdateROI(const FSculptBrushStamp& CurrentStamp, int32 BrushTriangleID);

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API bool ApplyStamp(const FSculptBrushStamp& InStamp);

	// Per-Transaction/Stroke bookkeeping (pimpl). 
	TSharedRef<UE::MeshVertexAttributePaintToolBase::Private::FStrokeAccumulator> StrokeAccumulator;

	// Per-edge cotangent weights for the relax operation. 
	TArray<double> CotangentEdgeWeights;

	UE_API float ComputeStampFalloffAtVertex(const FSculptBrushStamp& Stamp, const FVector3d& VertexPos);

	UE::Geometry::FDynamicMeshOctree3 Octree;
	UE_API void FullUpdateOctree();

	UE_API void UpdateBrushType(EMeshVertexAttributePaintToolEditOperation EditOperation);
	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	bool bPendingPickWeight = false;
	bool bBrushIsHoverMesh = false;
	bool bIsMirrorHiddenBecauseOfBrushStroke = false;

	TArray<int32> ROITriangleBuffer;
	TArray<double> ROIWeightValueBuffer;
	UE_API bool SyncMeshWithWeightBuffer();
	UE_API bool SyncWeightBufferWithMesh();

	friend class UE::MeshVertexAttributePaintToolBase::FVertexAttributePaintToolDetailCustomization;
	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void CancelChange();

	TArray<int32> UVSeamEdges;
	TArray<int32> NormalSeamEdges;
	UE_API void PrecomputeSeamEdges();

	FMeshVertexAttributePaintToolData VertexData;

protected:
	virtual bool ShowWorkPlane() const override { return false; }

	friend class UMeshAttributePaintToolV2Builder;

	bool bAnyChangeMade = false;

	/** update the vertex color of the mesh from the attribute */

	void UpdateVertexColorOverlay(const TSet<int32>* TrianglesToUpdate = nullptr);

	static constexpr int32 PaintBrushId = 0;
	static constexpr int32 SmoothBrushId = 1;

private:
	// this structure holds information for mirroring 
	struct FMirrorData
	{
	public:
		// lazily updates the mirror data tables for the current skeleton/mesh/mirror plane
		void EnsureMirrorDataIsUpdated(const FDynamicMesh3& Mesh, EAxis::Type InMirrorAxis, EMeshVertexAttributePaintToolMirrorDirection InMirrorDirection, bool bInObjectSpace);

		void FindMirroredIndices(const FDynamicMesh3& Mesh, const TArray<int32>& SelectedVertices, TArray<int32>& OutVerticesToUpdate);

	private:
		// return true if the point lies on the TARGET side of the mirror plane
		bool IsPointOnTargetMirrorSide(const FVector& InPoint) const;

		EAxis::Type Axis = EAxis::X;
		EMeshVertexAttributePaintToolMirrorDirection Direction = EMeshVertexAttributePaintToolMirrorDirection::PositiveToNegative;
		bool bObjectSpace = true;
		FVector PlaneOffsets = FVector{ 0, 0, 0 };
		TMap<int32, int32> VertexMap; // <Target, Source>
	};
	FMirrorData MirrorData;
};




#undef UE_API

