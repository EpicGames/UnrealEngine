// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMeshBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "ShapeSprayTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

/**
 * UMeshSurfacePointMeshEditingToolBuilder override for UShapeSprayTool
 */
UCLASS(MinimalAPI, Transient)
class UShapeSprayToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Settings UObject for UShapeSprayTool. 
 */
UCLASS(MinimalAPI, Transient)
class UShapeSprayToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UE_API UShapeSprayToolProperties();

	UPROPERTY(EditAnywhere, Category = Options)
	FLinearColor Color;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bRandomColor;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Speed", UIMin = "0.01", UIMax = "1.0"))
	float DropSpeed;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Shape Size", UIMin = "1.0", UIMax = "30.0"))
	float ObjectSize;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Repeat Per Stamp"))
	int NumSplats;

	UPROPERTY(EditAnywhere, Category = Options)
	TObjectPtr<UMaterialInterface> Material = nullptr;
};



/**
 * UShapeSprayTool is a brush-based tool that generates random points on the
 * target surface within the brush radius, and then creates small meshes
 * at those points. The accumulated meshes are appended and can
 * be emitted as a new StaticMeshComponent on Accept.
 */
UCLASS(MinimalAPI, Transient)
class UShapeSprayTool : public UDynamicMeshBrushTool
{
	GENERATED_BODY()

public:
	UE_API UShapeSprayTool();

	UE_API virtual void SetWorld(UWorld* World);

	// UInteractiveTool API

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// UMeshSurfacePointTool API
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;

protected:
	UPROPERTY()
	TObjectPtr<UShapeSprayToolProperties> Settings;

	// small meshes are accumulated here
	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> AccumMeshComponent;


protected:
	UWorld* TargetWorld;

	UE::Geometry::FDynamicMesh3 ShapeMesh;
	UE_API void UpdateShapeMesh();
	UE_API void SplatShape(const UE::Geometry::FFrame3d& LocalFrame, double Scale, UE::Geometry::FDynamicMesh3* TargetMesh);
	TArray<int> VertexMap;

	FRandomStream Random;

	UE_API virtual void EmitResult();
};

#undef UE_API
