// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicSubmesh3.h"
#include "MeshVertexAttributePaintToolBase.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "SkeletalMesh/IHotkeyHintProvider.h"
#include "SkeletalMesh/ISkeletalMeshGeometryIsolationAwareTool.h"

#include "SkeletalMeshVertexAttributePaintTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class USkeletalMeshEditorContextObjectBase;
class USkeletalMeshVertexAttributePaintTool;

/**
 * Tool builder for USkeletalMeshVertexAttributePaintTool.
 */
UCLASS(MinimalAPI)
class USkeletalMeshVertexAttributePaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Property set for USkeletalMeshVertexAttributePaintTool.
 *
 * Identical shape to UMeshAttributePaintToolV2Properties — the user picks a weight layer
 * from a dropdown and brushes into it.
 */
UCLASS(MinimalAPI)
class USkeletalMeshVertexAttributePaintToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Selected Attribute", GetOptions = GetAttributeNames, NoResetToDefault))
	FString Attribute;

	UFUNCTION()
	const TArray<FString>& GetAttributeNames() { return Attributes; };

	TArray<FString> Attributes;

public:
	UE_API void Initialize(const TArray<FName>& AttributeNames, bool bInitialize = false);

	UE_API int32 GetSelectedAttributeIndex();
};

/**
 * Skeletal-mesh-aware vertex attribute paint tool.
 *
 * UX is identical to UMeshAttributePaintToolV2: pick a weight layer, brush, accept.
 * Internally the tool reads pose and isolated-triangle state from
 * USkeletalMeshEditorContextObjectBase, paints in posed/submesh space, and commits
 * results back to the unposed full-mesh weight layer. Brush mirroring suppresses
 * posing so the mirror plane operates on the unposed mesh.
 */
UCLASS(MinimalAPI)
class USkeletalMeshVertexAttributePaintTool :
	public UMeshVertexAttributePaintToolBase,
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IHotkeyHintProvider
{
	GENERATED_BODY()

protected:
	UE_API virtual bool SetupToolMesh(UE::Geometry::FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex) override;
	UE_API virtual void CommitToolMesh(UE::Geometry::FDynamicMesh3& InToolMesh) override;
	UE_API virtual void UpdatePreview(const TSet<int32>* TrianglesToUpdate = nullptr, const TArray<int32>* VerticesToUpdate = nullptr) override;

public:
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	// ISkeletalMeshGeometryIsolationAwareTool
	UE_API virtual bool IsInputIsolationValidOnOutput() const override;

	// IHotkeyHintProvider
	UE_API virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;

protected:
	UE_API const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	UE_API TMap<FName, float> GetMorphTargetWeights();
	UE_API void ToggleBoneManipulation(bool bEnable);

	UE_API void HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload);

	UE_API void PosePreviewMesh(
		const TArray<FTransform>& ComponentSpaceTransforms,
		const TMap<FName, float>& MorphTargetWeights,
		const TSet<int32>* TrianglesToUpdate = nullptr,
		const TArray<int32>* VerticesToUpdate = nullptr);

protected:
	UPROPERTY()
	TObjectPtr<USkeletalMeshVertexAttributePaintToolProperties> AttribProps;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;

	SkeletalMeshToolsHelper::FPoseChangeDetector PoseChangeDetector;

	// Held only when isolation is active. ToolDynamicMesh becomes the submesh in that case,
	// so we keep a separate full unposed copy for both pose-preview source and commit target.
	// Without isolation this stays unset and ToolDynamicMesh serves both roles directly.
	TOptional<UE::Geometry::FDynamicMesh3> FullUnposedMesh;

	TOptional<UE::Geometry::FDynamicSubmesh3> IsolationSubmesh;

	TArray<FTransform> ComponentSpaceTransformsRefPose;
	TArray<FTransform> DefaultComponentSpaceBoneTransforms;

	TValueWatcher<int32> SelectedAttributeWatcher;

	bool bFastDeformPreviewMesh = false;
	bool bRebuildOctree = false;
	bool bMeshSelectorNeedsUpdate = false;
};

#undef UE_API
