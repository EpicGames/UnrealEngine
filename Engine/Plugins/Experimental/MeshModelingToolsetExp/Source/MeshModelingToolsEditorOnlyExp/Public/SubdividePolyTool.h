// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Operations/SubdividePoly.h"
#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "ModelingOperators.h"
#include "SingleSelectionTool.h"
#include "Components/DynamicMeshComponent.h"
#include "MeshOpPreviewHelpers.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "SubdividePolyTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API

class USubdividePolyTool;
class UPreviewGeometry;

/**
 * Tool builder
 */
UCLASS(MinimalAPI)
class USubdividePolyToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

/**
 * Properties
 */

UCLASS(MinimalAPI)
class USubdividePolyToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** Number of iterations/levels of subdivision to perform */
	UPROPERTY(EditAnywhere, Category=Settings, meta = (UIMin = "1", ClampMin = "1", Delta = 1, LinearDeltaSensitivity = 50))
	int SubdivisionLevel = 3;

	UPROPERTY(EditAnywhere, Category = Settings)
	ESubdivisionScheme SubdivisionScheme = ESubdivisionScheme::CatmullClark;

	// How to treat mesh boundaries
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "SubdivisionScheme != ESubdivisionScheme::Bilinear || bOverriddenSubdivisionScheme"))
	ESubdivisionBoundaryScheme BoundaryScheme = ESubdivisionBoundaryScheme::SmoothCorners;

	UPROPERTY(EditAnywhere, Category=Settings)
	ESubdivisionOutputNormals NormalComputationMethod = ESubdivisionOutputNormals::Generated;

	UPROPERTY(EditAnywhere, Category=Settings, meta = (DisplayName = "UV Computation Method"))
	ESubdivisionOutputUVs UVComputationMethod = ESubdivisionOutputUVs::Interpolated;

	/** Assign a new PolyGroup ID to each newly created face */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "New PolyGroups"))
	bool bNewPolyGroups = false;

	/** Display each PolyGroup with an auto-generated color */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bRenderGroups = true;

	/** Display the mesh corresponding to Subdivision Level 0 */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bRenderCage = true;

	/** When using the group topology for subdivision, whether to add extra corners at sharp group edge bends on mesh boundaries. Note: We cannot add extra corners on non-boundary group edges, as this would create non-manifold geometry on subdivision. */
	UPROPERTY(EditAnywhere, Category = TopologyOptions, meta = (DisplayName = "Add Extra Corners on Boundary", EditCondition = "SubdivisionScheme != ESubdivisionScheme::Loop"))
	bool bAddExtraCorners = true;

	/** How acute an angle between two edges needs to be to add an extra corner there when Add Extra Corners is true. */
	UPROPERTY(EditAnywhere, Category = TopologyOptions, meta = (ClampMin = "0", ClampMax = "180", EditCondition = "SubdivisionScheme != ESubdivisionScheme::Loop && bAddExtraCorners"))
	double ExtraCornerAngleThresholdDegrees = 135;

	/** Shows whether the current subdivision scheme is overridden to be "Loop" because the group topology is unsuitable. */
	UPROPERTY(VisibleAnywhere, Category = Settings, AdvancedDisplay, meta = (TransientToolProperty))
	bool bOverriddenSubdivisionScheme = false;
};


/**
 * Tool actual
 */
UCLASS(MinimalAPI)
class USubdividePolyTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnTick(float DeltaTime) override;

protected:

	friend class USubdividePolyOperatorFactory;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<USubdividePolyToolProperties> Properties = nullptr;

	// Input mesh
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	TSharedPtr<UE::Geometry::FGroupTopology> Topology;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;

	bool bPreviewGeometryNeedsUpdate;
	UE_API void CreateOrUpdatePreviewGeometry();

	// Determine if the mesh group topology can be used for Catmull-Clark or Bilinear subdivision. If not, we can only 
	// Loop subdivision on the original triangle mesh.
	UE_API bool CheckGroupTopology(FText& Message);

	UE_API void CapSubdivisionLevel(ESubdivisionScheme Scheme, int DesiredLevel);

	double ExtraCornerDotProductThreshold = 2;
	UE_API ESubdivisionScheme GetSubdivisionSchemeToUse();
	FText OverriddenSchemeMessage;
	FText CappedSubdivisionMessage;
	UE_API void UpdateDisplayedMessage();
};

#undef UE_API
