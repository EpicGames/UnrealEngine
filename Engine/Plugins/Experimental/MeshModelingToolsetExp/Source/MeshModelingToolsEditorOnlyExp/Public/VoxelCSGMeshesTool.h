// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseVoxelTool.h"


#include "VoxelCSGMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLYEXP_API



/**  */
UENUM()
enum class EVoxelCSGOperation : uint8
{
	/** Subtracts the first object from the second */
	DifferenceAB = 0 UMETA(DisplayName = "A - B"),

	/** Subtracts the second object from the first */
	DifferenceBA = 1 UMETA(DisplayName = "B - A"),

	/** intersection of two objects */
	Intersect = 2 UMETA(DisplayName = "Intersect"),

	/** union of two objects */
	Union = 3 UMETA(DisplayName = "Union"),

};


/**
 * Standard properties of the Voxel CSG operation
 */
UCLASS(MinimalAPI)
class UVoxelCSGMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The type of operation  */
	UPROPERTY(EditAnywhere, Category = BooleanOptions)
	EVoxelCSGOperation Operation = EVoxelCSGOperation::DifferenceAB;

	/** The size of the geometry bounding box major axis measured in voxels.*/
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "8", UIMax = "1024", ClampMin = "8", ClampMax = "1024"))
	int32 VoxelCount = 128;

	/** Remeshing adaptivity, prior to optional simplification */
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float MeshAdaptivity = 0.01f;

	/** Offset when remeshing, note large offsets with high voxels counts will be slow.  Hidden because this duplicates functionality of the voxel offset tool */
	UPROPERTY()
	float OffsetDistance = 0.0f;

	/** Automatically simplify the result of voxel-based merge.*/
	UPROPERTY(EditAnywhere, Category = VoxelSettings)
	bool bAutoSimplify = false;
};

UCLASS(MinimalAPI)
class UVoxelCSGMeshesTool : public UBaseVoxelTool
{
	GENERATED_BODY()

public:

	UVoxelCSGMeshesTool() {}

protected:

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UVoxelCSGMeshesToolProperties> CSGProps;

	virtual bool KeepCollisionFrom(int32 TargetIdx) const override
	{
		if (CSGProps->Operation == EVoxelCSGOperation::DifferenceAB)
		{
			return TargetIdx == 0;
		}
		else if (CSGProps->Operation == EVoxelCSGOperation::DifferenceBA)
		{
			return TargetIdx == 1;
		}
		return true;
	}

};




UCLASS(MinimalAPI)
class UVoxelCSGMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:

	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UVoxelCSGMeshesTool>(SceneState.ToolManager);
	}

	virtual TOptional<int32> MaxComponentsSupported() const override { return 2; }
	virtual int32 MinComponentsSupported() const override { return 2; }
};

#undef UE_API
