// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshModelingToolsExp : ModuleRules
{
	public MeshModelingToolsExp(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[]
			{
				// ... add public include paths required here ...
			}
		);

		PrivateIncludePaths.AddRange(
			new string[]
			{
				// ... add other private include paths required here ...
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"GeometryCore",
				"GeometryFramework",
				"DynamicMesh",
				"MeshConversion",
				"MeshDescription",
				"MeshModelingTools",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"ModelingComponents",
				"ModelingOperators"
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"GeometryAlgorithms",
				"RenderCore",
				"InputCore",
				"PhysicsCore",
				"SlateCore",
				// ... add private dependencies that you statically link with here ...
			}
		);

		if (Target.bCompileAgainstEditor) // #if WITH_EDITOR
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ModelingComponentsEditorOnly"
				});
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}
