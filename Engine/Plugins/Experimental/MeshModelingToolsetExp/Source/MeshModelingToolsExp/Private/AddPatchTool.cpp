// Copyright Epic Games, Inc. All Rights Reserved.

#include "AddPatchTool.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "InteractiveToolManager.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolSceneQueriesUtil.h"

#include "MeshDescriptionBuilder.h"
#include "Generators/RectangleMeshGenerator.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Drawing/MeshDebugDrawing.h"
#include "DynamicMesh/MeshNormals.h"

#include "Async/ParallelFor.h"

#include "DynamicMeshEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddPatchTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UAddPatchTool"

/*
 * ToolBuilder
 */
bool UAddPatchToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UAddPatchToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UAddPatchTool* NewTool = NewObject<UAddPatchTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}

/*
 * Tool
 */
UAddPatchToolProperties::UAddPatchToolProperties()
{
	Width = 10000;
	Subdivisions = 50;
	Rotation = 0;
	Shift = 0.0;
}

void UAddPatchTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UAddPatchTool::Setup()
{
	USingleClickTool::Setup();

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);

	ShapeSettings = NewObject<UAddPatchToolProperties>(this);
	AddToolPropertySource(ShapeSettings);
	ShapeSettings->RestoreProperties(this);

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	AddToolPropertySource(MaterialProperties);
	MaterialProperties->RestoreProperties(this);

	// create preview mesh object
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(TargetWorld, FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);
	PreviewMesh->SetVisible(false);
	PreviewMesh->SetMaterial(MaterialProperties->Material.Get());
	GeneratePreviewBaseMesh();

	WorldBounds = FBox(ForceInit);
	for (const ULevel* Level : TargetWorld->GetLevels())
	{
		for (AActor* Actor : Level->Actors)
		{
			if (Actor)
			{
				FBox ActorBox = Actor->GetComponentsBoundingBox(true);
				if (ActorBox.IsValid)
				{
					WorldBounds += ActorBox;
				}
			}
		}
	}
	float WorldDiag = WorldBounds.GetSize().Size();
	if (ShapeSettings->Width > WorldDiag * 0.25)
	{
		ShapeSettings->Width = WorldDiag * 0.25;
	}

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartAddPatchTool", "Position the Patch by moving the mouse over the scene. Drop a new instance by Left-clicking."),
		EToolMessageLevel::UserNotification);
}


void UAddPatchTool::Shutdown(EToolShutdownType ShutdownType)
{
	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	ShapeSettings->SaveProperties(this);
	MaterialProperties->SaveProperties(this);
}


void UAddPatchTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}



void UAddPatchTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
	PreviewMesh->SetMaterial(MaterialProperties->Material.Get());
	GeneratePreviewBaseMesh();
}



FInputRayHit UAddPatchTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	return FInputRayHit(0.0f);		// always hit in hover 
}

void UAddPatchTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
}

bool UAddPatchTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
	return true;
}

void UAddPatchTool::OnEndHover()
{
	// do nothing
}



void UAddPatchTool::OnTick(float DeltaTime)
{
	if (bPreviewValid == false)
	{
		UpdatePreviewMesh();
		PreviewMesh->SetVisible(true);
		bPreviewValid = true;
	}
}



void UAddPatchTool::UpdatePreviewPosition(const FInputDeviceRay& DeviceClickPos)
{
	FRay ClickPosWorldRay = DeviceClickPos.WorldRay;

	// cast ray into scene
	FHitResult Result;
	bool bHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, ClickPosWorldRay);
	if (bHit)
	{
		ShapeFrame = FFrame3f((FVector3f)Result.ImpactPoint, (FVector3f)Result.ImpactNormal);
		//ShapeFrame.ConstrainedAlignPerpAxes();
	}

	// clear rotation
	ShapeFrame.Rotation = FQuaternionf::Identity();

	// rotate around up axis
	if (ShapeSettings->Rotation != 0)
	{
		ShapeFrame.Rotate(FQuaternionf(ShapeFrame.Z(), ShapeSettings->Rotation, true));
	}

	if (bHit == false)
	{
		PreviewMesh->SetVisible(false);
	}
	else 
	{
		bPreviewValid = false;
	}
}


void UAddPatchTool::UpdatePreviewMesh()
{
	FVector3d Direction = FVector3d(0, 0, 1.0);
	float WorldMaxHeight = WorldBounds.Max.Z;
	float WorldMinHeight = WorldBounds.Min.Z;
	float WorldHeight = WorldMaxHeight - WorldMinHeight;

	FDynamicMesh3 Projected(*BaseMesh);

	FTransform MoveTransform = ShapeFrame.ToFTransform();

	FFrame3d RayFrame(MoveTransform);
	RayFrame.Origin.Z = WorldMaxHeight + 100.0f;

	TSet<int> Misses;
	FCriticalSection MissesLock;


	int NumVerts = Projected.MaxVertexID();
	ParallelFor(NumVerts, [this, &Misses, &MissesLock, &Projected, RayFrame, WorldMinHeight, Direction, MoveTransform](int vid)
	{
		FVector3d Pos = Projected.GetVertex(vid);
		Pos = RayFrame.FromFramePoint(Pos);
		FVector RayStart = (FVector)Pos;
		FVector RayEnd = RayStart; RayEnd.Z = WorldMinHeight;

		FHitResult Result;
		bool bHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, RayStart, RayEnd);
		if (bHit)
		{
			FVector3d HitPoint = (FVector3d)Result.ImpactPoint + (double)ShapeSettings->Shift * Direction;
			FVector HitPosWorld = (FVector)HitPoint;
			FVector HitPosLocal = MoveTransform.InverseTransformPosition(HitPosWorld);
			Projected.SetVertex(vid, (FVector3d)HitPosLocal);
		}
		else
		{
			MissesLock.Lock();
			Misses.Add(vid);
			MissesLock.Unlock();
		}
	});

	TArray<int> RemoveTris;
	for (int tid : Projected.TriangleIndicesItr())
	{
		FIndex3i Tri = Projected.GetTriangle(tid);
		if (Misses.Contains(Tri.A) || Misses.Contains(Tri.B) || Misses.Contains(Tri.C))
		{
			RemoveTris.Add(tid);
		}
	}

	FMeshNormals::QuickComputeVertexNormals(Projected);

	FDynamicMeshEditor Editor(&Projected);
	Editor.RemoveTriangles(RemoveTris, false);
	PreviewMesh->UpdatePreview(&Projected);
	PreviewMesh->SetTransform(MoveTransform);

	PreviewMesh->SetVisible(true);
}



void UAddPatchTool::GeneratePreviewBaseMesh()
{
	BaseMesh = MakeUnique<FDynamicMesh3>();
	GeneratePlane(BaseMesh.Get());

	if (MaterialProperties->UVScale != 1.0 || MaterialProperties->bWorldSpaceUVScale)
	{
		FDynamicMeshEditor Editor(BaseMesh.Get());
		float WorldUnitsInMetersFactor = MaterialProperties->bWorldSpaceUVScale ? .01f : 1.0f;
		Editor.RescaleAttributeUVs(MaterialProperties->UVScale * WorldUnitsInMetersFactor, MaterialProperties->bWorldSpaceUVScale);
	}

	// recenter mesh
	FAxisAlignedBox3d Bounds = BaseMesh->GetBounds(true);
	FVector3d TargetOrigin = Bounds.Center();
	for (int vid : BaseMesh->VertexIndicesItr())
	{
		FVector3d Pos = BaseMesh->GetVertex(vid);
		Pos -= TargetOrigin;
		BaseMesh->SetVertex(vid, Pos);
	}

	PreviewMesh->UpdatePreview(BaseMesh.Get());
}


void UAddPatchTool::OnClicked(const FInputDeviceRay& DeviceClickPos)
{
	const FDynamicMesh3* CurMesh = PreviewMesh->GetPreviewDynamicMesh();
	FTransform3d CurTransform(PreviewMesh->GetTransform());
	UMaterialInterface* Material = PreviewMesh->GetMaterial();
	GetToolManager()->BeginUndoTransaction(LOCTEXT("AddPatchToolTransactionName", "Add Patch Mesh"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = (FTransform)CurTransform;
	NewMeshObjectParams.BaseName = TEXT("Patch");
	NewMeshObjectParams.Materials.Add(Material);
	NewMeshObjectParams.SetMesh(CurMesh);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		// select newly-created object
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();
}





void UAddPatchTool::GeneratePlane(FDynamicMesh3* OutMesh)
{
	FRectangleMeshGenerator RectGen;
	RectGen.Width = RectGen.Height = ShapeSettings->Width;
	RectGen.WidthVertexCount = RectGen.HeightVertexCount = ShapeSettings->Subdivisions + 2;
	RectGen.Generate();
	OutMesh->Copy(&RectGen);
}



#undef LOCTEXT_NAMESPACE

