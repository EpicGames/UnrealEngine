// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlignObjectsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshAdapterTransforms.h"
#include "DynamicMesh/MeshTransforms.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include "Components/PrimitiveComponent.h"
#include "CollisionQueryParams.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlignObjectsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UAlignObjectsTool"

/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UAlignObjectsToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(
		UPrimitiveComponentBackedTarget::StaticClass()
		);
	return TypeRequirements;
}

bool UAlignObjectsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) >= 2;
}

UMultiSelectionMeshEditingTool* UAlignObjectsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UAlignObjectsTool>(SceneState.ToolManager);
}

/*
 * Tool
 */

UAlignObjectsTool::UAlignObjectsTool()
{
}


void UAlignObjectsTool::Setup()
{
	UInteractiveTool::Setup();

	UClickDragInputBehavior* ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
	ClickDragBehavior->Initialize(this);
	AddInputBehavior(ClickDragBehavior);

	AlignProps = NewObject<UAlignObjectsToolProperties>();
	AlignProps->RestoreProperties(this);
	AddToolPropertySource(AlignProps);

	Precompute();

	SetToolDisplayName(LOCTEXT("ToolName", "Align"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "This Tool aligns the Pivots or Bounding Boxes of the input Objects."),
		EToolMessageLevel::UserNotification);
}



void UAlignObjectsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	AlignProps->SaveProperties(this);

	// reset positions even on accept, because we need them to be updated below
	for (const FAlignInfo& Align : ComponentInfo)
	{
		Align.Component->SetWorldTransform(Align.SavedTransform);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("AlignObjectsTransactionName", "Align Objects"));

		for (const FAlignInfo& Align : ComponentInfo)
		{
			Align.Component->Modify();
		}
		bAlignDirty = true;
		UpdateAlignment();

		GetToolManager()->EndUndoTransaction();
	}
}




void UAlignObjectsTool::OnTick(float DeltaTime)
{
	if (bAlignDirty)
	{
		UpdateAlignment();
		bAlignDirty = false;
	}
}

void UAlignObjectsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UAlignObjectsTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	bAlignDirty = true;
}


void UAlignObjectsTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
}




void UAlignObjectsTool::Precompute()
{
	PivotBounds = FAxisAlignedBox3d::Empty();
	CombinedBounds = FAxisAlignedBox3d::Empty();
	AveragePivot = FVector3d::Zero();

	for (TObjectPtr<UToolTarget>& Target : Targets)
	{
		FAlignInfo AlignInfo;
		AlignInfo.Component = UE::ToolTarget::GetTargetComponent(Target);
		AlignInfo.SavedTransform = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Target);
		AlignInfo.WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
		AlignInfo.WorldBounds = FAxisAlignedBox3d(AlignInfo.Component->Bounds.GetBox());
		AlignInfo.WorldPivot = AlignInfo.WorldTransform.TransformPosition(FVector3d::Zero());
		ComponentInfo.Add(AlignInfo);

		CombinedBounds.Contain(AlignInfo.WorldBounds);
		PivotBounds.Contain(AlignInfo.WorldPivot);
		AveragePivot += AlignInfo.WorldPivot;
	}

	AveragePivot /= (double)ComponentInfo.Num();

	bAlignDirty = true;
}


static FVector3d GetBoxPoint(const FAxisAlignedBox3d& Box, EAlignObjectsBoxPoint BoxPoint)
{
	FVector3d Point = Box.Center();
	switch(BoxPoint)
	{
	default:
	case EAlignObjectsBoxPoint::Center:
		break;

	case EAlignObjectsBoxPoint::Top:
		Point.Z = Box.Max.Z; break;

	case EAlignObjectsBoxPoint::Bottom:
		Point.Z = Box.Min.Z; break;

	case EAlignObjectsBoxPoint::Left:
		Point.Y = Box.Min.Y; break;

	case EAlignObjectsBoxPoint::Right:
		Point.Y = Box.Max.Y; break;

	case EAlignObjectsBoxPoint::Front:
		Point.X = Box.Min.X; break;

	case EAlignObjectsBoxPoint::Back:
		Point.X = Box.Max.X; break;

	case EAlignObjectsBoxPoint::Min:
		Point = Box.Min; break;

	case EAlignObjectsBoxPoint::Max:
		Point = Box.Max; break;
	}

	return Point;
}


void UAlignObjectsTool::UpdateAlignment()
{
	if (AlignProps->AlignType == EAlignObjectsAlignTypes::Pivots)
	{
		UpdateAlignment_Pivots();
	}
	else if (AlignProps->AlignType == EAlignObjectsAlignTypes::BoundingBoxes)
	{
		UpdateAlignment_BoundingBoxes();
	}
}




void UAlignObjectsTool::UpdateAlignment_Pivots()
{
	FVector3d TargetPoint;
	if (AlignProps->AlignTo == EAlignObjectsAlignToOptions::FirstSelected)
	{
		TargetPoint = ComponentInfo[0].WorldPivot;
	}
	else if (AlignProps->AlignTo == EAlignObjectsAlignToOptions::LastSelected)
	{
		TargetPoint = ComponentInfo[ComponentInfo.Num()-1].WorldPivot;
	}
	else
	{
		TargetPoint = GetBoxPoint(PivotBounds, AlignProps->BoxPosition);
	}

	for (const FAlignInfo& AlignObj : ComponentInfo)
	{
		FVector3d SourcePoint = AlignObj.WorldPivot;

		FVector3d UseTarget(
			(AlignProps->bAlignX) ? TargetPoint.X : SourcePoint.X,
			(AlignProps->bAlignY) ? TargetPoint.Y : SourcePoint.Y,
			(AlignProps->bAlignZ) ? TargetPoint.Z : SourcePoint.Z);

		FVector3d Translation = UseTarget - SourcePoint;
		FTransform3d NewTransform = AlignObj.WorldTransform;
		NewTransform.SetTranslation(NewTransform.GetTranslation() + Translation);
		AlignObj.Component->SetWorldTransform((FTransform)NewTransform);
	}
}





void UAlignObjectsTool::UpdateAlignment_BoundingBoxes()
{
	FVector3d TargetPoint;
	if (AlignProps->AlignTo == EAlignObjectsAlignToOptions::FirstSelected)
	{
		TargetPoint = GetBoxPoint(ComponentInfo[0].WorldBounds, AlignProps->BoxPosition);
	}
	else if (AlignProps->AlignTo == EAlignObjectsAlignToOptions::LastSelected)
	{
		TargetPoint = GetBoxPoint(ComponentInfo[ComponentInfo.Num()-1].WorldBounds, AlignProps->BoxPosition);
	}
	else
	{
		TargetPoint = GetBoxPoint(CombinedBounds, AlignProps->BoxPosition);
	}

	for (const FAlignInfo& AlignObj : ComponentInfo)
	{
		FVector3d SourcePoint = GetBoxPoint(AlignObj.WorldBounds, AlignProps->BoxPosition);

		FVector3d UseTarget(
			(AlignProps->bAlignX) ? TargetPoint.X : SourcePoint.X,
			(AlignProps->bAlignY) ? TargetPoint.Y : SourcePoint.Y,
			(AlignProps->bAlignZ) ? TargetPoint.Z : SourcePoint.Z);

		FVector3d Translation = UseTarget - SourcePoint;
		FTransform3d NewTransform = AlignObj.WorldTransform;
		NewTransform.SetTranslation(NewTransform.GetTranslation() + Translation);
		AlignObj.Component->SetWorldTransform((FTransform)NewTransform);
	}
}



// does not make sense that CanBeginClickDragSequence() returns a RayHit? Needs to be an out-argument...
FInputRayHit UAlignObjectsTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return FInputRayHit();
}

void UAlignObjectsTool::OnClickPress(const FInputDeviceRay& PressPos)
{
}


void UAlignObjectsTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
}


void UAlignObjectsTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
}

void UAlignObjectsTool::OnTerminateDragSequence()
{
}


#undef LOCTEXT_NAMESPACE

