// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeMeshAttributeMapsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ModelingToolTargetUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Sampling/MeshNormalMapEvaluator.h"
#include "Sampling/MeshOcclusionMapEvaluator.h"
#include "Sampling/MeshCurvatureMapEvaluator.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "Sampling/MeshUVShellMapEvaluator.h"
#include "Sampling/MeshResampleImageEvaluator.h"
#include "Sampling/MeshHeightMapEvaluator.h"

#include "ImageUtils.h"

#include "AssetUtils/Texture2DUtil.h"

#include "EngineAnalytics.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "ToolTargetManager.h"

// required to pass UStaticMesh asset so we can save at same location
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/DynamicMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshAttributeMapsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeMeshAttributeMapsTool"


/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UBakeMeshAttributeMapsToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMeshDescriptionProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		UMaterialProvider::StaticClass()
		});
	return TypeRequirements;
}

bool UBakeMeshAttributeMapsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	const int32 NumTargets = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	if (NumTargets == 1 || NumTargets == 2)
	{
		bool bValidTargets = true;
		SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(),
			[&bValidTargets](UActorComponent* Component)
			{
				UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Component);
				USkeletalMeshComponent* SkeletalMesh = Cast<USkeletalMeshComponent>(Component);
				UDynamicMeshComponent* DynMesh = Cast<UDynamicMeshComponent>(Component);
				bValidTargets = bValidTargets && (StaticMesh || SkeletalMesh || DynMesh);
			});
		return bValidTargets;
	}
	return false;
}

UMultiSelectionMeshEditingTool* UBakeMeshAttributeMapsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UBakeMeshAttributeMapsTool>(SceneState.ToolManager);
}


const TArray<FString>& UBakeMeshAttributeMapsToolProperties::GetMapPreviewNamesFunc()
{
	return MapPreviewNamesList;
}



/*
 * Operators
 */

class FMeshMapBakerOp : public TGenericDataOperator<FMeshMapBaker>
{
public:
	using ImagePtr = TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>;
	
	// General bake settings
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> DetailMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;
	TSharedPtr<UE::Geometry::TMeshTangents<double>, ESPMode::ThreadSafe> DetailMeshTangents;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> BaseMesh;
	TUniquePtr<UE::Geometry::FMeshMapBaker> Baker;
	UBakeMeshAttributeMapsTool::FBakeSettings BakeSettings;
	TSharedPtr<UE::Geometry::TMeshTangents<double>, ESPMode::ThreadSafe> BaseMeshTangents;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts;
	bool bIsBakeToSelf = false;
	ImagePtr SampleFilterMask;

	// Map Type settings
	EBakeMapType Maps;
	FNormalMapSettings NormalSettings;
	FOcclusionMapSettings OcclusionSettings;
	FCurvatureMapSettings CurvatureSettings;
	FMeshPropertyMapSettings PropertySettings;
	FHeightMapSettings HeightSettings;
	FUVShellMapSettings UVShellSettings;
	FTexture2DSettings TextureSettings;
	FTexture2DSettings MultiTextureSettings;

	// NormalMap settings
	ImagePtr DetailMeshNormalMap;
	int32 DetailMeshNormalUVLayer = 0;
	IMeshBakerDetailSampler::EBakeDetailNormalSpace DetailMeshNormalSpace = IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent; 

	// Texture2DImage & MultiTexture settings
	ImagePtr TextureImage;
	TArray<ImagePtr> MaterialIDTextures;

	// Begin TGenericDataOperator interface
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		Baker = MakeUnique<FMeshMapBaker>();
		Baker->CancelF = [Progress]() {
			return Progress && Progress->Cancelled();
		};
		Baker->SetTargetMesh(BaseMesh.Get());
		Baker->SetTargetMeshUVLayer(BakeSettings.TargetUVLayer);
		Baker->SetDimensions(BakeSettings.Dimensions);
		Baker->SetProjectionDistance(BakeSettings.ProjectionDistance);
		Baker->SetSamplesPerPixel(BakeSettings.SamplesPerPixel);
		Baker->SetTargetMeshUVCharts(BaseMeshUVCharts.Get());
		Baker->SetTargetMeshTangents(BaseMeshTangents);
		if (bIsBakeToSelf)
		{
			Baker->SetCorrespondenceStrategy(FMeshBaseBaker::ECorrespondenceStrategy::Identity);
		}
		if (SampleFilterMask)
		{
			Baker->SampleFilterF = [this](const FVector2i& ImageCoords, const FVector2d& UV, int32 TriID)
			{
				const FVector4f Mask = SampleFilterMask->BilinearSampleUV<float>(UV, FVector4f::One());
				return (Mask.X + Mask.Y + Mask.Z) / 3;
			};
		}
		
		FMeshBakerDynamicMeshSampler DetailSampler(DetailMesh.Get(), DetailSpatial.Get(), DetailMeshTangents.Get());
		Baker->SetDetailSampler(&DetailSampler);

		// Occlusion evaluator is shared by both AmbientOcclusion and BentNormal.
		// Only initialize it once. OcclusionType is initialized to None. Callers
		// must manually update the OcclusionType.
		auto InitOcclusionEval = [this](TSharedPtr<FMeshOcclusionMapEvaluator>& Eval)
		{
			if (!Eval)
			{
				Eval = MakeShared<FMeshOcclusionMapEvaluator>();
				Eval->OcclusionType = EMeshOcclusionMapType::None;
				Eval->NumOcclusionRays = OcclusionSettings.OcclusionRays;
				Eval->MaxDistance = OcclusionSettings.MaxDistance;
				Eval->SpreadAngle = OcclusionSettings.SpreadAngle;
				Eval->BiasAngleDeg = OcclusionSettings.BiasAngle;
				switch (OcclusionSettings.NormalSpace)
				{
				case EBakeNormalSpace::Tangent:
					Eval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Tangent;
					break;
				case EBakeNormalSpace::Object:
					Eval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Object;
					break;
				}
				Baker->AddEvaluator(Eval);
			}
		};

		TSharedPtr<FMeshOcclusionMapEvaluator> OcclusionEval; 
		for (const EBakeMapType MapType : ENUM_EBAKEMAPTYPE_ALL)
		{
			switch (BakeSettings.BakeMapTypes & MapType)
			{
			case EBakeMapType::TangentSpaceNormal:
			{
				TSharedPtr<FMeshNormalMapEvaluator> NormalEval = MakeShared<FMeshNormalMapEvaluator>();
				DetailSampler.SetNormalTextureMap(DetailMesh.Get(), IMeshBakerDetailSampler::FBakeDetailNormalTexture(DetailMeshNormalMap.Get(), DetailMeshNormalUVLayer, DetailMeshNormalSpace));
				Baker->AddEvaluator(NormalEval);
				break;
			}
			case EBakeMapType::AmbientOcclusion:
			{
				InitOcclusionEval(OcclusionEval);
				OcclusionEval->OcclusionType |= EMeshOcclusionMapType::AmbientOcclusion;
				break;
			}
			case EBakeMapType::BentNormal:
			{
				InitOcclusionEval(OcclusionEval);
				OcclusionEval->OcclusionType |= EMeshOcclusionMapType::BentNormal;
				break;
			}
			case EBakeMapType::Curvature:
			{
				TSharedPtr<FMeshCurvatureMapEvaluator> CurvatureEval = MakeShared<FMeshCurvatureMapEvaluator>();
				CurvatureEval->RangeScale = FMathd::Clamp(CurvatureSettings.RangeMultiplier, 0.0001, 1000.0);
				CurvatureEval->MinRangeScale = FMathd::Clamp(CurvatureSettings.MinRangeMultiplier, 0.0, 1.0);
				CurvatureEval->UseCurvatureType = (FMeshCurvatureMapEvaluator::ECurvatureType)CurvatureSettings.CurvatureType;
				CurvatureEval->UseColorMode = (FMeshCurvatureMapEvaluator::EColorMode)CurvatureSettings.ColorMode;
				CurvatureEval->UseClampMode = (FMeshCurvatureMapEvaluator::EClampMode)CurvatureSettings.ClampMode;
				Baker->AddEvaluator(CurvatureEval);
				break;
			}
			case EBakeMapType::ObjectSpaceNormal:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::Normal;
				DetailSampler.SetNormalTextureMap(DetailMesh.Get(), IMeshBakerDetailSampler::FBakeDetailNormalTexture(DetailMeshNormalMap.Get(), DetailMeshNormalUVLayer, DetailMeshNormalSpace));
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::FaceNormal:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::FacetNormal;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::Position:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::Position;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::MaterialID:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::MaterialID;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::PolyGroupID:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::PolyGroupID;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::Height:
			{
				TSharedPtr<FMeshHeightMapEvaluator> HeightEval = MakeShared<FMeshHeightMapEvaluator>();
				HeightEval->RangeMode = (FMeshHeightMapEvaluator::EHeightRangeMode) HeightSettings.HeightRangeMode;
				switch(HeightEval->RangeMode)
				{
				case FMeshHeightMapEvaluator::EHeightRangeMode::Absolute:
					HeightEval->Range = FInterval1f::MakeFromUnordered(HeightSettings.InnerDistance, HeightSettings.OuterDistance);
					break;
				case FMeshHeightMapEvaluator::EHeightRangeMode::RelativeBounds:
					HeightEval->Range = FInterval1f::MakeFromUnordered(HeightSettings.InnerBoundsDistance, HeightSettings.OuterBoundsDistance);
					break;
				}
				Baker->AddEvaluator(HeightEval);
				break;
			}
			case EBakeMapType::UVShell:
			{
				TSharedPtr<FMeshUVShellMapEvaluator> UVShellEval = MakeShared<FMeshUVShellMapEvaluator>();
				UVShellEval->TexelSize = BakeSettings.Dimensions.GetTexelSize();
				UVShellEval->UVLayer = UVShellSettings.UVLayer;
				UVShellEval->WireframeThickness = UVShellSettings.WireframeThickness;
				UVShellEval->WireframeColor = UVShellSettings.WireframeColor;
				UVShellEval->ShellColor = UVShellSettings.ShellColor;
				UVShellEval->BackgroundColor = UVShellSettings.BackgroundColor;
				Baker->AddEvaluator(UVShellEval);
				break;
			}
			case EBakeMapType::VertexColor:
			{
				TSharedPtr<FMeshPropertyMapEvaluator> PropertyEval = MakeShared<FMeshPropertyMapEvaluator>();
				PropertyEval->Property = EMeshPropertyMapType::VertexColor;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::Texture:
			{
				TSharedPtr<FMeshResampleImageEvaluator> TextureEval = MakeShared<FMeshResampleImageEvaluator>();
				DetailSampler.SetTextureMap(DetailMesh.Get(), IMeshBakerDetailSampler::FBakeDetailTexture(TextureImage.Get(), TextureSettings.UVLayer));
				Baker->AddEvaluator(TextureEval);
				break;
			}
			case EBakeMapType::MultiTexture:
			{
				TSharedPtr<FMeshMultiResampleImageEvaluator> TextureEval = MakeShared<FMeshMultiResampleImageEvaluator>();
				TextureEval->DetailUVLayer = MultiTextureSettings.UVLayer;
				TextureEval->MultiTextures = MaterialIDTextures;
				Baker->AddEvaluator(TextureEval);
				break;
			}
			default:
				break;
			}
		}
		Baker->Bake();
		SetResult(MoveTemp(Baker));
	}
	// End TGenericDataOperator interface
};

/*
 * Tool
 */

void UBakeMeshAttributeMapsTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMeshAttributeMapsTool::Setup);
	
	Super::Setup();

	// Initialize preview mesh
	bIsBakeToSelf = (Targets.Num() == 1);

	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		TargetMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		TargetMesh->Copy(Mesh);

		TargetMeshTangents = MakeShared<FMeshTangentsd, ESPMode::ThreadSafe>(TargetMesh.Get());
		TargetMeshTangents->CopyTriVertexTangents(Mesh);
	});

	UToolTarget* Target = Targets[0];
	UToolTarget* DetailTarget = Targets[bIsBakeToSelf ? 0 : 1];
	
	// Setup tool property sets

	Settings = NewObject<UBakeMeshAttributeMapsToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->MapTypes, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->MapPreview, [this](FString) { UpdateVisualization(); GetToolManager()->PostInvalidation(); });
	Settings->WatchProperty(Settings->Resolution, [this](EBakeTextureResolution) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->BitDepth, [this](EBakeTextureBitDepth) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->SamplesPerPixel, [this](EBakeTextureSamplesPerPixel) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->SampleFilterMask, [this](UTexture2D*){ OpState |= EBakeOpState::Evaluate; });
	

	InputMeshSettings = NewObject<UBakeInputMeshProperties>(this);
	InputMeshSettings->RestoreProperties(this);
	AddToolPropertySource(InputMeshSettings);
	SetToolPropertySourceEnabled(InputMeshSettings, true);
	InputMeshSettings->bHasTargetUVLayer = true;
	InputMeshSettings->bHasSourceNormalMap = true;
	InputMeshSettings->TargetStaticMesh = UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetSkeletalMesh = UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetDynamicMesh = GetTargetActorViaIPersistentDynamicMeshSource(Target);
	InputMeshSettings->SourceStaticMesh = !bIsBakeToSelf ? UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(DetailTarget) : nullptr;
	InputMeshSettings->SourceSkeletalMesh = !bIsBakeToSelf ? UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(DetailTarget) : nullptr;
	InputMeshSettings->SourceDynamicMesh = !bIsBakeToSelf ? GetTargetActorViaIPersistentDynamicMeshSource(DetailTarget) : nullptr;
	InputMeshSettings->SourceNormalMap = nullptr;
	UpdateUVLayerNames(InputMeshSettings->TargetUVLayer, InputMeshSettings->TargetUVLayerNamesList, *TargetMesh);
	InputMeshSettings->WatchProperty(InputMeshSettings->bHideSourceMesh, [this](bool bState) { SetSourceObjectVisible(!bState); });
	InputMeshSettings->WatchProperty(InputMeshSettings->TargetUVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	InputMeshSettings->WatchProperty(InputMeshSettings->ProjectionDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	InputMeshSettings->WatchProperty(InputMeshSettings->bProjectionInWorldSpace, [this](bool) { OpState |= EBakeOpState::EvaluateDetailMesh; });
	InputMeshSettings->WatchProperty(InputMeshSettings->SourceNormalMapUVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	InputMeshSettings->WatchProperty(InputMeshSettings->SourceNormalMap, [this](UTexture2D*)
	{
		// Only invalidate detail mesh if we need to recompute tangents.
		if (!DetailMeshTangents && InputMeshSettings->SourceNormalSpace == EBakeNormalSpace::Tangent)
		{
			OpState |= EBakeOpState::EvaluateDetailMesh;
		}
		OpState |= EBakeOpState::Evaluate;
	});
	InputMeshSettings->WatchProperty(InputMeshSettings->SourceNormalSpace, [this](EBakeNormalSpace Space)
	{
		if (!DetailMeshTangents && Space == EBakeNormalSpace::Tangent)
		{
			OpState |= EBakeOpState::EvaluateDetailMesh;
		}
		OpState |= EBakeOpState::Evaluate;
	});
	SetSourceObjectVisible(!InputMeshSettings->bHideSourceMesh);
	

	ResultSettings = NewObject<UBakeMeshAttributeMapsResultToolProperties>(this);
	ResultSettings->RestoreProperties(this);
	AddToolPropertySource(ResultSettings);
	SetToolPropertySourceEnabled(ResultSettings, true);
	

	OcclusionSettings = NewObject<UBakeOcclusionMapToolProperties>(this);
	OcclusionSettings->RestoreProperties(this);
	AddToolPropertySource(OcclusionSettings);
	SetToolPropertySourceEnabled(OcclusionSettings, false);
	OcclusionSettings->WatchProperty(OcclusionSettings->OcclusionRays, [this](int32) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->MaxDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->SpreadAngle, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->BiasAngle, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->NormalSpace, [this](EBakeNormalSpace) { OpState |= EBakeOpState::Evaluate; });


	CurvatureSettings = NewObject<UBakeCurvatureMapToolProperties>(this);
	CurvatureSettings->RestoreProperties(this);
	AddToolPropertySource(CurvatureSettings);
	SetToolPropertySourceEnabled(CurvatureSettings, false);
	CurvatureSettings->WatchProperty(CurvatureSettings->ColorRangeMultiplier, [this](float) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->MinRangeMultiplier, [this](float) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->CurvatureType, [this](EBakeCurvatureTypeMode) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->ColorMapping, [this](EBakeCurvatureColorMode) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->Clamping, [this](EBakeCurvatureClampMode) { OpState |= EBakeOpState::Evaluate; });


	UVShellSettings = NewObject<UBakeUVShellMapToolProperties>(this);
	UVShellSettings->RestoreProperties(this);
	AddToolPropertySource(UVShellSettings);
	SetToolPropertySourceEnabled(UVShellSettings, false);
	UVShellSettings->WatchProperty(UVShellSettings->UVLayer, [this](int) { OpState |= EBakeOpState::Evaluate; });
	UVShellSettings->WatchProperty(UVShellSettings->WireframeThickness, [this](float) { OpState |= EBakeOpState::Evaluate; });
	UVShellSettings->WatchProperty(UVShellSettings->WireframeColor, [this](FLinearColor) { OpState |= EBakeOpState::Evaluate; });
	UVShellSettings->WatchProperty(UVShellSettings->ShellColor, [this](FLinearColor) { OpState |= EBakeOpState::Evaluate; });
	UVShellSettings->WatchProperty(UVShellSettings->BackgroundColor, [this](FLinearColor) { OpState |= EBakeOpState::Evaluate; });


	HeightSettings = NewObject<UBakeHeightMapToolProperties>(this);
	HeightSettings->RestoreProperties(this);
	AddToolPropertySource(HeightSettings);
	SetToolPropertySourceEnabled(HeightSettings, false);
	HeightSettings->WatchProperty(HeightSettings->HeightRangeMode, [this](EBakeHeightRangeMode) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->InnerDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->OuterDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->InnerBoundsDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->OuterBoundsDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });


	TextureSettings = NewObject<UBakeTexture2DProperties>(this);
	TextureSettings->RestoreProperties(this);
	AddToolPropertySource(TextureSettings);
	SetToolPropertySourceEnabled(TextureSettings, false);
	TextureSettings->WatchProperty(TextureSettings->UVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	TextureSettings->WatchProperty(TextureSettings->SourceTexture, [this](UTexture2D*) { OpState |= EBakeOpState::Evaluate; });

	MultiTextureSettings = NewObject<UBakeMultiTexture2DProperties>(this);
	MultiTextureSettings->RestoreProperties(this);
	AddToolPropertySource(MultiTextureSettings);
	SetToolPropertySourceEnabled(MultiTextureSettings, false);

	auto SetDirtyCallback = [this](decltype(MultiTextureSettings->MaterialIDSourceTextures)) { OpState |= EBakeOpState::Evaluate; };
	auto NotEqualsCallback = [](const decltype(MultiTextureSettings->MaterialIDSourceTextures)& A, const decltype(MultiTextureSettings->MaterialIDSourceTextures)& B) -> bool { return A != B; };
	MultiTextureSettings->WatchProperty(MultiTextureSettings->MaterialIDSourceTextures, SetDirtyCallback, NotEqualsCallback);
	MultiTextureSettings->WatchProperty(MultiTextureSettings->UVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	UpdateMultiTextureMaterialIDs(DetailTarget, MultiTextureSettings->AllSourceTextures, MultiTextureSettings->MaterialIDSourceTextures);

	UpdateOnModeChange();

	UpdateDetailMesh();

	SetToolDisplayName(LOCTEXT("ToolName", "Bake Textures"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Bake Maps. Select Bake Mesh (LowPoly) first, then (optionally) Detail Mesh second. Texture Assets will be created on Accept. "),
		EToolMessageLevel::UserNotification);

	PostSetup();
}




bool UBakeMeshAttributeMapsTool::CanAccept() const
{
	const bool bValidOp = (OpState & EBakeOpState::Invalid) != EBakeOpState::Invalid;
	bool bCanAccept = bValidOp && Compute ? Compute->HaveValidResult() : false;
	if (bCanAccept)
	{
		// Allow Accept if all non-None types have valid results.
		for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Result : ResultSettings->Result)
		{
			bCanAccept = bCanAccept && Result.Get<1>();
		}
	}
	return bCanAccept;
}


TUniquePtr<UE::Geometry::TGenericDataOperator<FMeshMapBaker>> UBakeMeshAttributeMapsTool::MakeNewOperator()
{
	TUniquePtr<FMeshMapBakerOp> Op = MakeUnique<FMeshMapBakerOp>();
	Op->DetailMesh = DetailMesh;
	Op->DetailSpatial = DetailSpatial;
	Op->BaseMesh = TargetMesh;
	Op->BakeSettings = CachedBakeSettings;
	Op->BaseMeshUVCharts = TargetMeshUVCharts;
	Op->bIsBakeToSelf = bIsBakeToSelf;
	Op->SampleFilterMask = CachedSampleFilterMask;

	constexpr EBakeMapType RequiresTangents = EBakeMapType::TangentSpaceNormal | EBakeMapType::BentNormal;
	if ((bool)(CachedBakeSettings.BakeMapTypes & RequiresTangents))
	{
		Op->BaseMeshTangents = TargetMeshTangents;
	}

	if (CachedDetailNormalMap)
	{
		Op->DetailMeshTangents = DetailMeshTangents;
		Op->DetailMeshNormalMap = CachedDetailNormalMap;
		Op->DetailMeshNormalUVLayer = CachedDetailMeshSettings.UVLayer;
		Op->DetailMeshNormalSpace = CachedDetailMeshSettings.NormalSpace == EBakeNormalSpace::Tangent ?
			IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent : IMeshBakerDetailSampler::EBakeDetailNormalSpace::Object;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::TangentSpaceNormal))
	{
		Op->NormalSettings = CachedNormalMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::AmbientOcclusion) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::BentNormal))
	{
		Op->OcclusionSettings = CachedOcclusionMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Curvature))
	{
		Op->CurvatureSettings = CachedCurvatureMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::ObjectSpaceNormal) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::FaceNormal) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Position) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::MaterialID) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::VertexColor))
	{
		Op->PropertySettings = CachedMeshPropertyMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Height))
	{
		Op->HeightSettings = CachedHeightMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::UVShell))
	{
		Op->UVShellSettings = CachedUVShellMapSettings;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Texture))
	{
		Op->TextureSettings = CachedTexture2DSettings;
		Op->TextureImage = CachedTextureImage;
	}

	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::MultiTexture))
	{
		Op->MultiTextureSettings = CachedMultiTexture2DSettings;
		Op->MaterialIDTextures = CachedMultiTextures;
	}

	return Op;
}


void UBakeMeshAttributeMapsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMeshAttributeMapsTool::Shutdown);
	
	Super::OnShutdown(ShutdownType);
	
	Settings->SaveProperties(this);
	InputMeshSettings->SaveProperties(this);
	OcclusionSettings->SaveProperties(this);
	CurvatureSettings->SaveProperties(this);
	HeightSettings->SaveProperties(this);
	TextureSettings->SaveProperties(this);
	MultiTextureSettings->SaveProperties(this);

	SetSourceObjectVisible(true);

	if (Compute)
	{
		Compute->Shutdown();
	}
	if (ShutdownType == EToolShutdownType::Accept)
	{
		// Check if we have a source asset to identify a location to store the texture assets.
		IStaticMeshBackedTarget* StaticMeshTarget = Cast<IStaticMeshBackedTarget>(Targets[0]);
		UObject* SourceAsset = StaticMeshTarget ? StaticMeshTarget->GetStaticMesh() : nullptr;
		if (!SourceAsset)
		{
			// Check if our target is a Skeletal Mesh Asset
			ISkeletalMeshBackedTarget* SkeletalMeshTarget = Cast<ISkeletalMeshBackedTarget>(Targets[0]);
			SourceAsset = SkeletalMeshTarget ? SkeletalMeshTarget->GetSkeletalMesh() : nullptr;
		}
		const UPrimitiveComponent* SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[0]);
		CreateTextureAssets(ResultSettings->Result, SourceComponent->GetWorld(), SourceAsset);
	}
}


bool UBakeMeshAttributeMapsTool::ValidDetailMeshTangents()
{
	if (!DetailMesh)
	{
		return false;
	}
	if (bCheckDetailMeshTangents)
	{
		bValidDetailMeshTangents = FDynamicMeshTangents(DetailMesh.Get()).HasValidTangents(true);
		bCheckDetailMeshTangents = false;
	}
	return bValidDetailMeshTangents;
}


void UBakeMeshAttributeMapsTool::UpdateDetailMesh()
{
	UToolTarget* DetailTarget = Targets[bIsBakeToSelf ? 0 : 1];

	const bool bWantMeshTangents = (InputMeshSettings->SourceNormalMap != nullptr);
	FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = bWantMeshTangents;
	DetailMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(DetailTarget, GetMeshParams));

	if (InputMeshSettings->bProjectionInWorldSpace && bIsBakeToSelf == false)
	{
		const FTransformSRT3d DetailToWorld = UE::ToolTarget::GetLocalToWorldTransform(DetailTarget);
		MeshTransforms::ApplyTransform(*DetailMesh, DetailToWorld, true);
		const FTransformSRT3d WorldToBase = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
		MeshTransforms::ApplyTransformInverse(*DetailMesh, WorldToBase, true);
	}

	DetailSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>();
	DetailSpatial->SetMesh(DetailMesh.Get(), true);

	// Extract tangents if a DetailMesh normal map was provided.
	if (bWantMeshTangents)
	{
		DetailMeshTangents = MakeShared<FMeshTangentsd, ESPMode::ThreadSafe>(DetailMesh.Get());
		DetailMeshTangents->CopyTriVertexTangents(*DetailMesh);
	}
	else
	{
		DetailMeshTangents = nullptr;
	}

	UpdateUVLayerNames(InputMeshSettings->SourceNormalMapUVLayer, InputMeshSettings->SourceUVLayerNamesList, *DetailMesh);
	UpdateUVLayerNames(TextureSettings->UVLayer, TextureSettings->UVLayerNamesList, *DetailMesh);
	UpdateUVLayerNames(MultiTextureSettings->UVLayer, MultiTextureSettings->UVLayerNamesList, *DetailMesh);

	// Clear detail mesh evaluation flag and mark evaluation.
	OpState &= ~EBakeOpState::EvaluateDetailMesh;
	OpState |= EBakeOpState::Evaluate;
	CachedBakeSettings = FBakeSettings();
	DetailMeshTimestamp++;
}


void UBakeMeshAttributeMapsTool::UpdateResult()
{
	if (static_cast<bool>(OpState & EBakeOpState::EvaluateDetailMesh))
	{
		UpdateDetailMesh();
	}

	if (OpState == EBakeOpState::Clean)
	{
		return;
	}

	// clear warning (ugh)
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

	int32 ImageSize = (int32)Settings->Resolution;
	FImageDimensions Dimensions(ImageSize, ImageSize);

	FBakeSettings BakeSettings;
	BakeSettings.Dimensions = Dimensions;
	BakeSettings.BitDepth = Settings->BitDepth;
	BakeSettings.TargetUVLayer = InputMeshSettings->TargetUVLayerNamesList.IndexOfByKey(InputMeshSettings->TargetUVLayer);
	BakeSettings.DetailTimestamp = this->DetailMeshTimestamp;
	BakeSettings.ProjectionDistance = InputMeshSettings->ProjectionDistance;
	BakeSettings.SamplesPerPixel = (int32)Settings->SamplesPerPixel;
	BakeSettings.bProjectionInWorldSpace = InputMeshSettings->bProjectionInWorldSpace;

	// Record the original map types and process the raw bitfield which may add
	// additional targets.
	BakeSettings.SourceBakeMapTypes = static_cast<EBakeMapType>(Settings->MapTypes);
	BakeSettings.BakeMapTypes = GetMapTypes(Settings->MapTypes);

	// update bake cache settings
	if (!(CachedBakeSettings == BakeSettings))
	{
		CachedBakeSettings = BakeSettings;

		CachedNormalMapSettings = FNormalMapSettings();
		CachedOcclusionMapSettings = FOcclusionMapSettings();
		CachedCurvatureMapSettings = FCurvatureMapSettings();
		CachedMeshPropertyMapSettings = FMeshPropertyMapSettings();
		CachedTexture2DSettings = FTexture2DSettings();
		CachedMultiTexture2DSettings = FTexture2DSettings();
	}

	// Clear our invalid bitflag to check again for valid inputs.
	OpState &= ~EBakeOpState::Invalid;

	OpState |= UpdateResult_TargetMeshTangents(CachedBakeSettings.BakeMapTypes);

	OpState |= UpdateResult_DetailNormalMap();

	OpState |= UpdateResult_SampleFilterMask(Settings->SampleFilterMask);

	// Update map type settings
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::TangentSpaceNormal))
	{
		OpState |= UpdateResult_Normal(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::AmbientOcclusion) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::BentNormal))
	{
		OpState |= UpdateResult_Occlusion(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Curvature))
	{
		OpState |= UpdateResult_Curvature(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::ObjectSpaceNormal) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::FaceNormal) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Position) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::MaterialID) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::PolyGroupID) ||
		(bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::VertexColor))
	{
		OpState |= UpdateResult_MeshProperty(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Height))
	{
		OpState |= UpdateResult_Height(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::UVShell))
	{
		OpState |= UpdateResult_UVShellMap(CachedBakeSettings.Dimensions);
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::Texture))
	{
		OpState |= UpdateResult_Texture2DImage(CachedBakeSettings.Dimensions, DetailMesh.Get());
	}
	if ((bool)(CachedBakeSettings.BakeMapTypes & EBakeMapType::MultiTexture))
	{
		OpState |= UpdateResult_MultiTexture(CachedBakeSettings.Dimensions, DetailMesh.Get());
	}

	// Early exit if op input parameters are invalid.
	if ((bool)(OpState & EBakeOpState::Invalid))
	{
		InvalidateResults();
		return;
	}

	// This should be the only point of compute invalidation to
	// minimize synchronization issues.
	InvalidateCompute();
}


EBakeOpState UBakeMeshAttributeMapsTool::UpdateResult_DetailMeshTangents(EBakeMapType BakeType)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	const bool bNeedDetailMeshTangents = (bool)(BakeType & (EBakeMapType::TangentSpaceNormal | EBakeMapType::BentNormal)); 
	if (bNeedDetailMeshTangents && !ValidDetailMeshTangents())
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidSourceTangentsWarning", "The Source Mesh does not have valid tangents."), EToolMessageLevel::UserWarning);
		ResultState = EBakeOpState::Invalid;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeMapsTool::UpdateResult_DetailNormalMap()
{
	EBakeOpState ResultState = EBakeOpState::Clean;

	const int DetailUVLayer = InputMeshSettings->SourceUVLayerNamesList.IndexOfByKey(InputMeshSettings->SourceNormalMapUVLayer);
	const FDynamicMeshUVOverlay* UVOverlay = DetailMesh->Attributes()->GetUVLayer(DetailUVLayer);
	if (UVOverlay == nullptr)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidUVWarning", "The Source Mesh does not have the selected UV layer"), EToolMessageLevel::UserWarning);
		return EBakeOpState::Invalid;
	}
	
	if (UTexture2D* DetailMeshNormalMap = InputMeshSettings->SourceNormalMap)
	{
		CachedDetailNormalMap = MakeShared<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>();
		if (!UE::AssetUtils::ReadTexture(DetailMeshNormalMap, *CachedDetailNormalMap, bPreferPlatformData))
		{
			// Report the failed texture read as a warning, but permit the bake to continue.
			GetToolManager()->DisplayMessage(LOCTEXT("CannotReadTextureWarning", "Cannot read from the source normal map texture"), EToolMessageLevel::UserWarning);
		}

		// Validate detail mesh tangents for detail normal map
		ResultState |= UpdateResult_DetailMeshTangents(CachedBakeSettings.BakeMapTypes);
	}
	else
	{
		CachedDetailNormalMap = nullptr;
	}

	FDetailMeshSettings DetailMeshSettings;
	DetailMeshSettings.UVLayer = DetailUVLayer;
	DetailMeshSettings.NormalSpace = InputMeshSettings->SourceNormalSpace;

	if (!(CachedDetailMeshSettings == DetailMeshSettings))
	{
		CachedDetailMeshSettings = DetailMeshSettings;
		ResultState |= EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeMapsTool::UpdateResult_UVShellMap(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;

	FUVShellMapSettings UVShellMapSettings;
	UVShellMapSettings.UVLayer = UVShellSettings->UVLayer;
	UVShellMapSettings.WireframeThickness = UVShellSettings->WireframeThickness;
	UVShellMapSettings.WireframeColor = UVShellSettings->WireframeColor;
	UVShellMapSettings.ShellColor = UVShellSettings->ShellColor;
	UVShellMapSettings.BackgroundColor = UVShellSettings->BackgroundColor;

	if (!(CachedUVShellMapSettings == UVShellMapSettings))
	{
		CachedUVShellMapSettings = UVShellMapSettings;
		ResultState |= EBakeOpState::Evaluate;
	}
	return ResultState;
}

void UBakeMeshAttributeMapsTool::UpdateVisualization()
{
	PreviewMesh->SetOverrideRenderMaterial(PreviewMaterial);

	// Populate Settings->Result from CachedMaps
	for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Map : CachedMaps)
	{
		if (ResultSettings->Result.Contains(Map.Get<0>()))
		{
			ResultSettings->Result[Map.Get<0>()] = Map.Get<1>();
		}
	}

	UpdatePreview(Settings->MapPreview, Settings->MapPreviewNamesMap);
}


void UBakeMeshAttributeMapsTool::UpdateOnModeChange()
{
	OnMapTypesUpdated(
		static_cast<EBakeMapType>(Settings->MapTypes),
		ResultSettings->Result,
		Settings->MapPreview,
		Settings->MapPreviewNamesList,
		Settings->MapPreviewNamesMap);

	// Update tool property sets.
	SetToolPropertySourceEnabled(OcclusionSettings, false);
	SetToolPropertySourceEnabled(CurvatureSettings, false);
	SetToolPropertySourceEnabled(HeightSettings, false);
	SetToolPropertySourceEnabled(UVShellSettings, false);
	SetToolPropertySourceEnabled(TextureSettings, false);
	SetToolPropertySourceEnabled(MultiTextureSettings, false);

	for (const EBakeMapType MapType : ENUM_EBAKEMAPTYPE_ALL)
	{
		switch ((EBakeMapType)Settings->MapTypes & MapType)
		{
		case EBakeMapType::TangentSpaceNormal:
			break;
		case EBakeMapType::AmbientOcclusion:
		case EBakeMapType::BentNormal:
			SetToolPropertySourceEnabled(OcclusionSettings, true);
			break;
		case EBakeMapType::Curvature:
			SetToolPropertySourceEnabled(CurvatureSettings, true);
			break;
		case EBakeMapType::ObjectSpaceNormal:
		case EBakeMapType::FaceNormal:
		case EBakeMapType::Position:
		case EBakeMapType::MaterialID:
		case EBakeMapType::PolyGroupID:
		case EBakeMapType::VertexColor:
			break;
		case EBakeMapType::Height:
			SetToolPropertySourceEnabled(HeightSettings, true);
			break;
		case EBakeMapType::UVShell:
			SetToolPropertySourceEnabled(UVShellSettings, true);
			break;
		case EBakeMapType::Texture:
			SetToolPropertySourceEnabled(TextureSettings, true);
			break;
		case EBakeMapType::MultiTexture:
			SetToolPropertySourceEnabled(MultiTextureSettings, true);
			break;
		default:
			break;
		}
	}
}


void UBakeMeshAttributeMapsTool::InvalidateResults()
{
	for (TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Result : ResultSettings->Result)
	{
		Result.Get<1>() = nullptr;
	}
}


void UBakeMeshAttributeMapsTool::GatherAnalytics(FBakeAnalytics::FMeshSettings& Data)
{
	if (FEngineAnalytics::IsAvailable())
	{
		Data.NumTargetMeshTris = TargetMesh->TriangleCount();
		Data.NumDetailMesh = 1;
		Data.NumDetailMeshTris = DetailMesh->TriangleCount();
	}
}




#undef LOCTEXT_NAMESPACE

