// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeMeshAttributeMapsToolBase.h"
#include "Baking/BakingTypes.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ImageUtils.h"

#include "Sampling/MeshOcclusionMapEvaluator.h"
#include "Sampling/MeshPropertyMapEvaluator.h"

#include "AssetUtils/Texture2DBuilder.h"
#include "AssetUtils/Texture2DUtil.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "ModelingToolTargetUtil.h"
#include "ModelingObjectsCreationAPI.h"

#include "EngineAnalytics.h"

#include "Sampling/MeshCurvatureMapEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshAttributeMapsToolBase)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeMeshAttributeMapsToolBase"


void UBakeMeshAttributeMapsToolBase::Setup()
{
	Super::Setup();

	InitializeEmptyMaps();

	// Setup preview materials
	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/BakePreviewMaterial"));
	check(Material);
	if (Material != nullptr)
	{
		PreviewMaterial = UMaterialInstanceDynamic::Create(Material, GetToolManager());
		ResetPreview();
	}
	UMaterial* BentNormalMaterial = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/BakeBentNormalPreviewMaterial"));
	check(BentNormalMaterial);
	if (BentNormalMaterial != nullptr)
	{
		BentNormalPreviewMaterial = UMaterialInstanceDynamic::Create(BentNormalMaterial, GetToolManager());
	}

	UE::ToolTarget::HideSourceObject(Targets[0]);

	// Initialize preview mesh
	PreviewMesh = CreateBakePreviewMesh(this, Targets[0], GetTargetWorld());
	PreviewMesh->SetOverrideRenderMaterial(PreviewMaterial);
}

void UBakeMeshAttributeMapsToolBase::PostSetup()
{
	VisualizationProps = NewObject<UBakeVisualizationProperties>(this);
	VisualizationProps->RestoreProperties(this);
	AddToolPropertySource(VisualizationProps);
	VisualizationProps->WatchProperty(VisualizationProps->bPreviewAsMaterial, [this](bool) { UpdateVisualization(); GetToolManager()->PostInvalidation(); });

	// Initialize UV charts
	// TODO: Compute UV charts asynchronously
	TargetMeshUVCharts = MakeShared<TArray<int32>, ESPMode::ThreadSafe>();
	FMeshMapBaker::ComputeUVCharts(*TargetMesh, *TargetMeshUVCharts);

	GatherAnalytics(BakeAnalytics.MeshSettings);
}

void UBakeMeshAttributeMapsToolBase::OnShutdown(EToolShutdownType ShutdownType)
{
	VisualizationProps->SaveProperties(this);

	if (PreviewMesh != nullptr)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	UE::ToolTarget::ShowSourceObject(Targets[0]);
}


void UBakeMeshAttributeMapsToolBase::OnTick(float DeltaTime)
{
	if (Compute)
	{
		Compute->Tick(DeltaTime);
	}

	if (static_cast<bool>(OpState & EBakeOpState::Invalid))
	{
		PreviewMesh->SetOverrideRenderMaterial(ErrorPreviewMaterial);
	}
	else
	{
		if (!CanAccept() && Compute && Compute->GetElapsedComputeTime() > SecondsBeforeWorkingMaterial)
		{
			PreviewMesh->SetOverrideRenderMaterial(WorkingPreviewMaterial);
		}
	}
}


void UBakeMeshAttributeMapsToolBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	UpdateResult();

	const float Brightness = VisualizationProps->Brightness;
	const FVector BrightnessColor(Brightness, Brightness, Brightness);
	const float AOMultiplier = VisualizationProps->AOMultiplier;

	PreviewMaterial->SetVectorParameterValue(TEXT("Brightness"), BrightnessColor );
	PreviewMaterial->SetScalarParameterValue(TEXT("AOMultiplier"), AOMultiplier );
	BentNormalPreviewMaterial->SetVectorParameterValue(TEXT("Brightness"), BrightnessColor );
	BentNormalPreviewMaterial->SetScalarParameterValue(TEXT("AOMultiplier"), AOMultiplier );
}


TUniquePtr<UE::Geometry::TGenericDataOperator<FMeshMapBaker>> UBakeMeshAttributeMapsToolBase::MakeNewOperator()
{
	return nullptr;
}


void UBakeMeshAttributeMapsToolBase::UpdateResult()
{
}


EBakeOpState UBakeMeshAttributeMapsToolBase::UpdateResult_SampleFilterMask(UTexture2D* SampleFilterMask)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	if (SampleFilterMask)
	{
		CachedSampleFilterMask = MakeShared<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>();
		if (!UE::AssetUtils::ReadTexture(SampleFilterMask, *CachedSampleFilterMask, bPreferPlatformData))
		{
			GetToolManager()->DisplayMessage(LOCTEXT("CannotReadTextureWarning", "Cannot read from the sample filter mask"), EToolMessageLevel::UserWarning);
			return EBakeOpState::Invalid;
		}
		ResultState = EBakeOpState::Evaluate;
	}
	else if (CachedSampleFilterMask)
	{
		// Clear CachedSampleFilterMask if SampleFilterMask is null and re-evaluate.
		CachedSampleFilterMask.Reset();
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


void UBakeMeshAttributeMapsToolBase::UpdateVisualization()
{
}


void UBakeMeshAttributeMapsToolBase::InvalidateCompute()
{
	if (!Compute)
	{
		// Initialize background compute
		Compute = MakeUnique<TGenericDataBackgroundCompute<FMeshMapBaker>>();
		Compute->Setup(this);
		Compute->OnResultUpdated.AddLambda([this](const TUniquePtr<FMeshMapBaker>& NewResult) { OnMapsUpdated(NewResult); });
	}
	Compute->InvalidateResult();
	OpState = EBakeOpState::Clean;
}


void UBakeMeshAttributeMapsToolBase::CreateTextureAssets(const TMap<EBakeMapType, TObjectPtr<UTexture2D>>& Textures, UWorld* SourceWorld, UObject* SourceAsset)
{
	bool bCreatedAssetOK = true;
	const FString BaseName = UE::ToolTarget::GetTargetActor(Targets[0])->GetActorNameOrLabel();
	for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Result : Textures)
	{
		FString TexName;
		GetTextureName(Result.Get<0>(), BaseName, TexName);
		bCreatedAssetOK = bCreatedAssetOK &&
			UE::Modeling::CreateTextureObject(
				GetToolManager(),
				FCreateTextureObjectParams{ 0, SourceWorld, SourceAsset, TexName, Result.Get<1>() }).IsOK();
	}
	ensure(bCreatedAssetOK);

	RecordAnalytics(BakeAnalytics, GetAnalyticsEventName());
}


void UBakeMeshAttributeMapsToolBase::UpdatePreview(const FString& PreviewDisplayName, const TMap<FString, FString>& MapPreviewNamesMap)
{
	if (const FString* PreviewNameString = MapPreviewNamesMap.Find(PreviewDisplayName))
	{
		const int64 PreviewValue = StaticEnum<EBakeMapType>()->GetValueByNameString(*PreviewNameString);
		if (PreviewValue != INDEX_NONE)
		{
			UpdatePreview(static_cast<EBakeMapType>(PreviewValue));
		}
	}
}


void UBakeMeshAttributeMapsToolBase::UpdatePreview(const EBakeMapType PreviewMapType)
{
	if (PreviewMapType == EBakeMapType::None)
	{
		return;
	}

	if (!CachedMaps.Contains(PreviewMapType))
	{
		return;
	}

	const bool bPreviewAsMaterial = VisualizationProps->bPreviewAsMaterial;
	UTexture2D* PreviewMap = CachedMaps[PreviewMapType];
	if (bPreviewAsMaterial)
	{
		switch (PreviewMapType)
		{
		default:
			PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
			PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), EmptyColorMapWhite);
			PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), EmptyColorMapWhite);
			break;
		case EBakeMapType::TangentSpaceNormal:
			PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), PreviewMap);
			PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), EmptyColorMapWhite);
			PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), EmptyColorMapWhite);
			break;
		case EBakeMapType::AmbientOcclusion:
			PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
			PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), PreviewMap);
			PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), EmptyColorMapWhite);
			break;
		case EBakeMapType::BentNormal:
		{
			UTexture2D* AOMap = CachedMaps.Contains(EBakeMapType::AmbientOcclusion) ? CachedMaps[EBakeMapType::AmbientOcclusion] : EmptyColorMapWhite;
			BentNormalPreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
			BentNormalPreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), AOMap);
			BentNormalPreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), EmptyColorMapWhite);
			BentNormalPreviewMaterial->SetTextureParameterValue(TEXT("BentNormalMap"), PreviewMap);
			PreviewMesh->SetOverrideRenderMaterial(BentNormalPreviewMaterial);
			break;
		}	
		case EBakeMapType::Curvature:
		case EBakeMapType::ObjectSpaceNormal:
		case EBakeMapType::FaceNormal:
		case EBakeMapType::Position:
		case EBakeMapType::MaterialID:
		case EBakeMapType::PolyGroupID:
		case EBakeMapType::Texture:
		case EBakeMapType::MultiTexture:
		case EBakeMapType::VertexColor:
		case EBakeMapType::Height:
		case EBakeMapType::UVShell:
			PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
			PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), EmptyColorMapWhite);
			PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), PreviewMap);
			break;
		}
	}
	else
	{
		PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
		PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), EmptyColorMapWhite);
		PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), PreviewMap);
	}
	PreviewMaterial->SetScalarParameterValue(TEXT("UVChannel"), CachedBakeSettings.TargetUVLayer);
	BentNormalPreviewMaterial->SetScalarParameterValue(TEXT("UVChannel"), CachedBakeSettings.TargetUVLayer);
}


void UBakeMeshAttributeMapsToolBase::ResetPreview()
{
	PreviewMaterial->SetTextureParameterValue(TEXT("NormalMap"), EmptyNormalMap);
	PreviewMaterial->SetTextureParameterValue(TEXT("OcclusionMap"), EmptyColorMapWhite);
	PreviewMaterial->SetTextureParameterValue(TEXT("ColorMap"), EmptyColorMapWhite);
}


void UBakeMeshAttributeMapsToolBase::UpdatePreviewNames(
	const EBakeMapType MapTypes,
	FString& MapPreview,
	TArray<FString>& MapPreviewNamesList,
	TMap<FString, FString>& MapPreviewNamesMap)
{
	// Update our preview names list.
	MapPreviewNamesList.Reset();
	MapPreviewNamesMap.Reset();
	bool bFoundMapType = false;
	for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Map : CachedMaps)
	{
		// Only populate map types that were requested. Some map types like
		// AO may have only been added for preview of other types (ex. BentNormal)
		if (static_cast<bool>(MapTypes & Map.Get<0>()))
		{
			const UEnum* BakeTypeEnum = StaticEnum<EBakeMapType>();
			const int64 BakeEnumValue = static_cast<int64>(Map.Get<0>());
			const FString BakeTypeDisplayName = BakeTypeEnum->GetDisplayNameTextByValue(BakeEnumValue).ToString();
			const FString BakeTypeNameString = BakeTypeEnum->GetNameStringByValue(BakeEnumValue);
			MapPreviewNamesList.Add(BakeTypeDisplayName);
			MapPreviewNamesMap.Emplace(BakeTypeDisplayName, BakeTypeNameString);
			if (MapPreview == MapPreviewNamesList.Last())
			{
				bFoundMapType = true;
			}
		}
	}
	if (!bFoundMapType)
	{
		MapPreview = MapPreviewNamesList.Num() > 0 ? MapPreviewNamesList[0] : TEXT("");
	}
}


void UBakeMeshAttributeMapsToolBase::OnMapTypesUpdated(
	const EBakeMapType ResultMapTypes,
	TMap<EBakeMapType, TObjectPtr<UTexture2D>>& Result,
	FString& MapPreview,
	TArray<FString>& MapPreviewNamesList,
	TMap<FString, FString>& MapPreviewNamesMap)
{
	ResetPreview();
	
	// Use the processed bitfield which may contain additional targets
	// (ex. AO if BentNormal was requested) to preallocate cached map storage.
	EBakeMapType CachedMapTypes = GetMapTypes(static_cast<int32>(ResultMapTypes));
	CachedMaps.Empty();
	Result.Empty();
	for (EBakeMapType MapType : ENUM_EBAKEMAPTYPE_ALL)
	{
		if (static_cast<bool>(CachedMapTypes & MapType))
		{
			CachedMaps.Add(MapType, nullptr);
		}
		
		if (static_cast<bool>(ResultMapTypes & MapType))
		{
			Result.Add(MapType, nullptr);
		}
	}
	UpdatePreviewNames(ResultMapTypes, MapPreview, MapPreviewNamesList, MapPreviewNamesMap);
}


void UBakeMeshAttributeMapsToolBase::OnMapsUpdated(const TUniquePtr<UE::Geometry::FMeshMapBaker>& NewResult)
{
	const FImageDimensions BakeDimensions = NewResult->GetDimensions();
	const EBakeTextureBitDepth Format = CachedBakeSettings.BitDepth;
	const int32 NumEval = NewResult->NumEvaluators();
	for (int32 EvalIdx = 0; EvalIdx < NumEval; ++EvalIdx)
	{
		FMeshMapEvaluator* Eval = NewResult->GetEvaluator(EvalIdx);

		auto UpdateCachedMap = [this, &NewResult, &EvalIdx, &BakeDimensions](const EBakeMapType MapType, const EBakeTextureBitDepth MapFormat, const int32 ResultIdx) -> void
		{
			// For 8-bit color textures, ensure that the source data is in sRGB.
			const FTexture2DBuilder::ETextureType TexType = GetTextureType(MapType, MapFormat);
			const bool bConvertToSRGB = TexType == FTexture2DBuilder::ETextureType::Color;
			const ETextureSourceFormat SourceDataFormat = MapFormat == EBakeTextureBitDepth::ChannelBits16 ? TSF_RGBA16F : TSF_BGRA8;
			
			FTexture2DBuilder TextureBuilder;
			TextureBuilder.Initialize(TexType, BakeDimensions);
			TextureBuilder.Copy(*NewResult->GetBakeResults(EvalIdx)[ResultIdx], bConvertToSRGB);
			TextureBuilder.Commit(false);

			// Copy image to source data after commit. This will avoid incurring
			// the cost of hitting the DDC for texture compile while iterating on
			// bake settings. Since this dirties the texture, the next time the texture
			// is used after accepting the final texture, the DDC will trigger and
			// properly recompile the platform data.
			const bool bConvertSourceToSRGB = bConvertToSRGB && SourceDataFormat == TSF_BGRA8;
			TextureBuilder.CopyImageToSourceData(*NewResult->GetBakeResults(EvalIdx)[ResultIdx], SourceDataFormat, bConvertSourceToSRGB);

			// The CachedMap can be thrown out of sync if updated during a background
			// compute. Validate the computed type against our cached maps.
			if (CachedMaps.Contains(MapType))
			{
				CachedMaps[MapType] = TextureBuilder.GetTexture2D();
			}
		};

		switch (Eval->Type())
		{
		case EMeshMapEvaluatorType::Normal:
		{
			constexpr EBakeMapType MapType = EBakeMapType::TangentSpaceNormal;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::Occlusion:
		{
			// Occlusion Evaluator always outputs AmbientOcclusion then BentNormal.
			const FMeshOcclusionMapEvaluator* OcclusionEval = static_cast<FMeshOcclusionMapEvaluator*>(Eval);
			int32 OcclusionIdx = 0;
			if ((bool)(OcclusionEval->OcclusionType & EMeshOcclusionMapType::AmbientOcclusion))
			{
				constexpr EBakeMapType MapType = EBakeMapType::AmbientOcclusion;
				UpdateCachedMap(MapType, Format, OcclusionIdx++);
			}
			if ((bool)(OcclusionEval->OcclusionType & EMeshOcclusionMapType::BentNormal))
			{
				constexpr EBakeMapType MapType = EBakeMapType::BentNormal;
				UpdateCachedMap(MapType, Format, OcclusionIdx++);
			}
			break;
		}
		case EMeshMapEvaluatorType::Curvature:
		{
			constexpr EBakeMapType MapType = EBakeMapType::Curvature;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::Property:
		{
			const FMeshPropertyMapEvaluator* PropertyEval = static_cast<FMeshPropertyMapEvaluator*>(Eval);
			EBakeMapType MapType = EBakeMapType::None;
			switch (PropertyEval->Property)
			{
			case EMeshPropertyMapType::Normal:
				MapType = EBakeMapType::ObjectSpaceNormal;
				break;
			case EMeshPropertyMapType::FacetNormal:
				MapType = EBakeMapType::FaceNormal;
				break;
			case EMeshPropertyMapType::Position:
				MapType = EBakeMapType::Position;
				break;
			case EMeshPropertyMapType::MaterialID:
				MapType = EBakeMapType::MaterialID;
				break;
			case EMeshPropertyMapType::PolyGroupID:
				MapType = EBakeMapType::PolyGroupID;
				break;
			case EMeshPropertyMapType::VertexColor:
				MapType = EBakeMapType::VertexColor;
				break;
			case EMeshPropertyMapType::UVPosition:
			default:
				break;
			}

			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::Height:
		{
			constexpr EBakeMapType MapType = EBakeMapType::Height;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::UVShell:
		{
			constexpr EBakeMapType MapType = EBakeMapType::UVShell;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::ResampleImage:
		{
			constexpr EBakeMapType MapType = EBakeMapType::Texture;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		case EMeshMapEvaluatorType::MultiResampleImage:
		{
			constexpr EBakeMapType MapType = EBakeMapType::MultiTexture;
			UpdateCachedMap(MapType, Format, 0);
			break;
		}
		default:
			break;
		}
	}

	GatherAnalytics(*NewResult, CachedBakeSettings, BakeAnalytics);
	UpdateVisualization();
	GetToolManager()->PostInvalidation();
}


EBakeMapType UBakeMeshAttributeMapsToolBase::GetMapTypes(const int32& MapTypes)
{
	EBakeMapType OutMapTypes = static_cast<EBakeMapType>(MapTypes);
	// Force AO bake for BentNormal preview
	if ((bool)(OutMapTypes & EBakeMapType::BentNormal))
	{
		OutMapTypes |= EBakeMapType::AmbientOcclusion;
	}
	return OutMapTypes;
}


FTexture2DBuilder::ETextureType UBakeMeshAttributeMapsToolBase::GetTextureType(const EBakeMapType MapType, const EBakeTextureBitDepth MapFormat)
{
	FTexture2DBuilder::ETextureType TexType = FTexture2DBuilder::ETextureType::Color;
	switch (MapType)
	{
	default:
		checkNoEntry();
		break;
	case EBakeMapType::TangentSpaceNormal:
		TexType = FTexture2DBuilder::ETextureType::NormalMap;
		break;
	case EBakeMapType::AmbientOcclusion:
		TexType = FTexture2DBuilder::ETextureType::AmbientOcclusion;
		break;
	case EBakeMapType::BentNormal:
		TexType = FTexture2DBuilder::ETextureType::NormalMap;
		break;
	case EBakeMapType::Curvature:
	case EBakeMapType::ObjectSpaceNormal:
	case EBakeMapType::FaceNormal:
	case EBakeMapType::Position:
	case EBakeMapType::Height:
		TexType = FTexture2DBuilder::ETextureType::ColorLinear;
		break;
	case EBakeMapType::MaterialID:
	case EBakeMapType::PolyGroupID:
	case EBakeMapType::VertexColor:
	case EBakeMapType::UVShell:
		TexType = FTexture2DBuilder::ETextureType::Color;
		break;
	case EBakeMapType::Texture:
	case EBakeMapType::MultiTexture:
		// For texture output with 16-bit source data, output HDR texture
		if (MapFormat == EBakeTextureBitDepth::ChannelBits16)
		{
			TexType = FTexture2DBuilder::ETextureType::EmissiveHDR;
		}
		break;
	}
	return TexType;
}


void UBakeMeshAttributeMapsToolBase::GetTextureName(const EBakeMapType MapType, const FString& BaseName, FString& TexName)
{
	switch (MapType)
	{
	default:
		checkNoEntry();
		break;
	case EBakeMapType::TangentSpaceNormal:
		TexName = FString::Printf(TEXT("%s_TangentNormal"), *BaseName);
		break;
	case EBakeMapType::AmbientOcclusion:
		TexName = FString::Printf(TEXT("%s_AmbientOcclusion"), *BaseName);
		break;
	case EBakeMapType::BentNormal:
		TexName = FString::Printf(TEXT("%s_BentNormal"), *BaseName);
		break;
	case EBakeMapType::Curvature:
		TexName = FString::Printf(TEXT("%s_Curvature"), *BaseName);
		break;
	case EBakeMapType::ObjectSpaceNormal:
		TexName = FString::Printf(TEXT("%s_ObjectNormal"), *BaseName);
		break;
	case EBakeMapType::FaceNormal:
		TexName = FString::Printf(TEXT("%s_FaceNormal"), *BaseName);
		break;
	case EBakeMapType::MaterialID:
		TexName = FString::Printf(TEXT("%s_MaterialID"), *BaseName);
		break;
	case EBakeMapType::PolyGroupID:
		TexName = FString::Printf(TEXT("%s_PolyGroupID"), *BaseName);
		break;
	case EBakeMapType::VertexColor:
		TexName = FString::Printf(TEXT("%s_VertexColor"), *BaseName);
		break;
	case EBakeMapType::Position:
		TexName = FString::Printf(TEXT("%s_Position"), *BaseName);
		break;
	case EBakeMapType::Height:
		TexName = FString::Printf(TEXT("%s_Height"), *BaseName);
		break;
	case EBakeMapType::UVShell:
		TexName = FString::Printf(TEXT("%s_UVShell"), *BaseName);
		break;
	case EBakeMapType::Texture:
		TexName = FString::Printf(TEXT("%s_Texture"), *BaseName);
		break;
	case EBakeMapType::MultiTexture:
		TexName = FString::Printf(TEXT("%s_MultiTexture"), *BaseName);
		break;
	}
}


void UBakeMeshAttributeMapsToolBase::InitializeEmptyMaps()
{
	FTexture2DBuilder NormalsBuilder;
	NormalsBuilder.Initialize(FTexture2DBuilder::ETextureType::NormalMap, FImageDimensions(16, 16));
	NormalsBuilder.Commit(false);
	EmptyNormalMap = NormalsBuilder.GetTexture2D();

	FTexture2DBuilder ColorBuilderBlack;
	ColorBuilderBlack.Initialize(FTexture2DBuilder::ETextureType::Color, FImageDimensions(16, 16));
	ColorBuilderBlack.Clear(FColor(0,0,0));
	ColorBuilderBlack.Commit(false);
	EmptyColorMapBlack = ColorBuilderBlack.GetTexture2D();

	FTexture2DBuilder ColorBuilderWhite;
	ColorBuilderWhite.Initialize(FTexture2DBuilder::ETextureType::Color, FImageDimensions(16, 16));
	ColorBuilderWhite.Clear(FColor::White);
	ColorBuilderWhite.Commit(false);
	EmptyColorMapWhite = ColorBuilderWhite.GetTexture2D();
}


void UBakeMeshAttributeMapsToolBase::GatherAnalytics(FBakeAnalytics::FMeshSettings& Data)
{
	
}


void UBakeMeshAttributeMapsToolBase::GatherAnalytics(const FMeshMapBaker& Result, const FBakeSettings& Settings, FBakeAnalytics& Data)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	Data.TotalBakeDuration = Result.BakeAnalytics.TotalBakeDuration;
	Data.WriteToImageDuration = Result.BakeAnalytics.WriteToImageDuration;
	Data.WriteToGutterDuration = Result.BakeAnalytics.WriteToGutterDuration;
	Data.NumSamplePixels = Result.BakeAnalytics.NumSamplePixels;
	Data.NumGutterPixels = Result.BakeAnalytics.NumGutterPixels;
	Data.BakeSettings = Settings;

	const int NumEvaluators = Result.NumEvaluators();
	for (int EvalId = 0; EvalId < NumEvaluators; ++EvalId)
	{
		FMeshMapEvaluator* Eval = Result.GetEvaluator(EvalId);
		switch(Eval->Type())
		{
		case EMeshMapEvaluatorType::Occlusion:
		{
			const FMeshOcclusionMapEvaluator* OcclusionEval = static_cast<FMeshOcclusionMapEvaluator*>(Eval);
			Data.OcclusionSettings.OcclusionRays = OcclusionEval->NumOcclusionRays;
			Data.OcclusionSettings.MaxDistance = OcclusionEval->MaxDistance;
			Data.OcclusionSettings.SpreadAngle = OcclusionEval->SpreadAngle;
			Data.OcclusionSettings.BiasAngle = OcclusionEval->BiasAngleDeg;
			break;
		}
		case EMeshMapEvaluatorType::Curvature:
		{
			const FMeshCurvatureMapEvaluator* CurvatureEval = static_cast<FMeshCurvatureMapEvaluator*>(Eval);
			Data.CurvatureSettings.CurvatureType = static_cast<int>(CurvatureEval->UseCurvatureType);
			Data.CurvatureSettings.RangeMultiplier = CurvatureEval->RangeScale;
			Data.CurvatureSettings.MinRangeMultiplier = CurvatureEval->MinRangeScale;
			Data.CurvatureSettings.ColorMode = static_cast<int>(CurvatureEval->UseColorMode);
			Data.CurvatureSettings.ClampMode = static_cast<int>(CurvatureEval->UseClampMode);
			break;
		}
		default:
			break;
		};
	}
}


void UBakeMeshAttributeMapsToolBase::RecordAnalytics(const FBakeAnalytics& Data, const FString& EventName)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;

	// General
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Duration.Total.Seconds"), Data.TotalBakeDuration));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Duration.WriteToImage.Seconds"), Data.WriteToImageDuration));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Duration.WriteToGutter.Seconds"), Data.WriteToGutterDuration));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Stats.NumSamplePixels"), Data.NumSamplePixels));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Stats.NumGutterPixels"), Data.NumGutterPixels));

	// Mesh data
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.TargetMesh.NumTriangles"), Data.MeshSettings.NumTargetMeshTris));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.DetailMesh.NumMeshes"), Data.MeshSettings.NumDetailMesh));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.DetailMesh.NumTriangles"), Data.MeshSettings.NumDetailMeshTris));

	// Bake settings
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Image.Width"), Data.BakeSettings.Dimensions.GetWidth()));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Image.Height"), Data.BakeSettings.Dimensions.GetHeight()));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.SamplesPerPixel"), Data.BakeSettings.SamplesPerPixel));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.ProjectionDistance"), Data.BakeSettings.ProjectionDistance));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.ProjectionInWorldSpace"), Data.BakeSettings.bProjectionInWorldSpace));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.TargetUVLayer"), Data.BakeSettings.TargetUVLayer));

	// Map types
	const bool bTangentSpaceNormal = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::TangentSpaceNormal);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.TangentSpaceNormal.Enabled"), bTangentSpaceNormal));

	const bool bAmbientOcclusion = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::AmbientOcclusion);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AmbientOcclusion.Enabled"), bAmbientOcclusion));
	if (bAmbientOcclusion)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AmbientOcclusion.OcclusionRays"), Data.OcclusionSettings.OcclusionRays));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AmbientOcclusion.MaxDistance"), Data.OcclusionSettings.MaxDistance));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AmbientOcclusion.SpreadAngle"), Data.OcclusionSettings.SpreadAngle));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.AmbientOcclusion.BiasAngle"), Data.OcclusionSettings.BiasAngle));
	}
	const bool bBentNormal = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::BentNormal);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.BentNormal.Enabled"), bBentNormal));
	if (bBentNormal)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.BentNormal.OcclusionRays"), Data.OcclusionSettings.OcclusionRays));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.BentNormal.MaxDistance"), Data.OcclusionSettings.MaxDistance));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.BentNormal.SpreadAngle"), Data.OcclusionSettings.SpreadAngle));
	}

	const bool bCurvature = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::Curvature);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.Enabled"), bCurvature));
	if (bCurvature)
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.CurvatureType"), Data.CurvatureSettings.CurvatureType));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.RangeMultiplier"), Data.CurvatureSettings.RangeMultiplier));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.MinRangeMultiplier"), Data.CurvatureSettings.MinRangeMultiplier));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.ClampMode"), Data.CurvatureSettings.ClampMode));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Curvature.ColorMode"), Data.CurvatureSettings.ColorMode));
	}

	const bool bObjectSpaceNormal = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::ObjectSpaceNormal);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.ObjectSpaceNormal.Enabled"), bObjectSpaceNormal));

	const bool bFaceNormal = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::FaceNormal);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.FaceNormal.Enabled"), bFaceNormal));

	const bool bPosition = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::Position);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Position.Enabled"), bPosition));

	const bool bMaterialID = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::MaterialID);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.MaterialID.Enabled"), bMaterialID));

	const bool bPolyGroupID = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::PolyGroupID);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.PolyGroupID.Enabled"), bPolyGroupID));
	
	const bool bTexture = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::Texture);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Texture.Enabled"), bTexture));

	const bool bMultiTexture = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::MultiTexture); 
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.MultiTexture.Enabled"), bMultiTexture));

	const bool bVertexColor = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::VertexColor);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.VertexColor.Enabled"), bVertexColor));

	const bool bHeight = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::Height);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Height.Enabled"), bHeight));

	const bool bUVShell = static_cast<bool>(Data.BakeSettings.SourceBakeMapTypes & EBakeMapType::UVShell);
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.UVShell.Enabled"), bUVShell));

	FEngineAnalytics::GetProvider().RecordEvent(FString(TEXT("Editor.Usage.MeshModelingMode.")) + EventName, Attributes);

	constexpr bool bLogAnalytics = false; 
	if constexpr (bLogAnalytics)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "[%ls] %ls = %ls", *EventName, *Attr.GetName(), *Attr.GetValue());
		}
	}
}



#undef LOCTEXT_NAMESPACE

