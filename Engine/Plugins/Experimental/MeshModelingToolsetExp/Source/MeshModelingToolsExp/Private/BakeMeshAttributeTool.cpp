// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeMeshAttributeTool.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "ToolSetupUtil.h"
#include "AssetUtils/Texture2DUtil.h"

#include "Sampling/MeshCurvatureMapEvaluator.h"
#include "Sampling/MeshHeightMapEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshAttributeTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeMeshAttributeTool"


void UBakeMeshAttributeTool::Setup()
{
	Super::Setup();

	// Setup preview materials
	WorkingPreviewMaterial = ToolSetupUtil::GetDefaultWorkingMaterialInstance(GetToolManager());
	ErrorPreviewMaterial = ToolSetupUtil::GetDefaultErrorMaterial(GetToolManager());
}


bool UBakeMeshAttributeTool::ValidTargetMeshTangents()
{
	if (bCheckTargetMeshTangents)
	{
		bValidTargetMeshTangents = TargetMeshTangents ? FDynamicMeshTangents(TargetMesh.Get()).HasValidTangents(true) : false;
		bCheckTargetMeshTangents = false;
	}
	return bValidTargetMeshTangents;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_TargetMeshTangents(EBakeMapType BakeType)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	const bool bNeedTargetMeshTangents = (bool)(BakeType & (EBakeMapType::TangentSpaceNormal | EBakeMapType::BentNormal));
	if (bNeedTargetMeshTangents && !ValidTargetMeshTangents())
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidTargetTangentsWarning", "The Target Mesh does not have valid tangents."), EToolMessageLevel::UserWarning);
		ResultState = EBakeOpState::Invalid;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_Normal(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;

	FNormalMapSettings NormalMapSettings;
	NormalMapSettings.Dimensions = Dimensions;

	if (!(CachedNormalMapSettings == NormalMapSettings))
	{
		CachedNormalMapSettings = NormalMapSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_Occlusion(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	if (!OcclusionSettings)
	{
		return EBakeOpState::Invalid;
	}

	FOcclusionMapSettings OcclusionMapSettings;
	OcclusionMapSettings.Dimensions = Dimensions;
	OcclusionMapSettings.MaxDistance = (OcclusionSettings->MaxDistance == 0) ? TNumericLimits<float>::Max() : OcclusionSettings->MaxDistance;
	OcclusionMapSettings.OcclusionRays = OcclusionSettings->OcclusionRays;
	OcclusionMapSettings.SpreadAngle = OcclusionSettings->SpreadAngle;
	OcclusionMapSettings.BiasAngle = OcclusionSettings->BiasAngle;
	OcclusionMapSettings.NormalSpace = OcclusionSettings->NormalSpace;

	if ( !(CachedOcclusionMapSettings == OcclusionMapSettings) )
	{
		CachedOcclusionMapSettings = OcclusionMapSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_Curvature(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	if (!CurvatureSettings)
	{
		return EBakeOpState::Invalid;
	}

	FCurvatureMapSettings CurvatureMapSettings;
	CurvatureMapSettings.Dimensions = Dimensions;
	CurvatureMapSettings.RangeMultiplier = CurvatureSettings->ColorRangeMultiplier;
	CurvatureMapSettings.MinRangeMultiplier = CurvatureSettings->MinRangeMultiplier;
	switch (CurvatureSettings->CurvatureType)
	{
	default:
	case EBakeCurvatureTypeMode::MeanAverage:
		CurvatureMapSettings.CurvatureType = (int32)FMeshCurvatureMapEvaluator::ECurvatureType::Mean;
		break;
	case EBakeCurvatureTypeMode::Gaussian:
		CurvatureMapSettings.CurvatureType = (int32)FMeshCurvatureMapEvaluator::ECurvatureType::Gaussian;
		break;
	case EBakeCurvatureTypeMode::Max:
		CurvatureMapSettings.CurvatureType = (int32)FMeshCurvatureMapEvaluator::ECurvatureType::MaxPrincipal;
		break;
	case EBakeCurvatureTypeMode::Min:
		CurvatureMapSettings.CurvatureType = (int32)FMeshCurvatureMapEvaluator::ECurvatureType::MinPrincipal;
		break;
	}
	switch (CurvatureSettings->ColorMapping)
	{
	default:
	case EBakeCurvatureColorMode::Grayscale:
		CurvatureMapSettings.ColorMode = (int32)FMeshCurvatureMapEvaluator::EColorMode::BlackGrayWhite;
		break;
	case EBakeCurvatureColorMode::RedBlue:
		CurvatureMapSettings.ColorMode = (int32)FMeshCurvatureMapEvaluator::EColorMode::RedBlue;
		break;
	case EBakeCurvatureColorMode::RedGreenBlue:
		CurvatureMapSettings.ColorMode = (int32)FMeshCurvatureMapEvaluator::EColorMode::RedGreenBlue;
		break;
	}
	switch (CurvatureSettings->Clamping)
	{
	default:
	case EBakeCurvatureClampMode::None:
		CurvatureMapSettings.ClampMode = (int32)FMeshCurvatureMapEvaluator::EClampMode::FullRange;
		break;
	case EBakeCurvatureClampMode::OnlyPositive:
		CurvatureMapSettings.ClampMode = (int32)FMeshCurvatureMapEvaluator::EClampMode::Positive;
		break;
	case EBakeCurvatureClampMode::OnlyNegative:
		CurvatureMapSettings.ClampMode = (int32)FMeshCurvatureMapEvaluator::EClampMode::Negative;
		break;
	}

	if (!(CachedCurvatureMapSettings == CurvatureMapSettings))
	{
		CachedCurvatureMapSettings = CurvatureMapSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_MeshProperty(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;

	FMeshPropertyMapSettings MeshPropertyMapSettings;
	MeshPropertyMapSettings.Dimensions = Dimensions;

	if (!(CachedMeshPropertyMapSettings == MeshPropertyMapSettings))
	{
		CachedMeshPropertyMapSettings = MeshPropertyMapSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_Height(const FImageDimensions& Dimensions)
{
	EBakeOpState ResultState = EBakeOpState::Clean;

	FHeightMapSettings HeightMapSettings;
	HeightMapSettings.Dimensions = Dimensions;
	HeightMapSettings.InnerDistance = HeightSettings->InnerDistance;
	HeightMapSettings.OuterDistance = HeightSettings->OuterDistance;
	HeightMapSettings.InnerBoundsDistance = HeightSettings->InnerBoundsDistance;
	HeightMapSettings.OuterBoundsDistance = HeightSettings->OuterBoundsDistance;
	
	switch (HeightSettings->HeightRangeMode)
	{
	case EBakeHeightRangeMode::Absolute:
		HeightMapSettings.HeightRangeMode = static_cast<int32>(FMeshHeightMapEvaluator::EHeightRangeMode::Absolute);
		break;
	case EBakeHeightRangeMode::RelativeBounds:
		HeightMapSettings.HeightRangeMode = static_cast<int32>(FMeshHeightMapEvaluator::EHeightRangeMode::RelativeBounds);
		break;
	}

	if (!(CachedHeightMapSettings == HeightMapSettings))
	{
		CachedHeightMapSettings = HeightMapSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_Texture2DImage(const FImageDimensions& Dimensions, const FDynamicMesh3* DetailMesh)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	if (!TextureSettings || !DetailMesh)
	{
		return EBakeOpState::Invalid;
	}

	FTexture2DSettings NewSettings;
	NewSettings.Dimensions = Dimensions;
	NewSettings.UVLayer = TextureSettings->UVLayerNamesList.IndexOfByKey(TextureSettings->UVLayer);

	const FDynamicMeshUVOverlay* UVOverlay = DetailMesh->Attributes()->GetUVLayer(NewSettings.UVLayer);
	if (UVOverlay == nullptr)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidUVWarning", "The Source Mesh does not have the selected UV layer"), EToolMessageLevel::UserWarning);
		return EBakeOpState::Invalid;
	}
	
	if (TextureSettings->SourceTexture == nullptr)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidTextureWarning", "The Source Texture is not valid"), EToolMessageLevel::UserWarning);
		return EBakeOpState::Invalid;
	}

	{
		CachedTextureImage = MakeShared<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>();
		if (!UE::AssetUtils::ReadTexture(TextureSettings->SourceTexture, *CachedTextureImage, bPreferPlatformData))
		{
			GetToolManager()->DisplayMessage(LOCTEXT("CannotReadTextureWarning", "Cannot read from the source texture"), EToolMessageLevel::UserWarning);
			return EBakeOpState::Invalid;
		}
	}

	if (!(CachedTexture2DSettings == NewSettings))
	{
		CachedTexture2DSettings = NewSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


EBakeOpState UBakeMeshAttributeTool::UpdateResult_MultiTexture(const FImageDimensions& Dimensions, const FDynamicMesh3* DetailMesh)
{
	EBakeOpState ResultState = EBakeOpState::Clean;
	if (!MultiTextureSettings || !DetailMesh)
	{
		return EBakeOpState::Invalid;
	}

	FTexture2DSettings NewSettings;
	NewSettings.Dimensions = Dimensions;
	NewSettings.UVLayer = MultiTextureSettings->UVLayerNamesList.IndexOfByKey(MultiTextureSettings->UVLayer);

	const FDynamicMeshUVOverlay* UVOverlay = DetailMesh->Attributes()->GetUVLayer(NewSettings.UVLayer);
	if (UVOverlay == nullptr)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidUVWarning", "The Source Mesh does not have the selected UV layer"), EToolMessageLevel::UserWarning);
		return EBakeOpState::Invalid;
	}

	const int NumMaterialIDs = MultiTextureSettings->MaterialIDSourceTextures.Num();
	CachedMultiTextures.Reset();
	CachedMultiTextures.SetNum(NumMaterialIDs);
	int NumValidTextures = 0;
	for ( int MaterialID = 0; MaterialID < NumMaterialIDs; ++MaterialID)
	{
		if (UTexture2D* Texture = MultiTextureSettings->MaterialIDSourceTextures[MaterialID])
		{
			CachedMultiTextures[MaterialID] = MakeShared<TImageBuilder<FVector4f>, ESPMode::ThreadSafe>();
			if (!UE::AssetUtils::ReadTexture(Texture, *CachedMultiTextures[MaterialID], bPreferPlatformData))
			{
				GetToolManager()->DisplayMessage(LOCTEXT("CannotReadTextureWarning", "Cannot read from the source texture"), EToolMessageLevel::UserWarning);
				return EBakeOpState::Invalid;
			}
			++NumValidTextures;
		}
	}
	if (NumValidTextures == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidTextureWarning", "The Source Texture is not valid"), EToolMessageLevel::UserWarning);
		return EBakeOpState::Invalid;
	}

	if (!(CachedMultiTexture2DSettings == NewSettings))
	{
		CachedMultiTexture2DSettings = NewSettings;
		ResultState = EBakeOpState::Evaluate;
	}
	return ResultState;
}


int UBakeMeshAttributeTool::SelectColorTextureToBake(const TArray<UTexture*>& Textures)
{
	TArray<int> TextureVotes;
	TextureVotes.Init(0, Textures.Num());

	for (int TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
	{
		UTexture* Tex = Textures[TextureIndex];
		UTexture2D* Tex2D = Cast<UTexture2D>(Tex);

		if (Tex2D)
		{
			// Texture uses SRGB
			if (Tex->SRGB != 0)
			{
				++TextureVotes[TextureIndex];
			}

#if WITH_EDITORONLY_DATA
			// Texture has multiple channels
			ETextureSourceFormat Format = Tex->Source.GetFormat();
			if (Format == TSF_BGRA8 || Format == TSF_BGRE8 || Format == TSF_RGBA16 || Format == TSF_RGBA16F || Format == TSF_RGBA32F)
			{
				++TextureVotes[TextureIndex];
			}
#endif

			// What else? Largest texture? Most layers? Most mipmaps?
		}
	}

	int MaxIndex = -1;
	int MaxVotes = -1;
	for (int TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
	{
		if (TextureVotes[TextureIndex] > MaxVotes)
		{
			MaxIndex = TextureIndex;
			MaxVotes = TextureVotes[TextureIndex];
		}
	}

	return MaxIndex;
}

void UBakeMeshAttributeTool::UpdateMultiTextureMaterialIDs(
	UToolTarget* Target,
	TArray<TObjectPtr<UTexture2D>>& AllSourceTextures,
	TArray<TObjectPtr<UTexture2D>>& MaterialIDTextures)
{
	ProcessComponentTextures(UE::ToolTarget::GetTargetComponent(Target),
		[&AllSourceTextures, &MaterialIDTextures](const int NumMaterials, const int MaterialID, const TArray<UTexture*>& Textures)
	{
		MaterialIDTextures.SetNumZeroed(NumMaterials);
			
		for (UTexture* Tex : Textures)
		{
			UTexture2D* Tex2D = Cast<UTexture2D>(Tex);
			if (Tex2D)
			{
				AllSourceTextures.Add(Tex2D);
			}
		}

		UTexture2D* Tex2D = nullptr;
		constexpr bool bGuessAtTextures = true;
		if constexpr (bGuessAtTextures)
		{
			const int SelectedTextureIndex = SelectColorTextureToBake(Textures);
			if (SelectedTextureIndex >= 0)
			{
				Tex2D = Cast<UTexture2D>(Textures[SelectedTextureIndex]);	
			}
		}
		MaterialIDTextures[MaterialID] = Tex2D;
	});
}


#undef LOCTEXT_NAMESPACE


