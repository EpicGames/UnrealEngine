// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeMeshAttributeVertexTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "DynamicMesh/MeshTransforms.h"
#include "MeshDescriptionToDynamicMesh.h"

#include "Sampling/MeshNormalMapEvaluator.h"
#include "Sampling/MeshOcclusionMapEvaluator.h"
#include "Sampling/MeshCurvatureMapEvaluator.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "Sampling/MeshResampleImageEvaluator.h"
#include "Sampling/MeshHeightMapEvaluator.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "ToolTargetManager.h"
#include "AssetUtils/Texture2DUtil.h"

#include "EngineAnalytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMeshAttributeVertexTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeMeshAttributeVertexTool"

/*
 * ToolBuilder
 */

bool UBakeMeshAttributeVertexToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	const int32 NumTargets = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	return (NumTargets == 1 || NumTargets == 2);
}

UMultiSelectionMeshEditingTool* UBakeMeshAttributeVertexToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UBakeMeshAttributeVertexTool>(SceneState.ToolManager);
}


/*
 * Operators
 */

class FMeshVertexBakerOp : public TGenericDataOperator<FMeshVertexBaker>
{
public:
	// General bake settings
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> DetailMesh;
	TSharedPtr<FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;
	const FDynamicMesh3* BaseMesh;
	TSharedPtr<TMeshTangents<double>, ESPMode::ThreadSafe> BaseMeshTangents;
	TUniquePtr<FMeshVertexBaker> Baker;
	bool bIsBakeToSelf = false;

	UBakeMeshAttributeVertexTool::FBakeSettings BakeSettings;
	FOcclusionMapSettings OcclusionSettings;
	FCurvatureMapSettings CurvatureSettings;
	FHeightMapSettings HeightSettings;
	FTexture2DSettings TextureSettings;
	FTexture2DSettings MultiTextureSettings;

	// Texture2DImage & MultiTexture settings
	using ImagePtr = TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>;
	ImagePtr TextureImage;
	TArray<ImagePtr> MaterialIDTextures;

	// Begin TGenericDataOperator interface
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		Baker = MakeUnique<FMeshVertexBaker>();
		Baker->CancelF = [Progress]()
		{
			return Progress && Progress->Cancelled();
		};
		Baker->SetTargetMesh(BaseMesh);
		Baker->SetTargetMeshTangents(BaseMeshTangents);
		Baker->SetProjectionDistance(BakeSettings.ProjectionDistance);
		if (bIsBakeToSelf)
		{
			Baker->SetCorrespondenceStrategy(FMeshBaseBaker::ECorrespondenceStrategy::Identity);
		}
		
		Baker->BakeMode = BakeSettings.OutputMode == EBakeVertexOutput::RGBA ? FMeshVertexBaker::EBakeMode::RGBA : FMeshVertexBaker::EBakeMode::PerChannel;
		
		FMeshBakerDynamicMeshSampler DetailSampler(DetailMesh.Get(), DetailSpatial.Get());
		Baker->SetDetailSampler(&DetailSampler);

		auto InitOcclusionEvaluator = [this] (FMeshOcclusionMapEvaluator* OcclusionEval, const EMeshOcclusionMapType OcclusionType)
		{
			OcclusionEval->OcclusionType = OcclusionType;
			OcclusionEval->NumOcclusionRays = OcclusionSettings.OcclusionRays;
			OcclusionEval->MaxDistance = OcclusionSettings.MaxDistance;
			OcclusionEval->SpreadAngle = OcclusionSettings.SpreadAngle;
			OcclusionEval->BiasAngleDeg = OcclusionSettings.BiasAngle;
			switch (OcclusionSettings.NormalSpace)
			{
			case EBakeNormalSpace::Tangent:
				OcclusionEval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Tangent;
				break;
			case EBakeNormalSpace::Object:
				OcclusionEval->NormalSpace = FMeshOcclusionMapEvaluator::ESpace::Object;
				break;
			}
		};

		auto InitCurvatureEvaluator = [this] (FMeshCurvatureMapEvaluator* CurvatureEval)
		{
			CurvatureEval->RangeScale = FMathd::Clamp(CurvatureSettings.RangeMultiplier, 0.0001, 1000.0);
			CurvatureEval->MinRangeScale = FMathd::Clamp(CurvatureSettings.MinRangeMultiplier, 0.0, 1.0);
			CurvatureEval->UseCurvatureType = static_cast<FMeshCurvatureMapEvaluator::ECurvatureType>(CurvatureSettings.CurvatureType);
			CurvatureEval->UseColorMode = static_cast<FMeshCurvatureMapEvaluator::EColorMode>(CurvatureSettings.ColorMode);
			CurvatureEval->UseClampMode = static_cast<FMeshCurvatureMapEvaluator::EClampMode>(CurvatureSettings.ClampMode);
		};

		auto InitHeightEvaluator = [this] (FMeshHeightMapEvaluator* HeightEval)
		{
			HeightEval->RangeMode = (FMeshHeightMapEvaluator::EHeightRangeMode) HeightSettings.HeightRangeMode;
			switch (HeightEval->RangeMode)
			{
			case FMeshHeightMapEvaluator::EHeightRangeMode::Absolute:
				HeightEval->Range = FInterval1f::MakeFromUnordered(HeightSettings.InnerDistance, HeightSettings.OuterDistance);
				break;
			case FMeshHeightMapEvaluator::EHeightRangeMode::RelativeBounds:
				HeightEval->Range = FInterval1f::MakeFromUnordered(HeightSettings.InnerBoundsDistance, HeightSettings.OuterBoundsDistance);
				break;
			}
		};

		if (BakeSettings.OutputMode == EBakeVertexOutput::PerChannel)
		{
			for(int ChannelIdx = 0; ChannelIdx < 4; ++ChannelIdx)
			{
				switch(BakeSettings.OutputTypePerChannel[ChannelIdx])
				{
				case EBakeMapType::AmbientOcclusion:
				{
					TSharedPtr<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe> OcclusionEval = MakeShared<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe>();
					InitOcclusionEvaluator(OcclusionEval.Get(), EMeshOcclusionMapType::AmbientOcclusion);
					Baker->ChannelEvaluators[ChannelIdx] = OcclusionEval;
					break;				
				}
				case EBakeMapType::Curvature:
				{
					TSharedPtr<FMeshCurvatureMapEvaluator, ESPMode::ThreadSafe> CurvatureEval = MakeShared<FMeshCurvatureMapEvaluator, ESPMode::ThreadSafe>();
					InitCurvatureEvaluator(CurvatureEval.Get());
					Baker->ChannelEvaluators[ChannelIdx] = CurvatureEval;
					break;
				}
				case EBakeMapType::Height:
				{
					TSharedPtr<FMeshHeightMapEvaluator> HeightEval = MakeShared<FMeshHeightMapEvaluator>();
					InitHeightEvaluator(HeightEval.Get());
					Baker->ChannelEvaluators[ChannelIdx] = HeightEval;
					break;
				}
				case EBakeMapType::One:
				case EBakeMapType::Zero:
				{
					const float Value = BakeSettings.OutputTypePerChannel[ChannelIdx] == EBakeMapType::One ? 1.0f : 0.0f;
					TSharedPtr<FMeshConstantMapEvaluator> ConstantEval = MakeShared<FMeshConstantMapEvaluator>(Value);
					Baker->ChannelEvaluators[ChannelIdx] = ConstantEval;
					break;
				}
				default:
				case EBakeMapType::None:
				{
					Baker->ChannelEvaluators[ChannelIdx] = nullptr;
					break;
				}
				}
			}
		}
		else // EBakeVertexOutput::RGBA
		{
			switch (BakeSettings.OutputType)
			{
			case EBakeMapType::TangentSpaceNormal:
			{
				Baker->ColorEvaluator = MakeShared<FMeshNormalMapEvaluator, ESPMode::ThreadSafe>();
				break;
			}
			case EBakeMapType::AmbientOcclusion:
			{
				TSharedPtr<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe> OcclusionEval = MakeShared<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe>();
				InitOcclusionEvaluator(OcclusionEval.Get(), EMeshOcclusionMapType::AmbientOcclusion);
				Baker->ColorEvaluator = OcclusionEval;
				break;
			}
			case EBakeMapType::BentNormal:
			{
				TSharedPtr<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe> OcclusionEval = MakeShared<FMeshOcclusionMapEvaluator, ESPMode::ThreadSafe>();
				InitOcclusionEvaluator(OcclusionEval.Get(), EMeshOcclusionMapType::BentNormal);
				Baker->ColorEvaluator = OcclusionEval;
				break;
			}
			case EBakeMapType::Curvature:
			{
				TSharedPtr<FMeshCurvatureMapEvaluator, ESPMode::ThreadSafe> CurvatureEval = MakeShared<FMeshCurvatureMapEvaluator, ESPMode::ThreadSafe>();
				InitCurvatureEvaluator(CurvatureEval.Get());
				Baker->ColorEvaluator = CurvatureEval;
				break;
			}
			case EBakeMapType::Position:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::Position;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			case EBakeMapType::ObjectSpaceNormal:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::Normal;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			case EBakeMapType::FaceNormal:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::FacetNormal;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			case EBakeMapType::MaterialID:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::MaterialID;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			case EBakeMapType::PolyGroupID:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::PolyGroupID;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			case EBakeMapType::Height:
			{
				TSharedPtr<FMeshHeightMapEvaluator> HeightEval = MakeShared<FMeshHeightMapEvaluator>();
				InitHeightEvaluator(HeightEval.Get());
				Baker->ColorEvaluator = HeightEval;
				break;
			}
			case EBakeMapType::Texture:
			{
				TSharedPtr<FMeshResampleImageEvaluator, ESPMode::ThreadSafe> TextureEval = MakeShared<FMeshResampleImageEvaluator, ESPMode::ThreadSafe>();
				DetailSampler.SetTextureMap(DetailMesh.Get(), IMeshBakerDetailSampler::FBakeDetailTexture(TextureImage.Get(), TextureSettings.UVLayer));
				Baker->ColorEvaluator = TextureEval;
				break;
			}
			case EBakeMapType::MultiTexture:
			{
				TSharedPtr<FMeshMultiResampleImageEvaluator, ESPMode::ThreadSafe> TextureEval = MakeShared<FMeshMultiResampleImageEvaluator, ESPMode::ThreadSafe>();
				TextureEval->DetailUVLayer = MultiTextureSettings.UVLayer;
				TextureEval->MultiTextures = MaterialIDTextures;
				Baker->ColorEvaluator = TextureEval;
				break;
			}
			case EBakeMapType::VertexColor:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::VertexColor;
				Baker->ColorEvaluator = PropertyEval;
				break;
			}
			default:
				break;
			}
		}

		Baker->Bake();
		SetResult(MoveTemp(Baker));
	}

	// End TGenericDataOperator interface
};


/*
 * Tool
 */

void UBakeMeshAttributeVertexTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMeshAttributeVertexTool::Setup);
	Super::Setup();

	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexColorMaterial"));
	check(Material);
	if (Material != nullptr)
	{
		PreviewMaterial = UMaterialInstanceDynamic::Create(Material, GetToolManager());
	}

	UMaterial* AlphaMaterial = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexAlphaMaterial"));
	check(AlphaMaterial);
	if (AlphaMaterial != nullptr)
	{
		PreviewAlphaMaterial = UMaterialInstanceDynamic::Create(AlphaMaterial, GetToolManager());
	}

	bIsBakeToSelf = (Targets.Num() == 1);

	UE::ToolTarget::HideSourceObject(Targets[0]);

	// TargetMesh stores the original target mesh. It is intended to remain
	// const throughout this tool and is used to refresh the PreviewMesh back
	// to its original state.
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	TargetMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Targets[0], GetMeshParams));
	TargetMeshTangents = MakeShared<FMeshTangentsd, ESPMode::ThreadSafe>(TargetMesh.Get());
	TargetMeshTangents->CopyTriVertexTangents(*TargetMesh);

	// PreviewMesh stores computed result mesh. On shutdown, PreviewMesh will be
	// used to commit the dynamic mesh to the target tool target.
	PreviewMesh = CreateBakePreviewMesh(this, Targets[0], GetTargetWorld());
	PreviewMesh->SetOverrideRenderMaterial(PreviewMaterial);

	UToolTarget* Target = Targets[0];
	UToolTarget* DetailTarget = Targets[bIsBakeToSelf ? 0 : 1];

	// Setup tool property sets

	Settings = NewObject<UBakeMeshAttributeVertexToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->OutputMode, [this](EBakeVertexOutput) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->OutputType, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->OutputTypeR, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->OutputTypeG, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->OutputTypeB, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->OutputTypeA, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->PreviewMode, [this](EBakeVertexChannel) { UpdateVisualization(); });
	Settings->WatchProperty(Settings->TopologyMode, [this](EBakeVertexTopology) { bColorTopologyValid = false; OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->bSplitAtNormalSeams, [this](bool) { bColorTopologyValid = false; OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->bSplitAtUVSeams, [this](bool) { bColorTopologyValid = false; OpState |= EBakeOpState::Evaluate; });

	InputMeshSettings = NewObject<UBakeInputMeshProperties>(this);
	InputMeshSettings->RestoreProperties(this);
	AddToolPropertySource(InputMeshSettings);
	SetToolPropertySourceEnabled(InputMeshSettings, true);
	InputMeshSettings->bHasTargetUVLayer = false;
	InputMeshSettings->bHasSourceNormalMap = false;
	InputMeshSettings->TargetStaticMesh = UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetSkeletalMesh = UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetDynamicMesh = GetTargetActorViaIPersistentDynamicMeshSource(Target);
	InputMeshSettings->SourceStaticMesh = !bIsBakeToSelf ? UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(DetailTarget) : nullptr;
	InputMeshSettings->SourceSkeletalMesh = !bIsBakeToSelf ? UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(DetailTarget) : nullptr;
	InputMeshSettings->SourceDynamicMesh = !bIsBakeToSelf ? GetTargetActorViaIPersistentDynamicMeshSource(DetailTarget) : nullptr;
	InputMeshSettings->SourceNormalMap = nullptr;
	InputMeshSettings->WatchProperty(InputMeshSettings->bHideSourceMesh, [this](bool bState) { SetSourceObjectVisible(!bState); });
	InputMeshSettings->WatchProperty(InputMeshSettings->ProjectionDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	InputMeshSettings->WatchProperty(InputMeshSettings->bProjectionInWorldSpace, [this](bool) { OpState |= EBakeOpState::EvaluateDetailMesh; });
	SetSourceObjectVisible(!InputMeshSettings->bHideSourceMesh);

	OcclusionSettings = NewObject<UBakeOcclusionMapToolProperties>(this);
	OcclusionSettings->RestoreProperties(this);
	AddToolPropertySource(OcclusionSettings);
	SetToolPropertySourceEnabled(OcclusionSettings, false);
	OcclusionSettings->WatchProperty(OcclusionSettings->OcclusionRays, [this](int32) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->MaxDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->SpreadAngle, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->BiasAngle, [this](float) { OpState |= EBakeOpState::Evaluate; });
	OcclusionSettings->WatchProperty(OcclusionSettings->NormalSpace, [this](EBakeNormalSpace) { OpState |= EBakeOpState::Evaluate; });

	CurvatureSettings = NewObject<UBakeCurvatureMapToolProperties>(this);
	CurvatureSettings->RestoreProperties(this);
	AddToolPropertySource(CurvatureSettings);
	SetToolPropertySourceEnabled(CurvatureSettings, false);
	CurvatureSettings->WatchProperty(CurvatureSettings->ColorRangeMultiplier, [this](float) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->MinRangeMultiplier, [this](float) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->CurvatureType, [this](EBakeCurvatureTypeMode) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->ColorMapping, [this](EBakeCurvatureColorMode) { OpState |= EBakeOpState::Evaluate; });
	CurvatureSettings->WatchProperty(CurvatureSettings->Clamping, [this](EBakeCurvatureClampMode) { OpState |= EBakeOpState::Evaluate; });

	HeightSettings = NewObject<UBakeHeightMapToolProperties>(this);
	HeightSettings->RestoreProperties(this);
	AddToolPropertySource(HeightSettings);
	SetToolPropertySourceEnabled(HeightSettings, false);
	HeightSettings->WatchProperty(HeightSettings->HeightRangeMode, [this](EBakeHeightRangeMode) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->InnerDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->OuterDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->InnerBoundsDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });
	HeightSettings->WatchProperty(HeightSettings->OuterBoundsDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });

	TextureSettings = NewObject<UBakeTexture2DProperties>(this);
	TextureSettings->RestoreProperties(this);
	AddToolPropertySource(TextureSettings);
	SetToolPropertySourceEnabled(TextureSettings, false);
	TextureSettings->WatchProperty(TextureSettings->UVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	TextureSettings->WatchProperty(TextureSettings->SourceTexture, [this](UTexture2D*) { OpState |= EBakeOpState::Evaluate; });

	MultiTextureSettings = NewObject<UBakeMultiTexture2DProperties>(this);
	MultiTextureSettings->RestoreProperties(this);
	AddToolPropertySource(MultiTextureSettings);
	SetToolPropertySourceEnabled(MultiTextureSettings, false);
	auto SetDirtyCallback = [this](decltype(MultiTextureSettings->MaterialIDSourceTextures)) { OpState |= EBakeOpState::Evaluate; };
	auto NotEqualsCallback = [](const decltype(MultiTextureSettings->MaterialIDSourceTextures)& A, const decltype(MultiTextureSettings->MaterialIDSourceTextures)& B) -> bool { return A != B; };
	MultiTextureSettings->WatchProperty(MultiTextureSettings->MaterialIDSourceTextures, SetDirtyCallback, NotEqualsCallback);
	MultiTextureSettings->WatchProperty(MultiTextureSettings->UVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	UpdateMultiTextureMaterialIDs(DetailTarget, MultiTextureSettings->AllSourceTextures, MultiTextureSettings->MaterialIDSourceTextures);

	UpdateOnModeChange();

	UpdateDetailMesh();

	SetToolDisplayName(LOCTEXT("ToolName", "Bake Vertex Colors"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool",
		        "Bake Vertex Colors. Select Bake Mesh (LowPoly) first, then (optionally) Detail Mesh second."),
		EToolMessageLevel::UserNotification);

	// Initialize background compute
	Compute = MakeUnique<TGenericDataBackgroundCompute<FMeshVertexBaker>>();
	Compute->Setup(this);
	Compute->OnResultUpdated.AddLambda([this](const TUniquePtr<FMeshVertexBaker>& NewResult)
	{
		OnResultUpdated(NewResult);
	});

	GatherAnalytics(BakeAnalytics.MeshSettings);
}

void UBakeMeshAttributeVertexTool::OnShutdown(EToolShutdownType ShutdownType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMeshAttributeVertexTool::Shutdown);
	
	Settings->SaveProperties(this);
	InputMeshSettings->SaveProperties(this);
	OcclusionSettings->SaveProperties(this);
	CurvatureSettings->SaveProperties(this);
	HeightSettings->SaveProperties(this);
	TextureSettings->SaveProperties(this);
	MultiTextureSettings->SaveProperties(this);

	UE::ToolTarget::ShowSourceObject(Targets[0]);
	SetSourceObjectVisible(true);

	Compute->Shutdown();

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("BakeMeshAttributeVertexToolTransactionName",
													   "Bake Mesh Attribute Vertex"));
		FConversionToMeshDescriptionOptions ConvertOptions;
		ConvertOptions.SetToVertexColorsOnly();
		ConvertOptions.bTransformVtxColorsSRGBToLinear = true;
		UE::ToolTarget::CommitDynamicMeshUpdate(
			Targets[0],
			*PreviewMesh->GetMesh(),
			false, // bHaveModifiedTopology
			ConvertOptions);
		GetToolManager()->EndUndoTransaction();
	}

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	RecordAnalytics(BakeAnalytics, TEXT("BakeVertex"));
}

void UBakeMeshAttributeVertexTool::OnTick(float DeltaTime)
{
	Compute->Tick(DeltaTime);

	if (static_cast<bool>(OpState & EBakeOpState::Invalid))
	{
		PreviewMesh->SetOverrideRenderMaterial(ErrorPreviewMaterial);
	}
	else if (!CanAccept() && Compute->GetElapsedComputeTime() > SecondsBeforeWorkingMaterial)
	{
		PreviewMesh->SetOverrideRenderMaterial(WorkingPreviewMaterial);
	}
}

void UBakeMeshAttributeVertexTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UpdateResult();
}

bool UBakeMeshAttributeVertexTool::CanAccept() const
{
	const bool bValidOp = (OpState & EBakeOpState::Invalid) != EBakeOpState::Invalid;
	return bValidOp && Compute->HaveValidResult();
}

TUniquePtr<UE::Geometry::TGenericDataOperator<FMeshVertexBaker>> UBakeMeshAttributeVertexTool::MakeNewOperator()
{
	TUniquePtr<FMeshVertexBakerOp> Op = MakeUnique<FMeshVertexBakerOp>();
	Op->DetailMesh = DetailMesh;
	Op->DetailSpatial = DetailSpatial;

	// Pass the PreviewMesh here instead of the TargetMesh. The PreviewMesh
	// contains the updated color topology. TargetMesh holds onto the original
	// color topology and values.
	Op->BaseMesh = PreviewMesh->GetMesh();
	Op->BaseMeshTangents = TargetMeshTangents;
	Op->BakeSettings = CachedBakeSettings;
	Op->OcclusionSettings = CachedOcclusionMapSettings;
	Op->CurvatureSettings = CachedCurvatureMapSettings;
	Op->HeightSettings = CachedHeightMapSettings;
	Op->TextureSettings = CachedTexture2DSettings;
	Op->MultiTextureSettings = CachedMultiTexture2DSettings;
	Op->bIsBakeToSelf = bIsBakeToSelf;

	// Texture2DImage & MultiTexture settings
	Op->TextureImage = CachedTextureImage;
	Op->MaterialIDTextures = CachedMultiTextures;
	return Op;
}

void UBakeMeshAttributeVertexTool::UpdateDetailMesh()
{
	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[0]);
	IPrimitiveComponentBackedTarget* DetailComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[bIsBakeToSelf ? 0 : 1]);
	UToolTarget* DetailTargetMesh = Targets[bIsBakeToSelf ? 0 : 1];

	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	const FDynamicMesh3 DetailMeshCopy = UE::ToolTarget::GetDynamicMeshCopy(DetailTargetMesh, GetMeshParams);
	DetailMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	DetailMesh->Copy(DetailMeshCopy);
	if (InputMeshSettings->bProjectionInWorldSpace && bIsBakeToSelf == false)
	{
		const FTransformSRT3d DetailToWorld(DetailComponent->GetWorldTransform());
		MeshTransforms::ApplyTransform(*DetailMesh, DetailToWorld, true);
		const FTransformSRT3d WorldToBase(TargetComponent->GetWorldTransform());
		MeshTransforms::ApplyTransformInverse(*DetailMesh, WorldToBase, true);
	}

	DetailSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>();
	DetailSpatial->SetMesh(DetailMesh.Get(), true);

	UpdateUVLayerNames(TextureSettings->UVLayer, TextureSettings->UVLayerNamesList, *DetailMesh);
	UpdateUVLayerNames(MultiTextureSettings->UVLayer, MultiTextureSettings->UVLayerNamesList, *DetailMesh);

	OpState &= ~EBakeOpState::EvaluateDetailMesh;
	OpState |= EBakeOpState::Evaluate;
	DetailMeshTimestamp++;
}

void UBakeMeshAttributeVertexTool::UpdateOnModeChange()
{
	SetToolPropertySourceEnabled(OcclusionSettings, false);
	SetToolPropertySourceEnabled(CurvatureSettings, false);
	SetToolPropertySourceEnabled(HeightSettings, false);
	SetToolPropertySourceEnabled(TextureSettings, false);
	SetToolPropertySourceEnabled(MultiTextureSettings, false);

	if (Settings->OutputMode == EBakeVertexOutput::RGBA)
	{
		switch (static_cast<EBakeMapType>(Settings->OutputType))
		{
			case EBakeMapType::AmbientOcclusion:
			case EBakeMapType::BentNormal:
				SetToolPropertySourceEnabled(OcclusionSettings, true);
				break;
			case EBakeMapType::Curvature:
				SetToolPropertySourceEnabled(CurvatureSettings, true);
				break;
			case EBakeMapType::Height:
				SetToolPropertySourceEnabled(HeightSettings, true);
				break;
			case EBakeMapType::Texture:
				SetToolPropertySourceEnabled(TextureSettings, true);
				break;
			case EBakeMapType::MultiTexture:
				SetToolPropertySourceEnabled(MultiTextureSettings, true);
				break;
			default:
				// No property sets to show.
				break;
		}
	}
	else // Settings->VertexMode == EBakeVertexOutput::PerChannel
	{
		const EBakeMapType PerChannelTypes[4] = {
			static_cast<EBakeMapType>(Settings->OutputTypeR),
			static_cast<EBakeMapType>(Settings->OutputTypeG),
			static_cast<EBakeMapType>(Settings->OutputTypeB),
			static_cast<EBakeMapType>(Settings->OutputTypeA)
		};
		for(int Idx = 0; Idx < 4; ++Idx)
		{
			switch(PerChannelTypes[Idx])
			{
				case EBakeMapType::AmbientOcclusion:
					SetToolPropertySourceEnabled(OcclusionSettings, true);
					break;
				case EBakeMapType::Curvature:
					SetToolPropertySourceEnabled(CurvatureSettings, true);
					break;
				case EBakeMapType::Height:
					SetToolPropertySourceEnabled(HeightSettings, true);
					break;
				case EBakeMapType::None:
				default:
					break;
			}
		}
	}
}

void UBakeMeshAttributeVertexTool::UpdateVisualization()
{
	if (Settings->PreviewMode == EBakeVertexChannel::A)
	{
		PreviewMesh->SetOverrideRenderMaterial(PreviewAlphaMaterial);
	}
	else
	{
		FLinearColor Mask(FLinearColor::Black);
		switch(Settings->PreviewMode)
		{
		case EBakeVertexChannel::R:
			Mask.R = 1.0f;
			break;
		case EBakeVertexChannel::G:
			Mask.G = 1.0f;
			break;
		case EBakeVertexChannel::B:
			Mask.B = 1.0f;
			break;
		case EBakeVertexChannel::RGBA:
		default:
			Mask = FLinearColor::White;
			break;
		}
		PreviewMaterial->SetVectorParameterValue("VertexColorMask", Mask);
		PreviewMesh->SetOverrideRenderMaterial(PreviewMaterial);
	}
}

/** Regenerates the VertexColorOverlay topology of the PreviewMesh per our Settings */
void UBakeMeshAttributeVertexTool::UpdateColorTopology()
{
	// Update PreviewMesh color topology
	if (Settings->TopologyMode == EBakeVertexTopology::CreateNew)
	{
		PreviewMesh->EditMesh([this](FDynamicMesh3& Mesh)
		{
			Mesh.EnableAttributes();
			Mesh.Attributes()->EnablePrimaryColors();
			Mesh.Attributes()->PrimaryColors()->ClearElements();

			FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
			FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();
			Mesh.Attributes()->PrimaryColors()->CreateFromPredicate(
				[this, NormalOverlay, UVOverlay](int /*ParentVID*/, int TriIDA, int TriIDB) -> bool
				{
					auto OverlayCanShare = [TriIDA, TriIDB] (auto Overlay) -> bool
					{
						return Overlay ? Overlay->AreTrianglesConnected(TriIDA, TriIDB) : true;
					};
				
					bool bCanShare = true;
					if (Settings->bSplitAtNormalSeams)
					{
						bCanShare = bCanShare && OverlayCanShare(NormalOverlay);
					}
					if (Settings->bSplitAtUVSeams)
					{
						bCanShare = bCanShare && OverlayCanShare(UVOverlay);
					}
					return bCanShare;
				}, 0.0f);
		});
	}
	else
	{
		PreviewMesh->EditMesh([this](FDynamicMesh3& Mesh)
		{
			// Copy original color overlay from TargetMesh
			Mesh.EnableAttributes();
			Mesh.Attributes()->EnablePrimaryColors();
			Mesh.Attributes()->PrimaryColors()->ClearElements();
			
			if (TargetMesh->Attributes() && TargetMesh->Attributes()->PrimaryColors())
			{
				Mesh.Attributes()->PrimaryColors()->Copy(*TargetMesh->Attributes()->PrimaryColors());
			}
		});
	}

	// Copy source vertex colors onto new color overlay topology.
	UpdateSourceVertexColors();
	
	NumColorElements = PreviewMesh->GetMesh()->Attributes()->PrimaryColors()->ElementCount();

	bColorTopologyValid = true;
}

/**	Copies the vertex colors from the TargetMesh to the PreviewMesh */
void UBakeMeshAttributeVertexTool::UpdateSourceVertexColors()
{
	PreviewMesh->EditMesh([this](FDynamicMesh3& Mesh)
	{
		const FDynamicMeshColorOverlay* TargetColorOverlay = TargetMesh->HasAttributes() ? TargetMesh->Attributes()->PrimaryColors() : nullptr;
		FDynamicMeshColorOverlay* PreviewColorOverlay = Mesh.Attributes()->PrimaryColors(); 
		if (TargetColorOverlay)
		{
			for (int VId : Mesh.VertexIndicesItr())
			{
				Mesh.EnumerateVertexTriangles(VId, [VId, TargetColorOverlay, PreviewColorOverlay](int32 TriID)
				{
					const FVector4f TargetColor = TargetColorOverlay->GetElementAtVertex(TriID, VId);
					const int ElemId = PreviewColorOverlay->GetElementIDAtVertex(TriID, VId);
					PreviewColorOverlay->SetElement(ElemId, TargetColor);
				});
			}
		}
	});
}


void UBakeMeshAttributeVertexTool::UpdateResult()
{
	if (static_cast<bool>(OpState & EBakeOpState::EvaluateDetailMesh))
	{
		UpdateDetailMesh();
	}

	if (!bColorTopologyValid)
	{
		UpdateColorTopology();
	}

	if (OpState == EBakeOpState::Clean)
	{
		return;
	}

	// clear warning (ugh)
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

	FBakeSettings BakeSettings;
	BakeSettings.OutputMode = Settings->OutputMode;
	BakeSettings.OutputType = static_cast<EBakeMapType>(Settings->OutputType);
	BakeSettings.OutputTypePerChannel[0] = static_cast<EBakeMapType>(Settings->OutputTypeR);
	BakeSettings.OutputTypePerChannel[1] = static_cast<EBakeMapType>(Settings->OutputTypeG);
	BakeSettings.OutputTypePerChannel[2] = static_cast<EBakeMapType>(Settings->OutputTypeB);
	BakeSettings.OutputTypePerChannel[3] = static_cast<EBakeMapType>(Settings->OutputTypeA);
	BakeSettings.TopologyMode = Settings->TopologyMode;
	BakeSettings.bSplitAtNormalSeams = Settings->bSplitAtNormalSeams;
	BakeSettings.bSplitAtUVSeams = Settings->bSplitAtUVSeams;
	BakeSettings.bProjectionInWorldSpace = InputMeshSettings->bProjectionInWorldSpace;
	BakeSettings.ProjectionDistance = InputMeshSettings->ProjectionDistance;
	if (!(BakeSettings == CachedBakeSettings))
	{
		CachedBakeSettings = BakeSettings;
	}

	// Clear our invalid bitflag to check again for valid inputs.
	OpState &= ~EBakeOpState::Invalid;

	// Validate bake inputs
	if (NumColorElements <= 0)
	{
		if (Settings->TopologyMode == EBakeVertexTopology::UseExisting)
		{
			GetToolManager()->DisplayMessage(LOCTEXT("InvalidExistingVertexColorTopology", "Topology Mode is set to UseExisting, but no valid existing vertex color topology was found."), EToolMessageLevel::UserWarning);
		}
		else
		{
			GetToolManager()->DisplayMessage(LOCTEXT("InvalidNewVertexColorTopology", "Invalid vertex color topology."), EToolMessageLevel::UserWarning);
		}
		OpState |= EBakeOpState::Invalid;
	}
	
	const FImageDimensions Dimensions(NumColorElements, 1);
	if (CachedBakeSettings.OutputMode == EBakeVertexOutput::RGBA)
	{
		switch(CachedBakeSettings.OutputType)
		{
		case EBakeMapType::TangentSpaceNormal:
			OpState |= UpdateResult_Normal(Dimensions);
			break;
		case EBakeMapType::AmbientOcclusion:
			OpState |= UpdateResult_Occlusion(Dimensions);
			break;
		case EBakeMapType::BentNormal:
			OpState |= UpdateResult_Occlusion(Dimensions);
			break;
		case EBakeMapType::Curvature:
			OpState |= UpdateResult_Curvature(Dimensions);
			break;
		case EBakeMapType::ObjectSpaceNormal:
		case EBakeMapType::FaceNormal:
		case EBakeMapType::Position:
		case EBakeMapType::MaterialID:
		case EBakeMapType::PolyGroupID:
			OpState |= UpdateResult_MeshProperty(Dimensions);
			break;
		case EBakeMapType::Height:
			OpState |= UpdateResult_Height(Dimensions);
			break;
		case EBakeMapType::VertexColor:
			OpState |= UpdateResult_MeshProperty(Dimensions);
			// Force copy the original vertex colors to our PreviewMesh so that
			// the baker samples the source vertex colors for identity bakes.
			UpdateSourceVertexColors();
			break;
		case EBakeMapType::Texture:
			OpState |= UpdateResult_Texture2DImage(Dimensions, DetailMesh.Get());
			break;
		case EBakeMapType::MultiTexture:
			OpState |= UpdateResult_MultiTexture(Dimensions, DetailMesh.Get());
			break;
		default:
			break;
		}

		OpState |= UpdateResult_TargetMeshTangents(CachedBakeSettings.OutputType);
	}
	else // CachedBakeSettings.VertexMode == EBakeVertexOutput::PerChannel
	{
		// The enabled state of these settings are precomputed in UpdateOnModeChange().
		if (OcclusionSettings->IsPropertySetEnabled())
		{
			OpState |= UpdateResult_Occlusion(Dimensions);
		}
		if (CurvatureSettings->IsPropertySetEnabled())
		{
			OpState |= UpdateResult_Curvature(Dimensions);
		}
		if (HeightSettings->IsPropertySetEnabled())
		{
			OpState |= UpdateResult_Height(Dimensions);
		}
		
		// Always force copy the original vertex colors to our PreviewMesh for
		// PerChannel bakes so that channels that are not targeted persist
		// through the bake.
		UpdateSourceVertexColors();
	}

	// Early exit if op input parameters are invalid.
	if ((bool)(OpState & EBakeOpState::Invalid))
	{
		return;
	}

	Compute->InvalidateResult();
	OpState = EBakeOpState::Clean;
}

void UBakeMeshAttributeVertexTool::OnResultUpdated(const TUniquePtr<FMeshVertexBaker>& NewResult)
{
	const TImageBuilder<FVector4f>* ImageResult = NewResult->GetBakeResult();
	if (!ImageResult)
	{
		return;
	}

	PreviewMesh->DeferredEditMesh([this, &ImageResult](FDynamicMesh3& Mesh)
	{
		if (CachedBakeSettings.OutputMode == EBakeVertexOutput::PerChannel)
		{
			// Precompute scale vectors for source and image pixel data to merge
			// the data according to the populated channels.
			FVector4f SrcScale = FVector4f::Zero();
			FVector4f ImgScale = FVector4f::Zero();
			for (int ChannelIdx = 0; ChannelIdx < 4; ++ChannelIdx)
			{
				const bool bOutputChannel = CachedBakeSettings.OutputTypePerChannel[ChannelIdx] != EBakeMapType::None;
				SrcScale[ChannelIdx] = static_cast<float>(!bOutputChannel);
				ImgScale[ChannelIdx] = static_cast<float>(bOutputChannel);
			}
			
			const int NumColors = Mesh.Attributes()->PrimaryColors()->ElementCount();
			check(NumColors == ImageResult->GetDimensions().GetWidth());
			for (int Idx = 0; Idx < NumColors; ++Idx)
			{
				if (const FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors())
				{
					FVector4f Pixel;
					ColorOverlay->GetElement(Idx, Pixel);
					Pixel *= SrcScale;

					// Merge the ImageResult pixels with the source vertex colors based on the requested channels.
					const FVector4f& ImagePixel = ImageResult->GetPixel(Idx);
					Pixel += ImagePixel * ImgScale;
					Mesh.Attributes()->PrimaryColors()->SetElement(Idx, Pixel);
				}
			}
		}
		else //if (Settings->OutputMode == EBakeVertexOutput::RGBA)
		{
			const int NumColors = Mesh.Attributes()->PrimaryColors()->ElementCount();
			check(NumColors == ImageResult->GetDimensions().GetWidth());
			for (int Idx = 0; Idx < NumColors; ++Idx)
			{
				const FVector4f& Pixel = ImageResult->GetPixel(Idx);
				Mesh.Attributes()->PrimaryColors()->SetElement(Idx, Pixel);
			}
		}
	}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::VertexColors, false);
	UpdateVisualization();

	GatherAnalytics(*NewResult, CachedBakeSettings, BakeAnalytics);
}

void UBakeMeshAttributeVertexTool::GatherAnalytics(FBakeAnalytics::FMeshSettings& Data)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	Data.NumTargetMeshVerts = TargetMesh->VertexCount();
	Data.NumTargetMeshTris = TargetMesh->TriangleCount();
	Data.NumDetailMesh = 1;
	Data.NumDetailMeshTris = DetailMesh->TriangleCount();
}


void UBakeMeshAttributeVertexTool::GatherAnalytics(
	const FMeshVertexBaker& Result,
	const FBakeSettings& Settings,
	FBakeAnalytics& Data)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	Data.TotalBakeDuration = Result.TotalBakeDuration;
	Data.BakeSettings = Settings;

	auto GatherEvaluatorData = [&Data](const FMeshMapEvaluator* Eval)
	{
		if (Eval)
		{
			switch(Eval->Type())
			{
			case EMeshMapEvaluatorType::Occlusion:
			{
				const FMeshOcclusionMapEvaluator* OcclusionEval = static_cast<const FMeshOcclusionMapEvaluator*>(Eval);
				Data.OcclusionSettings.OcclusionRays = OcclusionEval->NumOcclusionRays;
				Data.OcclusionSettings.MaxDistance = OcclusionEval->MaxDistance;
				Data.OcclusionSettings.SpreadAngle = OcclusionEval->SpreadAngle;
				Data.OcclusionSettings.BiasAngle = OcclusionEval->BiasAngleDeg;
				break;
			}
			case EMeshMapEvaluatorType::Curvature:
			{
				const FMeshCurvatureMapEvaluator* CurvatureEval = static_cast<const FMeshCurvatureMapEvaluator*>(Eval);
				Data.CurvatureSettings.CurvatureType = static_cast<int>(CurvatureEval->UseCurvatureType);
				Data.CurvatureSettings.RangeMultiplier = CurvatureEval->RangeScale;
				Data.CurvatureSettings.MinRangeMultiplier = CurvatureEval->MinRangeScale;
				Data.CurvatureSettings.ColorMode = static_cast<int>(CurvatureEval->UseColorMode);
				Data.CurvatureSettings.ClampMode = static_cast<int>(CurvatureEval->UseClampMode);
				break;
			}
			default:
				break;
			};
		}
	};

	if (Result.BakeMode == FMeshVertexBaker::EBakeMode::RGBA)
	{
		GatherEvaluatorData(Result.ColorEvaluator.Get());
	}
	else // Result.BakeMode == FMeshVertexBaker::EBakeMode::Channel
	{
		for (int EvalId = 0; EvalId < 4; ++EvalId)
		{
			GatherEvaluatorData(Result.ChannelEvaluators[EvalId].Get());
		}
	}
}


void UBakeMeshAttributeVertexTool::RecordAnalytics(const FBakeAnalytics& Data, const FString& EventName)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}
	
	TArray<FAnalyticsEventAttribute> Attributes;

	// General
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Bake.Duration.Total.Seconds"), Data.TotalBakeDuration));

	// Mesh data
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.TargetMesh.NumTriangles"), Data.MeshSettings.NumTargetMeshTris));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.TargetMesh.NumVertices"), Data.MeshSettings.NumTargetMeshVerts));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.DetailMesh.NumMeshes"), Data.MeshSettings.NumDetailMesh));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Input.DetailMesh.NumTriangles"), Data.MeshSettings.NumDetailMeshTris));

	// Bake settings
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Split.NormalSeams"), Data.BakeSettings.bSplitAtNormalSeams));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Split.UVSeams"), Data.BakeSettings.bSplitAtUVSeams));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.ProjectionDistance"), Data.BakeSettings.ProjectionDistance));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.ProjectionInWorldSpace"), Data.BakeSettings.bProjectionInWorldSpace));

	const FString OutputType = Data.BakeSettings.OutputMode == EBakeVertexOutput::RGBA ? TEXT("RGBA") : TEXT("PerChannel");
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Settings.Output.Type"), OutputType));

	auto RecordAmbientOcclusionSettings = [&Attributes, &Data](const FString& ModeName)
	{
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.AmbientOcclusion.OcclusionRays"), *ModeName), Data.OcclusionSettings.OcclusionRays));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.AmbientOcclusion.MaxDistance"), *ModeName), Data.OcclusionSettings.MaxDistance));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.AmbientOcclusion.SpreadAngle"), *ModeName), Data.OcclusionSettings.SpreadAngle));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.AmbientOcclusion.BiasAngle"), *ModeName), Data.OcclusionSettings.BiasAngle));
	};

	auto RecordBentNormalSettings = [&Attributes, &Data](const FString& ModeName)
	{
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.BentNormal.OcclusionRays"), *ModeName), Data.OcclusionSettings.OcclusionRays));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.BentNormal.MaxDistance"), *ModeName), Data.OcclusionSettings.MaxDistance));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.BentNormal.SpreadAngle"), *ModeName), Data.OcclusionSettings.SpreadAngle));
	};

	auto RecordCurvatureSettings = [&Attributes, &Data](const FString& ModeName)
	{
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Curvature.CurvatureType"), *ModeName), Data.CurvatureSettings.CurvatureType));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Curvature.RangeMultiplier"), *ModeName), Data.CurvatureSettings.RangeMultiplier));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Curvature.MinRangeMultiplier"), *ModeName), Data.CurvatureSettings.MinRangeMultiplier));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Curvature.ClampMode"), *ModeName), Data.CurvatureSettings.ClampMode));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Curvature.ColorMode"), *ModeName), Data.CurvatureSettings.ColorMode));
	};

	if (Data.BakeSettings.OutputMode == EBakeVertexOutput::RGBA)
	{
		const FString OutputName(TEXT("RGBA"));

		FString OutputTypeName = StaticEnum<EBakeMapType>()->GetNameStringByValue(static_cast<int>(Data.BakeSettings.OutputType));
		Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Type"), *OutputName), OutputTypeName));

		switch (Data.BakeSettings.OutputType)
		{
		case EBakeMapType::AmbientOcclusion:
			RecordAmbientOcclusionSettings(OutputName);
			break;
		case EBakeMapType::BentNormal:
			RecordBentNormalSettings(OutputName);
			break;
		case EBakeMapType::Curvature:
			RecordCurvatureSettings(OutputName);
			break;
		default:
			break;
		}
	}
	else
	{
		ensure(Data.BakeSettings.OutputMode == EBakeVertexOutput::PerChannel);
		for (int EvalId = 0; EvalId < 4; ++EvalId)
		{
			FString OutputName = StaticEnum<EBakeVertexChannel>()->GetNameStringByIndex(EvalId);
			FString OutputTypeName = StaticEnum<EBakeMapType>()->GetNameStringByValue(static_cast<int>(Data.BakeSettings.OutputTypePerChannel[EvalId]));
			Attributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Settings.Output.%s.Type"), *OutputName), OutputTypeName));

			switch (Data.BakeSettings.OutputTypePerChannel[EvalId])
			{
			case EBakeMapType::AmbientOcclusion:
				RecordAmbientOcclusionSettings(OutputName);
				break;
			case EBakeMapType::Curvature:
				RecordCurvatureSettings(OutputName);
				break;
			default:
				break;
			}
		}
	}

	FEngineAnalytics::GetProvider().RecordEvent(FString(TEXT("Editor.Usage.MeshModelingMode.")) + EventName, Attributes);

	constexpr bool bLogAnalytics = false; 
	if constexpr (bLogAnalytics)
	{
		for (const FAnalyticsEventAttribute& Attr : Attributes)
		{
			UE_LOGF(LogGeometry, Log, "[%ls] %ls = %ls", *EventName, *Attr.GetName(), *Attr.GetValue());
		}
	}
}


#undef LOCTEXT_NAMESPACE

