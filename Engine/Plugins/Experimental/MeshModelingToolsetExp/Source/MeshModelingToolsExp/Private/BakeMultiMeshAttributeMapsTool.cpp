// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeMultiMeshAttributeMapsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Sampling/MeshNormalMapEvaluator.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "Sampling/MeshResampleImageEvaluator.h"

#include "ImageUtils.h"

#include "AssetUtils/Texture2DUtil.h"

#include "EngineAnalytics.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

// required to pass UStaticMesh asset so we can save at same location
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeMultiMeshAttributeMapsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeMultiMeshAttributeMapsTool"

/*
* ToolBuilder
*/

const FToolTargetTypeRequirements& UBakeMultiMeshAttributeMapsToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMeshDescriptionProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		UMaterialProvider::StaticClass()
		});
	return TypeRequirements;
}

bool UBakeMultiMeshAttributeMapsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	const int32 NumTargets = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	if (NumTargets > 1)
	{
		// FMeshSceneAdapter currently only supports StaticMesh targets.
		// Restrict source targets to StaticMesh.
		bool bValidTargets = true;
		int TargetId = 0;
		SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(),
			[&bValidTargets, &TargetId](UActorComponent* Component)
			{
				if (TargetId > 0)
				{
					const UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Component);
					bValidTargets = bValidTargets && StaticMesh;
				}
				++TargetId;
			});
		return bValidTargets;
	}
	return false;
}

UMultiSelectionMeshEditingTool* UBakeMultiMeshAttributeMapsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UBakeMultiMeshAttributeMapsTool>(SceneState.ToolManager);
}


/**
 * MeshSceneAdapter bake detail sampler for baking N detail meshes to 1 target mesh.
 */
class FMeshBakerMeshSceneSampler : public IMeshBakerDetailSampler
{
public:
	using FDetailTextureMap = TMap<void*, FBakeDetailTexture>;
	using FDetailNormalMap = TMap<void*, FBakeDetailNormalTexture>;

public:
	/**
	 * Constructor.
	 *
	 * Input FMeshSceneAdapter's spatial evaluation cache is assumed to be built.
	 */
	FMeshBakerMeshSceneSampler(FMeshSceneAdapter* Scene)
		: MeshScene(Scene)
	{
	}

	// Begin IMeshBakerDetailSampler interface
	virtual void ProcessMeshes(TFunctionRef<void(const void*)> ProcessFn) const override
	{
		auto ProcessChildMesh = [ProcessFn](const FActorAdapter*, const FActorChildMesh* ChildMesh)
		{
			if (ChildMesh)
			{
				ProcessFn(ChildMesh);
			}
		};
		MeshScene->ProcessActorChildMeshes(ProcessChildMesh);
	}

	virtual int32 GetTriangleCount(const void* Mesh) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->GetTriangleCount();
	}

	virtual void SetTextureMap(const void* Mesh, const FBakeDetailTexture& Map) override
	{
		DetailTextureMaps[Mesh] = Map;
	}

	virtual void SetNormalTextureMap(const void* Mesh, const FBakeDetailNormalTexture& Map) override
	{
		DetailNormalMaps[Mesh] = Map;
	}

	virtual const FBakeDetailTexture* GetTextureMap(const void* Mesh) const override
	{
		return DetailTextureMaps.Find(Mesh);
	}

	virtual const FBakeDetailNormalTexture* GetNormalTextureMap(const void* Mesh) const override
	{
		return DetailNormalMaps.Find(Mesh);
	}

	virtual bool SupportsIdentityCorrespondence() const override
	{
		return false;
	}

	virtual bool SupportsNearestPointCorrespondence() const override
	{
		return false;
	}

	virtual bool SupportsRaycastCorrespondence() const override
	{
		return true;
	}

	virtual const void* FindNearestHitTriangle(
		const FRay3d& Ray,
		double& NearestT,
		int& TriId,
		FVector3d& TriBaryCoords,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const override
	{
		// Transform ray to world space
		const FRay3d WorldRay = BaseToWorldTransform.TransformRay(Ray);
		
		// TODO: Pass-through max distance to FMeshSceneAdapter query
		FMeshSceneRayHit HitResult;
		const bool bHit = MeshScene->FindNearestRayIntersection(WorldRay, HitResult);
		const void* HitMesh = nullptr;

		// Use TNumericLimits<float>::Max() for consistency with MeshAABBTree3.
		NearestT = (Options.MaxDistance < TNumericLimits<float>::Max()) ? Options.MaxDistance : TNumericLimits<float>::Max();
		if (bHit && HitResult.RayDistance < Options.MaxDistance)
		{
			TriId = HitResult.HitMeshTriIndex;
			NearestT = HitResult.RayDistance;
			TriBaryCoords = HitResult.HitMeshBaryCoords;

			// Use the component and component element index to identify the actual instance
			// (FActorChildMesh) that was hit.
			if (const FActorAdapter* const* FoundActor = ActorComponentMap.Find(HitResult.HitComponent); FoundActor != nullptr)
			{
				if (HitResult.HitComponentElementIndex < (*FoundActor)->ChildMeshes.Num())
				{
					HitMesh = (*FoundActor)->ChildMeshes[FMath::Max(0, HitResult.HitComponentElementIndex)].Get();
				}
			}
		}
		return HitMesh;
	}

	virtual bool TestAnyHitTriangle(
		const FRay3d& Ray,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const override
	{
		// TODO: Add proper test any hit triangle support (for Occlusion)
		// TODO: Pass through max distance to FMeshSceneAdapter query
		double NearestT = TNumericLimits<double>::Max();
		int TriId = IndexConstants::InvalidID;
		FVector3d TriBaryCoords;
		return FindNearestHitTriangle(Ray, NearestT, TriId, TriBaryCoords, Options) != nullptr;
	}

	virtual FAxisAlignedBox3d GetBounds() const override
	{
		return MeshScene->GetBoundingBox();
	}

	virtual bool IsTriangle(const void* Mesh, const int TriId) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->IsTriangle(TriId);
	}

	virtual FIndex3i GetTriangle(const void* Mesh, const int TriId) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->GetTriangle(TriId);
	}

	virtual FVector3d GetTriNormal(const void* Mesh, const int TriId) const override
	{
		// TODO
		checkSlow(false);
		return FVector3d::Zero();
	}

	virtual bool GetTriUVs(const void* Mesh, int TriId, int UVLayer, FVector2f& UV0, FVector2f& UV1, FVector2f& UV2) const override
	{
		checkSlow(false);
		return false;
	}

	virtual int32 GetMaterialID(const void* Mesh, const int TriId) const override
	{
		// TODO
		checkSlow(false);
		return IndexConstants::InvalidID;
	}

	virtual int32 GetPolyGroupID(const void* Mesh, const int TriId) const override
	{
		// TODO
		checkSlow(false);
		return 0;
	}

	virtual bool HasNormals(const void* Mesh) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->HasNormals();
	}

	virtual bool HasUVs(const void* Mesh, const int UVLayer = 0) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->HasUVs(UVLayer);
	}

	virtual bool HasTangents(const void* Mesh) const override
	{
		// TODO
		checkSlow(false);
		return false;
	}

	virtual bool HasColors(const void* Mesh) const override
	{
		// TODO
		checkSlow(false);
		return false;
	}

	virtual FVector3d TriBaryInterpolatePoint(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		const FVector3d Point = ChildMesh->MeshSpatial->TriBaryInterpolatePoint(TriId, BaryCoords);
		const FVector3d WorldPoint = ChildMesh->WorldTransform.TransformPosition(Point);
		return WorldPoint;
	}

	virtual bool TriBaryInterpolateNormal(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3f& NormalOut) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		const bool bSuccess = ChildMesh->MeshSpatial->TriBaryInterpolateNormal(TriId, BaryCoords, NormalOut);
		const FVector3d NormalOut3d(NormalOut);
		// As a stop-gap fix, transform our normal to the local space of the Base mesh here.
		// TODO: Handle WorldToBase transformation in the evaluator.
		NormalOut = BaseToWorldTransform.InverseTransformNormal(FVector3f(ChildMesh->WorldTransform.TransformNormal(NormalOut3d)));
		return bSuccess;
	}

	virtual bool TriBaryInterpolateUV(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		const int UVLayer,
		FVector2f& UVOut ) const override
	{
		const FActorChildMesh* ChildMesh = static_cast<const FActorChildMesh*>(Mesh);
		return ChildMesh->MeshSpatial->TriBaryInterpolateUV(TriId, BaryCoords, UVLayer, UVOut);
	}

	virtual bool TriBaryInterpolateColor(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector4f& ColorOut) const override
	{
		// TODO
		checkSlow(false);
		return false;
	}

	virtual bool TriBaryInterpolateTangents(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3d& TangentX,
		FVector3d& TangentY) const override
	{
		// TODO
		checkSlow(false);
		return false;
	}

	virtual void GetCurvature(
		const void* Mesh,
		FMeshVertexCurvatureCache& CurvatureCache) const override
	{
		// TODO
		checkSlow(false);
	}
	// End IMeshBakerDetailSampler interface

	/** Initialize the mesh to color textures map */
	void SetTextureMaps(const FDetailTextureMap& Map)
	{
		DetailTextureMaps = Map;
	}

	/** Initialize the mesh to normal textures map */
	void SetNormalMaps(const FDetailNormalMap& Map)
	{
		DetailNormalMaps = Map;
	}

public:
	FTransformSRT3d BaseToWorldTransform;
	TMap<UActorComponent*, const FActorAdapter*> ActorComponentMap;

protected:
	FMeshSceneAdapter* MeshScene = nullptr;
	FDetailTextureMap DetailTextureMaps;
	FDetailNormalMap DetailNormalMaps;
};


/*
 * Operators
 */

class FMultiMeshMapBakerOp : public TGenericDataOperator<FMeshMapBaker>
{
public:
	// General bake settings
	TSharedPtr<FMeshSceneAdapter, ESPMode::ThreadSafe> DetailMeshScene;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> BaseMesh;
	TSharedPtr<UE::Geometry::FMeshTangentsd, ESPMode::ThreadSafe> BaseMeshTangents;
	TUniquePtr<UE::Geometry::FMeshMapBaker> Baker;
	UBakeMultiMeshAttributeMapsTool::FBakeSettings BakeSettings;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts;
	TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>> SampleFilterMask;

	FTransformSRT3d BaseToWorldTransform;

	// Detail bake data
	TArray<TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>>> CachedColorImages;
	UBakeMultiMeshAttributeMapsTool::FTextureImageMap CachedMeshToColorImageMap;

	// Begin TGenericDataOperator interface
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		Baker = MakeUnique<FMeshMapBaker>();
		Baker->CancelF = [Progress]() {
			return Progress && Progress->Cancelled();
		};
		Baker->SetTargetMesh(BaseMesh.Get());
		Baker->SetTargetMeshUVLayer(BakeSettings.TargetUVLayer);
		Baker->SetDimensions(BakeSettings.Dimensions);
		Baker->SetProjectionDistance(BakeSettings.ProjectionDistance);
		Baker->SetSamplesPerPixel(BakeSettings.SamplesPerPixel);
		Baker->SetTargetMeshTangents(BaseMeshTangents);
		Baker->SetTargetMeshUVCharts(BaseMeshUVCharts.Get());
		if (SampleFilterMask)
		{
			Baker->SampleFilterF = [this](const FVector2i& ImageCoords, const FVector2d& UV, int32 TriID)
			{
				const FVector4f Mask = SampleFilterMask->BilinearSampleUV<float>(UV, FVector4f::One());
				return (Mask.X + Mask.Y + Mask.Z) / 3;
			};
		}
		
		FMeshBakerMeshSceneSampler DetailSampler(DetailMeshScene.Get());

		// TODO: Precompute the ActorComponent to ActorChildMesh map in the tool.
		auto ProcessChildMesh = [&DetailSampler](const FActorAdapter* Actor, const FActorChildMesh* ChildMesh)
		{
			if (Actor && ChildMesh)
			{
				// Here we rely on the fact that we populated the MeshSceneAdapter using AddComponents.
				// This guarantees that each component has its own ActorAdapter. So we build a map
				// of source component to actor adapter.
				const FActorAdapter** FoundActor = DetailSampler.ActorComponentMap.Find(ChildMesh->SourceComponent);
				if (!FoundActor)
				{
					DetailSampler.ActorComponentMap.Add(ChildMesh->SourceComponent, Actor);
				}
			}
		};
		DetailMeshScene->ProcessActorChildMeshes(ProcessChildMesh);
		DetailSampler.BaseToWorldTransform = BaseToWorldTransform;
		Baker->SetDetailSampler(&DetailSampler);

		for (const EBakeMapType MapType : ENUM_EBAKEMAPTYPE_ALL)
		{
			switch (BakeSettings.BakeMapTypes & MapType)
			{
			case EBakeMapType::TangentSpaceNormal:
			{
				TSharedPtr<FMeshNormalMapEvaluator, ESPMode::ThreadSafe> NormalEval = MakeShared<FMeshNormalMapEvaluator, ESPMode::ThreadSafe>();
				Baker->AddEvaluator(NormalEval);
				break;	
			}
			case EBakeMapType::Position:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::Position;
				Baker->AddEvaluator(PropertyEval);
				break;
			}
			case EBakeMapType::ObjectSpaceNormal:
			{
				TSharedPtr<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
				PropertyEval->Property = EMeshPropertyMapType::Normal;
				Baker->AddEvaluator(PropertyEval);
			}
			case EBakeMapType::Texture:
			{
				TSharedPtr<FMeshResampleImageEvaluator, ESPMode::ThreadSafe> TextureEval = MakeShared<FMeshResampleImageEvaluator, ESPMode::ThreadSafe>();
				DetailSampler.SetTextureMaps(CachedMeshToColorImageMap);
				Baker->AddEvaluator(TextureEval);
				break;
			}
			default:
				break;
			}
		}
		Baker->Bake();
		SetResult(MoveTemp(Baker));
	}
	// End TGenericDataOperator interface
};


/*
 * Tool
 */

void UBakeMultiMeshAttributeMapsTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMultiMeshAttributeMapsTool::Setup);
	
	Super::Setup();

	// Initialize base mesh
	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		TargetMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		TargetMesh->Copy(Mesh);

		TargetMeshTangents = MakeShared<FMeshTangentsd, ESPMode::ThreadSafe>(TargetMesh.Get());
		TargetMeshTangents->CopyTriVertexTangents(Mesh);
	});

	// Initialize detail sampler
	const int NumTargets = Targets.Num();
	TArray<UActorComponent*> DetailComponents;
	for (int Idx = 1; Idx < NumTargets; ++Idx)
	{
		if (UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Targets[Idx]))
		{
			DetailComponents.Add(Component);
		}
	}
	DetailMeshScene = MakeShared<FMeshSceneAdapter, ESPMode::ThreadSafe>();
	// FMultiMeshMapBakerOp depends on adding meshes via AddComponents.
	// If this changed, ensure the logic for generating the ActorComponentMap
	// is updated in FMultiMeshMapBakerOp.
	DetailMeshScene->AddComponents(DetailComponents);
	DetailMeshScene->Build(FMeshSceneAdapterBuildOptions());
	DetailMeshScene->BuildSpatialEvaluationCache();

	UToolTarget* Target = Targets[0];

	// Setup tool property sets

	Settings = NewObject<UBakeMultiMeshAttributeMapsToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->MapTypes, [this](int32) { OpState |= EBakeOpState::Evaluate; UpdateOnModeChange(); });
	Settings->WatchProperty(Settings->MapPreview, [this](FString) { UpdateVisualization(); GetToolManager()->PostInvalidation(); });
	Settings->WatchProperty(Settings->Resolution, [this](EBakeTextureResolution) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->BitDepth, [this](EBakeTextureBitDepth) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->SamplesPerPixel, [this](EBakeTextureSamplesPerPixel) { OpState |= EBakeOpState::Evaluate; });
	Settings->WatchProperty(Settings->SampleFilterMask, [this](UTexture2D*){ OpState |= EBakeOpState::Evaluate; });
	
	InputMeshSettings = NewObject<UBakeMultiMeshInputToolProperties>(this);
	InputMeshSettings->RestoreProperties(this);
	AddToolPropertySource(InputMeshSettings);
	InputMeshSettings->TargetStaticMesh = UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetSkeletalMesh = UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(Target);
	InputMeshSettings->TargetDynamicMesh = GetTargetActorViaIPersistentDynamicMeshSource(Target);
	UpdateUVLayerNames(InputMeshSettings->TargetUVLayer, InputMeshSettings->TargetUVLayerNamesList, *TargetMesh);
	InputMeshSettings->WatchProperty(InputMeshSettings->TargetUVLayer, [this](FString) { OpState |= EBakeOpState::Evaluate; });
	InputMeshSettings->WatchProperty(InputMeshSettings->ProjectionDistance, [this](float) { OpState |= EBakeOpState::Evaluate; });

	ResultSettings = NewObject<UBakeMeshAttributeMapsResultToolProperties>(this);
	ResultSettings->RestoreProperties(this);
	AddToolPropertySource(ResultSettings);
	SetToolPropertySourceEnabled(ResultSettings, true);


	// Pre-populate detail mesh data
	TArray<UTexture2D*> DetailColorTextures;
	for (int Idx = 1; Idx < NumTargets; ++Idx)
	{
		UToolTarget* DetailTarget = Targets[Idx];

		// Hide each of our detail targets since this baker operates solely in world space
		// which will occlude the preview of the target mesh. 
		UE::ToolTarget::HideSourceObject(DetailTarget);
		
		const UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Targets[Idx]);
		UTexture2D* DetailColorTexture = nullptr;
		ProcessComponentTextures(Component, [&DetailColorTexture](const int NumMaterials, const int MaterialID, const TArray<UTexture*>& Textures)
		{
			// TODO: Support multiple materialIDs per detail mesh
			if (MaterialID == 0)
			{
				const int SelectedTextureIndex = SelectColorTextureToBake(Textures);
				if (SelectedTextureIndex >= 0)
				{
					DetailColorTexture = Cast<UTexture2D>(Textures[SelectedTextureIndex]);
				}
			}
		});

		FBakeMultiMeshDetailProperties SourceMeshProp;
		SourceMeshProp.SourceMesh = UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(DetailTarget);
		SourceMeshProp.SourceTexture = DetailColorTexture;
		InputMeshSettings->SourceMeshes.Add(SourceMeshProp);
		InputMeshSettings->WatchProperty(InputMeshSettings->SourceMeshes[Idx-1].SourceTexture, [this](UTexture2D*) { OpState |= EBakeOpState::Evaluate; });
		InputMeshSettings->WatchProperty(InputMeshSettings->SourceMeshes[Idx-1].SourceTextureUVLayer, [this](int) { OpState |= EBakeOpState::Evaluate; });
	}

	UpdateOnModeChange();

	OpState |= EBakeOpState::Evaluate;

	SetToolDisplayName(LOCTEXT("ToolName", "Bake Textures"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Bake Maps. Select Bake Mesh (LowPoly) first, then select Detail Meshes (HiPoly) to bake. Texture Assets will be created on Accept. "),
		EToolMessageLevel::UserNotification);

	PostSetup();
}




bool UBakeMultiMeshAttributeMapsTool::CanAccept() const
{
	const bool bValidOp = (OpState & EBakeOpState::Invalid) != EBakeOpState::Invalid;
	bool bCanAccept = bValidOp && Compute ? Compute->HaveValidResult() : false;
	if (bCanAccept)
	{
		// Allow Accept if all non-None types have valid results.
		for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Result : ResultSettings->Result)
		{
			bCanAccept = bCanAccept && Result.Get<1>();
		}
	}
	return bCanAccept;
}


TUniquePtr<UE::Geometry::TGenericDataOperator<FMeshMapBaker>> UBakeMultiMeshAttributeMapsTool::MakeNewOperator()
{
	TUniquePtr<FMultiMeshMapBakerOp> Op = MakeUnique<FMultiMeshMapBakerOp>();
	Op->DetailMeshScene = DetailMeshScene;
	Op->BaseMesh = TargetMesh;
	Op->BaseMeshUVCharts = TargetMeshUVCharts;
	Op->BaseToWorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	Op->BakeSettings = CachedBakeSettings;
	Op->SampleFilterMask = CachedSampleFilterMask;

	constexpr EBakeMapType RequiresTangents = EBakeMapType::TangentSpaceNormal | EBakeMapType::BentNormal;
	if (static_cast<bool>(CachedBakeSettings.BakeMapTypes & RequiresTangents))
	{
		Op->BaseMeshTangents = TargetMeshTangents;
	}

	if (static_cast<bool>(CachedBakeSettings.BakeMapTypes & EBakeMapType::Texture))
	{
		Op->CachedColorImages = CachedColorImages;
		Op->CachedMeshToColorImageMap = CachedMeshToColorImagesMap;
	}
	return Op;
}


void UBakeMultiMeshAttributeMapsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBakeMultiMeshAttributeMapsTool::Shutdown);
	
	Super::OnShutdown(ShutdownType);
	
	Settings->SaveProperties(this);
	InputMeshSettings->SaveProperties(this);

	if (Compute)
	{
		Compute->Shutdown();
	}
	DetailMeshScene = nullptr;

	// Restore visibility of detail targets
	const int NumTargets = Targets.Num();
	for (int Idx = 1; Idx < NumTargets; ++Idx)
	{
		UE::ToolTarget::ShowSourceObject(Targets[Idx]);
	}
	
	if (ShutdownType == EToolShutdownType::Accept)
	{
		IStaticMeshBackedTarget* StaticMeshTarget = Cast<IStaticMeshBackedTarget>(Targets[0]);
		UObject* SourceAsset = StaticMeshTarget ? StaticMeshTarget->GetStaticMesh() : nullptr;
		const UPrimitiveComponent* SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[0]);
		CreateTextureAssets(ResultSettings->Result, SourceComponent->GetWorld(), SourceAsset);
	}
}


void UBakeMultiMeshAttributeMapsTool::UpdateResult()
{
	if (OpState == EBakeOpState::Clean)
	{
		return;
	}

	// clear warning (ugh)
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

	const int32 ImageSize = static_cast<int32>(Settings->Resolution);
	const FImageDimensions Dimensions(ImageSize, ImageSize);

	FBakeSettings BakeSettings;
	BakeSettings.Dimensions = Dimensions;
	BakeSettings.BitDepth = Settings->BitDepth;
	BakeSettings.TargetUVLayer = InputMeshSettings->TargetUVLayerNamesList.IndexOfByKey(InputMeshSettings->TargetUVLayer);
	BakeSettings.ProjectionDistance = InputMeshSettings->ProjectionDistance;
	BakeSettings.bProjectionInWorldSpace = true;  // Always world space
	BakeSettings.SamplesPerPixel = static_cast<int32>(Settings->SamplesPerPixel);

	// Record the original map types and process the raw bitfield which may add
	// additional targets.
	BakeSettings.SourceBakeMapTypes = static_cast<EBakeMapType>(Settings->MapTypes);
	BakeSettings.BakeMapTypes = GetMapTypes(Settings->MapTypes);

	// update bake cache settings
	if (!(CachedBakeSettings == BakeSettings))
	{
		CachedBakeSettings = BakeSettings;
		CachedDetailSettings = FBakeMultiMeshDetailSettings();
	}

	// Clear our invalid bitflag to check again for valid inputs.
	OpState &= ~EBakeOpState::Invalid;

	OpState |= UpdateResult_TargetMeshTangents(CachedBakeSettings.BakeMapTypes);

	OpState |= UpdateResult_SampleFilterMask(Settings->SampleFilterMask);

	// Update map type settings
	OpState |= UpdateResult_DetailMeshes();

	// Early exit if op input parameters are invalid.
	if (static_cast<bool>(OpState & EBakeOpState::Invalid))
	{
		InvalidateResults();
		return;
	}

	// This should be the only point of compute invalidation to
	// minimize synchronization issues.
	InvalidateCompute();
}


EBakeOpState UBakeMultiMeshAttributeMapsTool::UpdateResult_DetailMeshes()
{
	FBakeMultiMeshDetailSettings NewSettings;

	// Iterate through our detail properties to build our detail mesh data.
	const int32 NumDetail = InputMeshSettings->SourceMeshes.Num();
	CachedColorImages.SetNum(NumDetail);
	CachedColorUVLayers.SetNum(NumDetail);
	TMap<UActorComponent*, int> ActorToDataMap;
	for (int Idx = 0; Idx < NumDetail; ++Idx)
	{
		UActorComponent* ActorComponent = UE::ToolTarget::GetTargetComponent(Targets[Idx+1]);
		ActorToDataMap.Emplace(ActorComponent, Idx);
		
		// Color map data
		if (static_cast<bool>(CachedBakeSettings.BakeMapTypes & EBakeMapType::Texture))
		{
			UTexture2D* ColorMapSourceTexture = InputMeshSettings->SourceMeshes[Idx].SourceTexture;
			const int ColorMapUVLayer = InputMeshSettings->SourceMeshes[Idx].SourceTextureUVLayer;
			if (!ColorMapSourceTexture)
			{
				GetToolManager()->DisplayMessage(LOCTEXT("InvalidTextureWarning", "The Source Texture is not valid"), EToolMessageLevel::UserWarning);
				return EBakeOpState::Invalid;
			}

			TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> ColorTextureImage = MakeShared<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>();
			if (!UE::AssetUtils::ReadTexture(ColorMapSourceTexture, *ColorTextureImage, bPreferPlatformData))
			{
				GetToolManager()->DisplayMessage(LOCTEXT("CannotReadTextureWarning", "Cannot read from the source texture"), EToolMessageLevel::UserWarning);
				return EBakeOpState::Invalid;
			}
			CachedColorImages[Idx] = ColorTextureImage;
			CachedColorUVLayers[Idx] = ColorMapUVLayer;
		}
	}

	// Iterate through mesh scene adapter and build mesh to data maps.
	CachedMeshToColorImagesMap.Empty();
	auto BuildMeshToDataMaps = [this, ActorToDataMap](const FActorAdapter*, const FActorChildMesh* ChildMesh)
	{
		if (ChildMesh)
		{
			if (const int* DataIndex = ActorToDataMap.Find(ChildMesh->SourceComponent))
			{
				if (static_cast<bool>(CachedBakeSettings.BakeMapTypes & EBakeMapType::Texture))
				{
					CachedMeshToColorImagesMap.Emplace(
					(void*)ChildMesh,
					FTextureImageData(CachedColorImages[*DataIndex].Get(), CachedColorUVLayers[*DataIndex]));
				}
			}
		}
	};
	DetailMeshScene->ProcessActorChildMeshes(BuildMeshToDataMaps);

	// This method will always force a re-evaluation.
	return EBakeOpState::Evaluate;
}


void UBakeMultiMeshAttributeMapsTool::UpdateVisualization()
{
	PreviewMesh->SetOverrideRenderMaterial(PreviewMaterial);

	// Populate Settings->Result from CachedMaps
	for (const TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Map : CachedMaps)
	{
		if (ResultSettings->Result.Contains(Map.Get<0>()))
		{
			ResultSettings->Result[Map.Get<0>()] = Map.Get<1>();
		}
	}

	UpdatePreview(Settings->MapPreview, Settings->MapPreviewNamesMap);
}


void UBakeMultiMeshAttributeMapsTool::UpdateOnModeChange()
{
	OnMapTypesUpdated(
		static_cast<EBakeMapType>(Settings->MapTypes),
		ResultSettings->Result,
		Settings->MapPreview,
		Settings->MapPreviewNamesList,
		Settings->MapPreviewNamesMap);
}


void UBakeMultiMeshAttributeMapsTool::InvalidateResults()
{
	for (TTuple<EBakeMapType, TObjectPtr<UTexture2D>>& Result : ResultSettings->Result)
	{
		Result.Get<1>() = nullptr;
	}
}


void UBakeMultiMeshAttributeMapsTool::GatherAnalytics(FBakeAnalytics::FMeshSettings& Data)
{
	if (FEngineAnalytics::IsAvailable())
	{
		Data.NumTargetMeshTris = TargetMesh->TriangleCount();
		Data.NumDetailMesh = 0;
		Data.NumDetailMeshTris = 0;
		DetailMeshScene->ProcessActorChildMeshes([&Data](const FActorAdapter* ActorAdapter, const FActorChildMesh* ChildMesh)
		{
			if (ChildMesh)
			{
				++Data.NumDetailMesh;
				Data.NumDetailMeshTris += ChildMesh->MeshSpatial->GetTriangleCount();
			}
		});
	}
}



#undef LOCTEXT_NAMESPACE

