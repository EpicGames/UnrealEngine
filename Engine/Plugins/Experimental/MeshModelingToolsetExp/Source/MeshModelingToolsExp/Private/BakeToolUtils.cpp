// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeToolUtils.h"

#include "PreviewMesh.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Components/DynamicMeshComponent.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "ToolTargets/ToolTarget.h"

namespace UE
{
namespace Geometry
{

void UpdateUVLayerNames(FString& UVLayer, TArray<FString>& UVLayerNamesList, const FDynamicMesh3& Mesh)
{
	UVLayerNamesList.Reset();
	int32 FoundIndex = -1;
	for (int32 k = 0; k < Mesh.Attributes()->NumUVLayers(); ++k)
	{
		UVLayerNamesList.Add(FString::Printf(TEXT("UV %d"), k));
		if (UVLayer == UVLayerNamesList.Last())
		{
			FoundIndex = k;
		}
	}
	if (FoundIndex == -1)
	{
		UVLayer = UVLayerNamesList[0];
	}
}

// TODO Some variation of this function may be able to be reused in other tools
UPreviewMesh* CreateBakePreviewMesh(
	UObject* Tool,
	UToolTarget* ToolTarget,
	UWorld* World)
{
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	const FDynamicMesh3 InputMesh = UE::ToolTarget::GetDynamicMeshCopy(ToolTarget, GetMeshParams);
	const FTransformSRT3d BaseToWorld = UE::ToolTarget::GetLocalToWorldTransform(ToolTarget);
	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(ToolTarget);

	UPreviewMesh* PreviewMesh = NewObject<UPreviewMesh>(Tool);

	PreviewMesh->CreateInWorld(World, FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);
	PreviewMesh->SetTransform(static_cast<FTransform>(BaseToWorld));
	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);
	PreviewMesh->ReplaceMesh(InputMesh);
	PreviewMesh->SetMaterials(MaterialSet.Materials);
	PreviewMesh->SetVisible(true);

	return PreviewMesh;
}


// TODO We could probably use UE::ToolTarget::GetTargetActor to implement/replace this function
AActor* GetTargetActorViaIPersistentDynamicMeshSource(UToolTarget* Target)
{
	IPersistentDynamicMeshSource* TargetDynamicMeshTarget = Cast<IPersistentDynamicMeshSource>(Target);
	if (TargetDynamicMeshTarget)
	{
		UDynamicMeshComponent* TargetDynamicMeshComponent = TargetDynamicMeshTarget->GetDynamicMeshComponent();
		if (TargetDynamicMeshComponent)
		{
			return TargetDynamicMeshComponent->GetOwner();
		}
	}
	return nullptr;
}

} // namespace Geometry
} // namespace UE
