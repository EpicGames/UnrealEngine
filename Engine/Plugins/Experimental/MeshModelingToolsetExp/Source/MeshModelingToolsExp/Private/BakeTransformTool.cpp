// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeTransformTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "BaseBehaviors/MultiClickSequenceInputBehavior.h"
#include "Selection/SelectClickedAction.h"

#include "MeshAdapterTransforms.h"
#include "MeshDescriptionAdapter.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "Physics/ComponentCollisionUtil.h"
#include "PhysicsEngine/BodySetup.h"

#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeTransformTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBakeTransformTool"

/*
 * ToolBuilder
 */

UMultiSelectionMeshEditingTool* UBakeTransformToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UBakeTransformTool>(SceneState.ToolManager);
}

const FToolTargetTypeRequirements& UBakeTransformToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


/*
 * Tool
 */

UBakeTransformToolProperties::UBakeTransformToolProperties()
{
}


UBakeTransformTool::UBakeTransformTool()
{
}


void UBakeTransformTool::Setup()
{
	UInteractiveTool::Setup();

	BasicProperties = NewObject<UBakeTransformToolProperties>(this);
	AddToolPropertySource(BasicProperties);
	BasicProperties->RestoreProperties(this);

	FText AllTheWarnings = LOCTEXT("BakeTransformWarning", "WARNING: This Tool will Modify the selected StaticMesh Assets! If you do not wish to modify the original Assets, please make copies in the Content Browser first!");

	// detect and warn about any meshes in selection that correspond to same source data
	bool bSharesSources = GetMapToSharedSourceData(MapToFirstOccurrences);
	if (bSharesSources)
	{
		AllTheWarnings = FText::Format(FTextFormat::FromString("{0}\n\n{1}"), AllTheWarnings, LOCTEXT("BakeTransformSharedAssetsWarning", "WARNING: Multiple meshes in your selection use the same source asset!  This is not supported -- each asset can only have one baked transform."));
	}

	bool bHasZeroScales = false;
	BasicProperties->bAllowNoScale = true;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FTransform Transform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]);
		FVector Scale = Transform.GetScale3D();
		// Set variable so a DetailCustomization can disable "DoNotBakeScale" if we have some targets w/ both rotation and non-uniform scale
		// Note we could relax this to allow DoNotBakeScale if the rotation axis is aligned to the non-uniform scale axis, but that might make the enable/disable condition harder to understand.
		BasicProperties->bAllowNoScale = BasicProperties->bAllowNoScale && (Transform.GetRotation().IsIdentity() || Transform.GetScale3D().AllComponentsEqual());
		if (Scale.GetAbsMin() < KINDA_SMALL_NUMBER)
		{
			bHasZeroScales = true;
		}
	}
	if (bHasZeroScales)
	{
		AllTheWarnings = FText::Format(FTextFormat::FromString("{0}\n\n{1}"), AllTheWarnings, LOCTEXT("BakeTransformWithZeroScale", "WARNING: Baking a zero scale in any dimension will permanently flatten the asset."));
	}
	
	GetToolManager()->DisplayMessage(AllTheWarnings, EToolMessageLevel::UserWarning);

	SetToolDisplayName(LOCTEXT("ToolName", "Bake Transform"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "This Tool applies the current Rotation and/or Scaling of the object's Transform to the underlying mesh Asset."),
		EToolMessageLevel::UserNotification);
}



void UBakeTransformTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		UpdateAssets();
	}

	BasicProperties->SaveProperties(this);
}



void UBakeTransformTool::UpdateAssets()
{
	// Make sure mesh descriptions are deserialized before we open transaction.
	// This is to avoid potential stability issues related to creation/load of
	// mesh descriptions inside a transaction.
	// TODO: this may not be necessary anymore. Also may not be the most efficient
	// Note: for the crash workaround below, this also now pre-computes the source mesh bounds
	TArray<FBox> BoundsOfScaledRotatedMesh;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		bool bTargetSupportsLODs = false;
		TArray<EMeshLODIdentifier> LODs = UE::ToolTarget::GetMeshDescriptionLODs(Targets[ComponentIdx], bTargetSupportsLODs, !BasicProperties->bApplyToAllLODs);
		// Make sure all the LODs after the first are loaded, but don't do anything with them
		for (int32 LODIdx = 1; LODIdx < LODs.Num(); ++LODIdx)
		{
			UE::ToolTarget::GetMeshDescription(Targets[ComponentIdx], FGetMeshParameters(bTargetSupportsLODs, LODs[LODIdx]));
		}
		// Use the first LOD to update the bounds (or the default LOD, if LODs are not supported or if the update does not need to apply to all LODs)
		const FMeshDescription* MeshDescription = UE::ToolTarget::GetMeshDescription(Targets[ComponentIdx], FGetMeshParameters(bTargetSupportsLODs, LODs[0]));
		if (MapToFirstOccurrences[ComponentIdx] < ComponentIdx)
		{
			FBox Bounds = BoundsOfScaledRotatedMesh[MapToFirstOccurrences[ComponentIdx]];
			BoundsOfScaledRotatedMesh.Add(Bounds);
		}
		else
		{
			// Apply the Scale and Rotation for this mesh and compute bounds.
			FTransformSRT3d ComponentToWorld = UE::ToolTarget::GetLocalToWorldTransform( Targets[ComponentIdx] );
			ComponentToWorld.SetTranslation(FVector::Zero());
			
			FBox BoundingBox(ForceInit);
			for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				FVector3f Pos = MeshDescription->GetVertexPosition(VertexID);
				FVector3f NewPos = ComponentToWorld.TransformPosition(Pos);
				BoundingBox += FVector(NewPos);
			}

			BoundsOfScaledRotatedMesh.Add(BoundingBox);
		}
	}

	constexpr bool bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction = false;
	bool bNeedSeparateTransactionForSimpleCollision = false;
	if constexpr (bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction)
	{
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			UToolTarget* Target = Targets[ComponentIdx];
			UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
			if (UE::Geometry::ComponentTypeSupportsCollision(Component))
			{
				if (UBodySetup* BodySetup = UE::Geometry::GetBodySetup(Component))
				{
					if (BodySetup->AggGeom.ConvexElems.Num() > 0)
					{
						bNeedSeparateTransactionForSimpleCollision = true;
						break;
					}
				}
			}
		}
	}

	// Compute all the transforms that we should bake to the assets or apply to the components
	TArray<FTransformSRT3d> BakedTransforms, ComponentTransforms;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UToolTarget* Target = Targets[ComponentIdx];

		FTransformSRT3d ComponentToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
		FTransformSRT3d ToBakePart = FTransformSRT3d::Identity();
		FTransformSRT3d NewWorldPart = ComponentToWorld;

		if (MapToFirstOccurrences[ComponentIdx] < ComponentIdx)
		{
			ToBakePart = BakedTransforms[MapToFirstOccurrences[ComponentIdx]];
			BakedTransforms.Add(ToBakePart);
			// try to invert baked transform
			NewWorldPart = FTransformSRT3d(
				NewWorldPart.GetRotation() * ToBakePart.GetRotation().Inverse(),
				NewWorldPart.GetTranslation(),
				NewWorldPart.GetScale() * FTransformSRT3d::GetSafeScaleReciprocal(ToBakePart.GetScale())
			);
			NewWorldPart.SetTranslation(NewWorldPart.GetTranslation() - NewWorldPart.TransformVector(ToBakePart.GetTranslation()));
		}
		else
		{
			if (BasicProperties->bBakeRotation)
			{
				ToBakePart.SetRotation(ComponentToWorld.GetRotation());
				NewWorldPart.SetRotation(FQuaterniond::Identity());
			}
			FVector3d ScaleVec = ComponentToWorld.GetScale();

			// weird algo to choose what to keep around as uniform scale in the case where we want to bake out the non-uniform scaling
			FVector3d AbsScales(FMathd::Abs(ScaleVec.X), FMathd::Abs(ScaleVec.Y), FMathd::Abs(ScaleVec.Z));
			double RemainingUniformScale = AbsScales.X;
			{
				FVector3d Dists;
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int OtherA = (SubIdx + 1) % 3;
					int OtherB = (SubIdx + 2) % 3;
					Dists[SubIdx] = FMathd::Abs(AbsScales[SubIdx] - AbsScales[OtherA]) + FMathd::Abs(AbsScales[SubIdx] - AbsScales[OtherB]);
				}
				int BestSubIdx = 0;
				for (int CompareSubIdx = 1; CompareSubIdx < 3; CompareSubIdx++)
				{
					if (Dists[CompareSubIdx] < Dists[BestSubIdx])
					{
						BestSubIdx = CompareSubIdx;
					}
				}
				RemainingUniformScale = AbsScales[BestSubIdx];
				if (RemainingUniformScale <= FLT_MIN)
				{
					RemainingUniformScale = MaxAbsElement(AbsScales);
				}
			}
			switch (BasicProperties->BakeScale)
			{
			case EBakeScaleMethod::BakeFullScale:
				ToBakePart.SetScale(ScaleVec);
				NewWorldPart.SetScale(FVector3d::One());
				break;
			case EBakeScaleMethod::BakeNonuniformScale:
				check(RemainingUniformScale > FLT_MIN); // avoid baking a ~zero scale
				ToBakePart.SetScale(ScaleVec / RemainingUniformScale);
				NewWorldPart.SetScale(FVector3d(RemainingUniformScale, RemainingUniformScale, RemainingUniformScale));
				break;
			case EBakeScaleMethod::DoNotBakeScale:
				break;
			default:
				check(false); // must explicitly handle all cases
			}

			// do this part within the commit because we have the MeshDescription already computed
			if (BasicProperties->bRecenterPivot)
			{
				FBox BBox = BoundsOfScaledRotatedMesh[ComponentIdx];
				FVector3d Center(BBox.GetCenter());
				FFrame3d LocalFrame(Center);
				ToBakePart.SetTranslation(ToBakePart.GetTranslation() - Center);
				NewWorldPart.SetTranslation(NewWorldPart.GetTranslation() + NewWorldPart.TransformVector(Center));
			}

			BakedTransforms.Add(ToBakePart);
		}

		ComponentTransforms.Add(NewWorldPart);
	}

	auto UpdateSimpleCollision = [](UPrimitiveComponent* Component, const FTransformSRT3d& ToBakePart)
	{
		// try to transform simple collision
		if (UE::Geometry::ComponentTypeSupportsCollision(Component))
		{
			UE::Geometry::TransformSimpleCollision(Component, ToBakePart);
		}
	};

	// If necessary, make a first transaction to bake simple collision updates separately (works around crash in undo/redo of transactions that update both hulls and static meshes together)
	if (bNeedSeparateTransactionForSimpleCollision)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("BakeTransformToolSimpleCollisionTransactionName", "Bake Transforms Part 1 (Simple Collision)"));
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			UToolTarget* Target = Targets[ComponentIdx];
			UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
			Component->Modify();
			UpdateSimpleCollision(Component, BakedTransforms[ComponentIdx]);
		}
		GetToolManager()->EndUndoTransaction();
	}

	GetToolManager()->BeginUndoTransaction(bNeedSeparateTransactionForSimpleCollision ?
		LOCTEXT("BakeTransformToolGeometryTransactionName", "Bake Transforms Part 2 (Visible Geometry)") :
		LOCTEXT("BakeTransformToolTransactionName", "Bake Transforms"));

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UToolTarget* Target = Targets[ComponentIdx];
		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
		Component->Modify();

		FTransformSRT3d ToBakePart = BakedTransforms[ComponentIdx];

		if (MapToFirstOccurrences[ComponentIdx] == ComponentIdx)
		{
			bool bTargetSupportsLODs = false;
			TArray<EMeshLODIdentifier> LODs = UE::ToolTarget::GetMeshDescriptionLODs(Targets[ComponentIdx], bTargetSupportsLODs, !BasicProperties->bApplyToAllLODs);
			for (EMeshLODIdentifier LOD : LODs)
			{
				FMeshDescription SourceMesh(UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx], FGetMeshParameters(bTargetSupportsLODs, LOD)));
				FMeshDescriptionEditableTriangleMeshAdapter EditableMeshDescAdapter(&SourceMesh);

				MeshAdapterTransforms::ApplyTransform(EditableMeshDescAdapter, ToBakePart);

				FVector3d BakeScaleVec = ToBakePart.GetScale();
				if (BakeScaleVec.X * BakeScaleVec.Y * BakeScaleVec.Z < 0)
				{
					SourceMesh.ReverseAllPolygonFacing();
				}

				// todo: support vertex-only update
				UE::ToolTarget::CommitMeshDescriptionUpdate(Target, &SourceMesh, nullptr /*no material set changes*/, FCommitMeshParameters(bTargetSupportsLODs, LOD));
			}

			if (!bNeedSeparateTransactionForSimpleCollision)
			{
				UpdateSimpleCollision(Component, ToBakePart);
			}

			BakedTransforms.Add(ToBakePart);
		}

		Component->SetWorldTransform((FTransform)ComponentTransforms[ComponentIdx]);

		AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
		if (TargetActor)
		{
			TargetActor->MarkComponentsRenderStateDirty();
		}
	}

	if (BasicProperties->bRecenterPivot)
	{
		// hack to ensure user sees the updated pivot immediately: request re-select of the original selection
		FSelectedObjectsChangeList NewSelection;
		NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
		for (int OrigMeshIdx = 0; OrigMeshIdx < Targets.Num(); OrigMeshIdx++)
		{
			AActor* OwnerActor = UE::ToolTarget::GetTargetActor(Targets[OrigMeshIdx]);
			if (OwnerActor)
			{
				NewSelection.Actors.Add(OwnerActor);
			}
		}
		GetToolManager()->RequestSelectionChange(NewSelection);
	}

	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE

