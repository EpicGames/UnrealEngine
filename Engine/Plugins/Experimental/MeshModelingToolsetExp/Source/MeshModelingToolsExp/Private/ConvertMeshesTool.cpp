// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConvertMeshesTool.h"

#include "Algo/Accumulate.h"
#include "Algo/Count.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "ComponentSourceInterfaces.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "Physics/ComponentCollisionUtil.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#if WITH_EDITOR
#include "ToolTargets/StaticMeshComponentToolTarget.h" // for GetActiveEditingLOD
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConvertMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UConvertMeshesTool"

/*
 * ToolBuilder
 */
UInteractiveTool* UConvertMeshesToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UConvertMeshesTool* Tool = NewObject<UConvertMeshesTool>(SceneState.ToolManager);
	TArray<TWeakObjectPtr<UPrimitiveComponent>> Inputs;
	auto TryAddSelected = [&Inputs](UActorComponent* Selected)
	{
		if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Selected))
		{
			if (UE::Conversion::CanConvertSceneComponentToDynamicMesh(PrimComponent))
			{
				Inputs.Emplace(PrimComponent);
			}
		}
	};
	if (SceneState.SelectedComponents.Num() > 0)
	{
		for (UActorComponent* Selected : SceneState.SelectedComponents)
		{
			TryAddSelected(Selected);
		}
	}
	else
	{
		for (AActor* SelectedActor : SceneState.SelectedActors)
		{
			for (UActorComponent* Selected : SelectedActor->GetComponents())
			{
				TryAddSelected(Selected);
			}
		}
	}
	Tool->InitializeInputs(MoveTemp(Inputs));

#if WITH_EDITOR
	 
	if (UStaticMeshComponentToolTargetFactory* StaticMeshComponentTargetFactory = SceneState.TargetManager->FindFirstFactoryByType<UStaticMeshComponentToolTargetFactory>())
	{
		EMeshLODIdentifier LOD = StaticMeshComponentTargetFactory->GetActiveEditingLOD();
		Tool->SetTargetLOD(LOD);
	}
#endif

	return Tool;
}

bool UConvertMeshesToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int32 CanConvertCount = 0;
	auto CanConvert = [](UActorComponent* Comp)->bool { return UE::Conversion::CanConvertSceneComponentToDynamicMesh(Cast<USceneComponent>(Comp)); };
	if (SceneState.SelectedComponents.Num() > 0)
	{
		CanConvertCount = static_cast<int32>(Algo::CountIf(SceneState.SelectedComponents, CanConvert));
	}
	else
	{
		CanConvertCount =
			Algo::TransformAccumulate(SceneState.SelectedActors,
				[&CanConvert](AActor* Actor)
				{
					return static_cast<int>(Algo::CountIf(Actor->GetComponents(), CanConvert));
				},
				0);
	}
	return CanConvertCount > 0;
}


/*
 * Tool
 */


void UConvertMeshesTool::Setup()
{
	UInteractiveTool::Setup();

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this, TEXT("ConvertMeshesTool"));
	OutputTypeProperties->InitializeDefault();
	

	BasicProperties = NewObject<UConvertMeshesToolProperties>(this);
	BasicProperties->RestoreProperties(this);
	BasicProperties->bShowTransferMaterials = !OutputTypeProperties->bShowVolumeList;
	
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { 
																							OutputTypeProperties->UpdatePropertyVisibility(); 
																							BasicProperties->bShowTransferMaterials = !OutputTypeProperties->bShowVolumeList;
																							});
	AddToolPropertySource(OutputTypeProperties);
	AddToolPropertySource(BasicProperties);

	SetToolDisplayName(LOCTEXT("ToolName", "Convert"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Convert Meshes to a different Mesh Object type"),
		EToolMessageLevel::UserNotification);
}

bool UConvertMeshesTool::CanAccept() const
{
	// Refuse to accept if we have any invalid inputs
	for (const TWeakObjectPtr<UPrimitiveComponent>& Input : Inputs)
	{
		if (!Input.IsValid())
		{
			return false;
		}
	}
	return true;
}

void UConvertMeshesTool::Shutdown(EToolShutdownType ShutdownType)
{
	OutputTypeProperties->SaveProperties(this, TEXT("ConvertMeshesTool"));
	BasicProperties->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("ConvertMeshesToolTransactionName", "Convert Meshes"));

		TArray<AActor*> NewSelectedActors;
		TSet<AActor*> DeleteActors;
		TIndirectArray<FCreateMeshObjectParams> NewMeshObjects;

		// Accumulate info for new mesh objects. Do not immediately create them because then
		// the new Actors will get a unique-name incremented suffix, because the convert-from
		// Actors still exist.
		for (TWeakObjectPtr<UPrimitiveComponent> Input : Inputs)
		{
			if (!Input.IsValid())
			{
				continue;
			}
			UPrimitiveComponent* InputComponent = Input.Get();
			FDynamicMesh3 SourceMesh;
			FTransform SourceTransform;
			TArray<UMaterialInterface*> ComponentMaterials;
			TArray<UMaterialInterface*> AssetMaterials;
			FText ErrorMessage;

			UE::Conversion::FToMeshOptions Options;
			Options.LODType = UE::Conversion::EMeshLODType::SourceModel;
			Options.LODIndex = 0;
			Options.bUseClosestLOD = true;
			if ((int32)TargetLOD <= (int32)EMeshLODIdentifier::LOD7)
			{
				Options.LODIndex = (int32)TargetLOD;
			}
			else if (TargetLOD == EMeshLODIdentifier::MaxQuality)
			{
				Options.LODType = UE::Conversion::EMeshLODType::MaxAvailable;
			}
			else if (TargetLOD == EMeshLODIdentifier::HiResSource)
			{
				Options.LODType = UE::Conversion::EMeshLODType::HiResSourceModel;
			}

			bool bSuccess = UE::Conversion::SceneComponentToDynamicMesh(InputComponent, Options, false, SourceMesh, SourceTransform, ErrorMessage, &ComponentMaterials, &AssetMaterials);
			if (!bSuccess)
			{
				UE_LOGF(LogGeometry, Warning, "Convert Tool failed to convert %ls: %ls", *InputComponent->GetName(), *ErrorMessage.ToString());
				continue;
			}

			AActor* TargetActor = InputComponent->GetOwner();
			check(TargetActor != nullptr);
			DeleteActors.Add(TargetActor);

			// if not transferring materials, need to clear out any existing MaterialIDs
			if (BasicProperties->bTransferMaterials == false && SourceMesh.HasAttributes())
			{
				if (FDynamicMeshMaterialAttribute* MaterialIDs = SourceMesh.Attributes()->GetMaterialID())
				{
					for (int32 tid : SourceMesh.TriangleIndicesItr())
					{
						MaterialIDs->SetValue(tid, 0);
					}
				}
			}

			FString AssetName = TargetActor->GetActorNameOrLabel();

			FCreateMeshObjectParams* NewMeshObjectParamsPtr = new FCreateMeshObjectParams();
			FCreateMeshObjectParams& NewMeshObjectParams = *NewMeshObjectParamsPtr;
			NewMeshObjectParams.TargetWorld = InputComponent->GetWorld();
			NewMeshObjectParams.Transform = (FTransform)SourceTransform;
			NewMeshObjectParams.BaseName = AssetName;
			if (BasicProperties->bTransferMaterials)
			{
				NewMeshObjectParams.Materials = ComponentMaterials;
				NewMeshObjectParams.AssetMaterials = AssetMaterials;
			}
			NewMeshObjectParams.SetMesh(MoveTemp(SourceMesh));
			
			if (BasicProperties->bTransferCollision)
			{
				if (UE::Geometry::ComponentTypeSupportsCollision(InputComponent, UE::Geometry::EComponentCollisionSupportLevel::ReadOnly))
				{
					NewMeshObjectParams.bEnableCollision = true;
					FComponentCollisionSettings CollisionSettings = UE::Geometry::GetCollisionSettings(InputComponent);
					NewMeshObjectParams.CollisionMode = (ECollisionTraceFlag)CollisionSettings.CollisionTypeFlag;

					FSimpleShapeSet3d ShapeSet;
					if (UE::Geometry::GetCollisionShapes(InputComponent, ShapeSet))
					{
						NewMeshObjectParams.CollisionShapeSet = MoveTemp(ShapeSet);
					}
				}
			}

			NewMeshObjects.Add(NewMeshObjectParamsPtr);
		}

		// delete all the existing Actors we want to get rid of
		for (AActor* DeleteActor : DeleteActors)
		{
			DeleteActor->Destroy();
		}

		// spawn new mesh objects
		for (FCreateMeshObjectParams& NewMeshObjectParams : NewMeshObjects)
		{
			OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
			FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
			if (Result.IsOK())
			{
				NewSelectedActors.Add(Result.NewActor);
			}
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewSelectedActors);

		GetToolManager()->EndUndoTransaction();
	}
}





#undef LOCTEXT_NAMESPACE

