// Copyright Epic Games, Inc. All Rights Reserved.

#include "CubeGridTool.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "CanvasTypes.h"
#include "CompositionOps/CubeGridBooleanOp.h"
#include "Distance/DistLine3Ray3.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/LineSetComponent.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/Engine.h"  // GEngine->GetSmallFont()
#include "InteractiveToolChange.h"
#include "InteractiveToolManager.h"
#include "Input/Reply.h"
#include "InputState.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "MeshOpPreviewHelpers.h" //UMeshOpPreviewWithBackgroundCompute
#include "MeshDescriptionToDynamicMesh.h"
#include "Misc/MessageDialog.h"
#include "ModelingObjectsCreationAPI.h"
#include "ModelingToolTargetUtil.h"
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "SceneView.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ToolContextInterfaces.h"
#include "ToolHostCustomizationAPI.h"
#include "ToolTargetManager.h"
#include "ToolTargets/ToolTarget.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "MeshOpPreviewHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CubeGridTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UCubeGridTool"

// General note: This tool used to largely operate in cube grid space. However it turned out
// to be valuable to be able to keep a selection more or less constant while resizing the grid,
// and this was easier to do by generally operating in just the frame space of the grid (not
// scaled into the grid size).

namespace CubeGridToolLocals
{
	FText PushPullModeMessage = LOCTEXT("CubeGridPushPullModeDescription", "Select grid cells and push/pull them to create geometry. "
		"Refer to side panel for shortcuts.");

	FText CornerModeMessage = LOCTEXT("CubeGridCornerModeDescription", "Toggle corner selection for push/pulling by clicking or dragging. "
		"Press Enter or click \"Done\" to accept the result.");

	FText CreatingAssetMessage = LOCTEXT("CreatingNewAssetLabel", "Creating new asset.");

	FText EditingAssetMessage = LOCTEXT("EditingExistingAssetLabel", "Editing existing asset.");

	const FText SelectionChangeTransactionName = LOCTEXT("SelectionChangeTransaction", "Cube Grid Selection Change");
	const FText ModeChangeTransactionName = LOCTEXT("ModeChangeTransaction", "Cube Grid Mode Change");
	const FText CornerModeExtrudeAmountChangeTransactionName = LOCTEXT("CornerExtrudeAmountTransaction", "Corner Push/Pull Amount");

	const FString PropertyCacheIdentifier = TEXT("CubeGridTool");

	const FString& HoverLineSetID(TEXT("HoverLines"));
	const FString& GridLineSetID(TEXT("GridLines"));
	const FString& SelectionLineSetID(TEXT("SelectionLines"));
	const FString& CornerModeLineSetID(TEXT("CornerModeLines"));

	const FColor HoverLineColor(255, 255, 128); // Pale yellow
	const double HoverLineThickness = 2;
	const double HoverLineDepthBias = 0.1;

	const FColor SelectionLineColor(255, 128, 0); // Orange
	const double SelectionLineDepthBias = 0.1;
	const double SelectionGridLineThickness = 1;
	const double SelectionMainLineThickness = 3;

	const FColor GridLineColor(200, 200, 200);
	const double GridLineDepthBias = 0.05;
	const double GridLineThickness = 0.5;

	const FColor UnselectedCornerLineColor(64, 0, 128); // dark purple
	const FColor SelectedCornerLineColor = FColor::Yellow;
	const double CornerLineThickness = 3;
	const int32 CornerCircleNumSteps = 12;
	const FColor CornerModeWireframeColor = FColor::Red;
	const double CornerModeWireframeThickness = SelectionGridLineThickness;
	const double CornerModeWireframeDepthBias = 0.1;

	/**
	 * Undoes the actual mesh change (i.e. after boolean operations)
	 */
	class FCubeGridToolMeshChange : public FToolCommandChange
	{
	public:
		FCubeGridToolMeshChange(TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn)
			: MeshChange(MoveTemp(MeshChangeIn))
		{};

		virtual void Apply(UObject* Object) override
		{
			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			Tool->UpdateUsingMeshChange(*MeshChange, false);
		}
		virtual void Revert(UObject* Object) override
		{
			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			Tool->UpdateUsingMeshChange(*MeshChange, true);
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridToolMeshChange");
		}

	protected:
		TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChange;
	};

	class FCubeGridMeshTransformChange : public FToolCommandChange
	{
	public:
		FCubeGridMeshTransformChange(const FTransform& BeforeIn, const FTransform& AfterIn)
			: Before(BeforeIn)
			, After(AfterIn)
		{};

		virtual void Apply(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCurrentMeshTransform(After);
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCurrentMeshTransform(Before);
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridMeshTransformChange");
		}

	protected:
		FTransform Before;
		FTransform After;
	};

	/** Undoes selection changes */
	class FCubeGridToolSelectionChange : public FToolCommandChange
	{
	public:
		FCubeGridToolSelectionChange(
			bool bHaveStartSelectionBeforeIn, bool bHaveStartSelectionAfterIn,
			const UCubeGridTool::FSelection& SelectionBeforeIn,
			const UCubeGridTool::FSelection& SelectionAfterIn
			)
			: bHaveStartSelectionBefore(bHaveStartSelectionBeforeIn)
			, bHaveStartSelectionAfter(bHaveStartSelectionAfterIn)
			, SelectionBefore(SelectionBeforeIn)
			, SelectionAfter(SelectionAfterIn)
		{};

		virtual void Apply(UObject* Object) override
		{
			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			if (!bHaveStartSelectionAfter)
			{
				Tool->ClearSelection(false);
			}
			else
			{
				Tool->SetSelection(SelectionAfter, false);
			}
		}
		virtual void Revert(UObject* Object) override
		{

			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			if (!bHaveStartSelectionBefore)
			{
				Tool->ClearSelection(false);
			}
			else
			{
				Tool->SetSelection(SelectionBefore, false);
			}
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridToolSelectionChange");
		}

	protected:
		bool bHaveStartSelectionBefore;
		bool bHaveStartSelectionAfter;
		UCubeGridTool::FSelection SelectionBefore;
		UCubeGridTool::FSelection SelectionAfter;
	};

	/** Undoes activating "corner" mode. Not redoable. */
	class FCubeGridToolModeChange : public FToolCommandChange
	{
	public:
		FCubeGridToolModeChange() {};

		virtual bool HasExpired(UObject* Object) const override
		{
			return Cast<UCubeGridTool>(Object)->IsInDefaultMode();
		}

		virtual void Apply(UObject* Object) override
		{
			return;
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->RevertToDefaultMode();
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridToolModeChange");
		}
	};

	class FCornerModeExtrudeAmountChange : public FToolCommandChange
	{
	public:
		FCornerModeExtrudeAmountChange(int32 ExtrudeAmountBeforeIn, int32 ExtrudeAmountAfterIn) 
			: ExtrudeAmountBefore(ExtrudeAmountBeforeIn)
			, ExtrudeAmountAfter(ExtrudeAmountAfterIn)
		
		{};

		virtual bool HasExpired(UObject* Object) const override
		{
			return !Cast<UCubeGridTool>(Object)->IsInCornerMode();
		}

		virtual void Apply(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCurrentExtrudeAmount(ExtrudeAmountAfter);
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCurrentExtrudeAmount(ExtrudeAmountBefore);
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCornerModeExtrudeAmountChange");
		}

	protected:
		int32 ExtrudeAmountBefore = 0;
		int32 ExtrudeAmountAfter = 0;
	};

	class FCornerModeSelectedCornerChange : public FToolCommandChange
	{
	public:
		FCornerModeSelectedCornerChange(bool CornerFlagsBeforeIn[4], bool CornerFlagsAfterIn[4])
		{
			for (int i = 0; i < 4; ++i)
			{
				CornerFlagsBefore[i] = CornerFlagsBeforeIn[i];
				CornerFlagsAfter[i] = CornerFlagsAfterIn[i];
			}
		};

		virtual bool HasExpired(UObject* Object) const override
		{
			return !Cast<UCubeGridTool>(Object)->IsInCornerMode();
		}

		virtual void Apply(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCornerSelection(CornerFlagsAfter);
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetCornerSelection(CornerFlagsBefore);
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCornerModeSelectedCornerChange");
		}

	protected:
		bool CornerFlagsBefore[4] = { false,false,false,false };
		bool CornerFlagsAfter[4] = { false,false,false,false };
	};

	/** Undoes setting bChangesMade, which is used to determine whether the target needs saving. */
	class FCubeGridChangesMadeChange : public FToolCommandChange
	{
	public:
		FCubeGridChangesMadeChange(bool bNowMadeIn = true) 
			: bNowMade(bNowMadeIn) {};

		virtual void Apply(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetChangesMade(bNowMade);
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UCubeGridTool>(Object)->SetChangesMade(!bNowMade);
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridChangesMadeChange");
		}

	protected:
		bool bNowMade = true;
	};

	/** Deals with the tool target portion of an "Accept and Start New" change. The actual
	  asset update and current mesh updates should happen alongside this change.*/
	class FCubeGridTargetResetChange : public FToolCommandChange
	{
	public:
		FCubeGridTargetResetChange(UToolTarget* ToolTargetIn)
			: ToolTarget(ToolTargetIn)
		{};

		virtual void Apply(UObject* Object) override
		{
			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			if (ToolTarget.IsValid())
			{
				UE::ToolTarget::SetSourceObjectVisible(ToolTarget.Get(), true);
			}
			Tool->SetTarget(nullptr);
			Tool->GetToolManager()->DisplayMessage(CreatingAssetMessage, EToolMessageLevel::UserWarning);
		}
		virtual void Revert(UObject* Object) override
		{
			UCubeGridTool* Tool = Cast<UCubeGridTool>(Object);
			Tool->SetTarget(ToolTarget.Get());
			Tool->GetToolManager()->DisplayMessage(EditingAssetMessage, EToolMessageLevel::UserWarning);
			if (ToolTarget.IsValid())
			{
				UE::ToolTarget::SetSourceObjectVisible(ToolTarget.Get(), false);
			}
		}

		virtual FString ToString() const override
		{
			return TEXT("CubeGridToolLocals::FCubeGridAcceptAndStartNewChange");
		}

	protected:
		TWeakObjectPtr<UToolTarget> ToolTarget;
	};
	
	// Attach a frame to the box such that Z points along the given direction.
	// If we change the choices we make here, we may need to adjust how we pick the welded vertices in
	// subtract mode (i.e., which way the values are mirrored when Z is flipped) and the output of
	// GetFaceUVOrientations.
	FOrientedBox3d ConvertToOrientedBox(const FAxisAlignedBox3d& Box, FCubeGrid::EFaceDirection Direction)
	{
		int FlatDim = FCubeGrid::DirToFlatDim(Direction);

		FVector3d GridSpaceZ = FCubeGrid::DirToNormal(Direction);
		FVector3d GridSpaceX = (FlatDim == 0) ? FVector3d::UnitY() : FVector3d::UnitX();
		FVector3d GridSpaceY = GridSpaceZ.Cross(GridSpaceX);

		FVector3d BoxExtents = Box.Extents();

		if (FlatDim == 0)
		{
			// If selection dir was along x axis, then frame z is x, frame x is y, and frame y is x
			BoxExtents = FVector3d(BoxExtents[1], BoxExtents[2], BoxExtents[0]);
		}
		else if (FlatDim == 1)
		{
			// If selection dir was along y axis, then frame z is y, frame x is x, and frame y is z
			Swap(BoxExtents[2], BoxExtents[1]);
		}

		return FOrientedBox3d(FFrame3d(Box.Center(), GridSpaceX, GridSpaceY, GridSpaceZ), BoxExtents);
	}

	
	// Based on direction of the operation, figures out how the faces of the opmesh need to be rotated
	// relative to the UV plane to make the UV's always keep the same orientation relative to the grid.
	// This depends on the choices we make in ConvertToOrientedBox, and is hard to reason through without
	// trial and error... though there may have been a smarter way to pick our frame from direction that
	// would have made this table easier to produce.
	TArray<int32, TFixedAllocator<6>> GetFaceUVOrientations(FCubeGrid::EFaceDirection Direction)
	{
		switch (Direction)
		{
			case FCubeGrid::EFaceDirection::NegativeZ:
				return TArray<int32, TFixedAllocator<6>>{ 2, 2, 2, 2, 2, 2 };
			case FCubeGrid::EFaceDirection::PositiveX:
				return TArray<int32, TFixedAllocator<6>>{ 0, 0, 3, 1, 3, 1 };
			case FCubeGrid::EFaceDirection::NegativeX:
				return TArray<int32, TFixedAllocator<6>>{ 2, 2, 3, 1, 1, 3 };
			case FCubeGrid::EFaceDirection::PositiveY:
				return TArray<int32, TFixedAllocator<6>>{ 2, 2, 0, 0, 1, 3 };
			case FCubeGrid::EFaceDirection::NegativeY:
				return TArray<int32, TFixedAllocator<6>>{ 0, 0, 2, 2, 3, 1 };
		}
		// FCubeGrid::EFaceDirection::PositiveZ:
		return TArray<int32, TFixedAllocator<6>>{ 0, 0, 0, 0, 0, 0 };
	}

	void GetNewSelectionFaceInBox(const FCubeGrid& Grid, const FAxisAlignedBox3d& Box, 
		const FCubeGrid::FCubeFace& FaceIn, FCubeGrid::FCubeFace& FaceOut)
	{
		// Start at the corner of the selection and move a little bit to make sure
		// you're in the first face in the corner.
		FVector3d TowardOtherCorner(Box.Max - Box.Min);
		TowardOtherCorner.Normalize();

		FVector3d PointOnDesiredFace = (Box.Min + Grid.GetCurrentGridCellSize() * TowardOtherCorner)
			/ Grid.GetCurrentGridCellSize();

		FaceOut = FCubeGrid::FCubeFace(
			PointOnDesiredFace,
			FaceIn.GetDirection(),
			Grid.GetGridPower());
	}

	/**
	 * Given grid, start point, extrude direction, and number of blocks to extrude, produces
	 * the frame-space extrusion distance. If the start point is not on the grid, the first
	 * "block" is counted as the distance to get back onto grid in the extrusion direction.
	 */
	double GetFrameSpaceExtrudeDist(const FCubeGrid& CubeGrid, const FVector3d& FrameSpaceStartPoint,
		int32 CurrentExtrudeAmount, FCubeGrid::EFaceDirection Direction)
	{
		double GridSpaceExtrudeDist = CurrentExtrudeAmount; // Will be adjusted

		int FlatDim = FCubeGrid::DirToFlatDim(Direction);
		double GridSpaceExtrudeCoord = FrameSpaceStartPoint[FlatDim] / CubeGrid.GetCurrentGridCellSize();
		double ClosestOnGridCoord = FMath::RoundToDouble(GridSpaceExtrudeCoord);

		// See if we're actually off-grid
		if (FMath::Abs(GridSpaceExtrudeCoord - ClosestOnGridCoord) > KINDA_SMALL_NUMBER)
		{
			double NextOnGrid = CurrentExtrudeAmount > 0 ? FMath::CeilToDouble(GridSpaceExtrudeCoord)
				: FMath::FloorToDouble(GridSpaceExtrudeCoord);
			GridSpaceExtrudeDist += (NextOnGrid - GridSpaceExtrudeCoord);
			GridSpaceExtrudeDist += (CurrentExtrudeAmount > 0 ? -1 : 1);
		}

		return GridSpaceExtrudeDist * CubeGrid.GetCurrentGridCellSize();
	}

	bool IsAnyCornerSelected(bool CornerSelectedFlags[])
	{
		for (int i = 0; i < 4; ++i)
		{
			if (CornerSelectedFlags[i])
			{
				return true;
			}
		}
		return false;
	}

	/** 
	 * @param CornerVector Lines to draw on the corners, for instance to show the depth direction
	 */
	void DrawGridRectangle(ULineSetComponent& LineSet, const FCubeGrid& Grid,
		const FVector3d& GridMin, const FVector3d& GridMax,
		const FColor& Color, double Thickness,
		double DepthBias, const FVector3d* CornerVector = nullptr)
	{
		// We'll step from max/min along one of the differing dimensions to get corners
		int32 DifferingDimension = (GridMin[0] == GridMax[0]) ? 1 : 0;

		FVector3d Corner1 = GridMin;
		Corner1[DifferingDimension] = GridMax[DifferingDimension];

		FVector3d Corner2 = GridMax;
		Corner2[DifferingDimension] = GridMin[DifferingDimension];

		LineSet.AddLine(GridMin, Corner1, Color, Thickness, DepthBias);
		LineSet.AddLine(GridMin, Corner2, Color, Thickness, DepthBias);
		LineSet.AddLine(GridMax, Corner1, Color, Thickness, DepthBias);
		LineSet.AddLine(GridMax, Corner2, Color, Thickness, DepthBias);

		if (CornerVector)
		{
			LineSet.AddLine(GridMin, GridMin + *CornerVector, Color, Thickness, DepthBias);
			LineSet.AddLine(GridMax, GridMax + *CornerVector, Color, Thickness, DepthBias);
			LineSet.AddLine(Corner1, Corner1 + *CornerVector, Color, Thickness, DepthBias);
			LineSet.AddLine(Corner2, Corner2 + *CornerVector, Color, Thickness, DepthBias);
		}
	}

	const int32 MAX_NUM_INTERIOR_GRID_LINES = 1000;

	void DrawGridSection(ULineSetComponent& LineSet, const FCubeGrid& Grid, 
		const FAxisAlignedBox3d& BBox,
		const FColor& Color, double Thickness, 
		double DepthBias, const FVector3d* CornerVector = nullptr)
	{
		// Draw the boundary
		DrawGridRectangle(LineSet, Grid, BBox.Min, BBox.Max, Color, Thickness, DepthBias, CornerVector);

		// Find the two nonzero dimensions of the box
		const FVector3d BoxDimensions = BBox.Max - BBox.Min;
		int32 Dim1 = BoxDimensions[0] != 0 ? 0 : 2;
		int32 Dim2 = BoxDimensions[1] != 0 ? 1 : 2;

		double StepSize = Grid.GetCurrentGridCellSize();

		// Draw the inside only if there aren't too many lines to draw (approximate)
		if (StepSize <= 0 || BoxDimensions[Dim1] / StepSize + BoxDimensions[Dim2] / StepSize > MAX_NUM_INTERIOR_GRID_LINES)
		{
			return;
		}

		// Draws lines that lie in the DimToDrawAlong dimension, along border in DimToStepAlong dimension
		auto DrawParallelInteriorLines = [&BBox, &LineSet, &Grid, &Color, &BoxDimensions, StepSize,
			Thickness, DepthBias](int32 DimToStepAlong, int32 DimToDrawAlong)
		{
			FVector3d BorderDirection = FVector3d::Zero();
			BorderDirection[DimToStepAlong] = StepSize;

			int NumSteps = BoxDimensions[DimToStepAlong] / StepSize;
			if (NumSteps * StepSize == BoxDimensions[DimToStepAlong])
			{
				--NumSteps;
			}

			for (int32 i = 0; i < NumSteps; ++i)
			{
				FVector3d SidePoint = BBox.Min + BorderDirection * (i + 1);
				FVector3d OtherSidePoint = SidePoint;
				OtherSidePoint[DimToDrawAlong] = BBox.Max[DimToDrawAlong];

				LineSet.AddLine(SidePoint, OtherSidePoint, Color, Thickness, DepthBias);
			}
		};
		DrawParallelInteriorLines(Dim1, Dim2);
		DrawParallelInteriorLines(Dim2, Dim1);
	}


	void DisplayLengthsOfEveryNonzeroBoxSide(const FAxisAlignedBox3d& Box, const FTransform& BoxTransform, 
		FCanvas& Canvas, const FSceneView& SceneView)
	{
		if (Box.IsEmpty())
		{
			return;
		}

		FVector3d LocalDimensions = Box.Max - Box.Min;

		if (LocalDimensions.IsZero())
		{
			return;
		}

		FVector3d LocalCenter = Box.Center();
		double DPIScale = Canvas.GetDPIScale();
		UFont* UseFont = GEngine->GetSmallFont();

		for (int MeasuredDim = 0; MeasuredDim < 3; ++MeasuredDim)
		{
			if (LocalDimensions[MeasuredDim] == 0)
			{
				// Flat on this side
				continue;
			}

			double Length = FMath::Abs(LocalDimensions[MeasuredDim] * BoxTransform.GetScale3D()[MeasuredDim]);
			FString String;
			if (FMath::Abs(Length - FMath::RoundToDouble(Length)) < KINDA_SMALL_NUMBER)
			{
				String = FString::Printf(TEXT("%.0f"), Length);
			}
			else
			{
				// Two decimal places if we don't have a round number
				String = FString::Printf(TEXT("%.2f"), Length);
			}

			// Iterate in a square across the other two dimensions
			int OtherDim1 = MeasuredDim == 0 ? 1 : 0;
			int OtherDim2 = MeasuredDim == 2 ? 1 : 2;
			for (int i = 0; i < 2; ++i)
			{
				if (i == 1 && LocalDimensions[OtherDim1] == 0)
				{
					break;
				}

				for (int j = 0; j < 2; ++j)
				{
					if (j == 1 && LocalDimensions[OtherDim2] == 0)
					{
						break;
					}

					FVector3d LocalWriteLocation = Box.Min;
					LocalWriteLocation[MeasuredDim] = LocalCenter[MeasuredDim];
					if (i == 1)
					{
						LocalWriteLocation[OtherDim1] = Box.Max[OtherDim1];
					}
					if (j == 1)
					{
						LocalWriteLocation[OtherDim2] = Box.Max[OtherDim2];
					}

					FVector3d WorldPosition = BoxTransform.TransformPosition(LocalWriteLocation);
					FVector2D PixelPosition;
					SceneView.WorldToPixel(WorldPosition, PixelPosition);
					Canvas.DrawShadowedString(PixelPosition.X / DPIScale, PixelPosition.Y / DPIScale, *String, 
						UseFont, FLinearColor::White);
				}
			}
			
		}

	}

	/** Given a world hit, get a hit face. */
	void ConvertToFaceHit(const FCubeGrid& CubeGrid, ECubeGridToolFaceSelectionMode SelectionMode, 
		 const FRay& WorldRay, double HitT, const FVector3d& Normal, FCubeGrid::FCubeFace& FaceOut,
		 double Tolerance)
	{
		FVector3d WorldPoint = WorldRay.PointAt(HitT);
		bool bSuccess = false;
		switch (SelectionMode)
		{
		case ECubeGridToolFaceSelectionMode::OutsideBasedOnNormal:
			bSuccess = CubeGrid.GetHitGridFaceBasedOnRay(WorldPoint, Normal, FaceOut, false, Tolerance);
			break;
		case ECubeGridToolFaceSelectionMode::InsideBasedOnNormal:
			bSuccess = CubeGrid.GetHitGridFaceBasedOnRay(WorldPoint, Normal, FaceOut, true, Tolerance);
			break;
		case ECubeGridToolFaceSelectionMode::OutsideBasedOnViewRay:
			bSuccess = CubeGrid.GetHitGridFaceBasedOnRay(WorldPoint, -WorldRay.Direction, FaceOut, false, Tolerance);
			break;
		case ECubeGridToolFaceSelectionMode::InsideBasedOnViewRay:
			bSuccess = CubeGrid.GetHitGridFaceBasedOnRay(WorldPoint, -WorldRay.Direction, FaceOut, true, Tolerance);
			break;
		}

		ensureMsgf(bSuccess, TEXT("CubeGridTool: Unable to convert hit location to proper grid face."));
	}

	/**
	 * Given a world ray and a flat box in grid frame space, intersect the ray with the plane containing the box, 
	 * find the selected cell in the cube grid, and project that cell onto the same plane to produce a new frame 
	 * space box.
	 * 
	 * @param bExpandBoxWithStartBox If true, the output is expanded to contain the original box.
	 * @return true If the plane was actually hit.
	 */
	bool GetCoplanarFrameSpaceSelectedBox(const FCubeGrid& CubeGrid, const FRay& WorldRay, 
		const FAxisAlignedBox3d& StartBox, bool bExpandOutputBoxWithStartBox, FAxisAlignedBox3d& BoxOut)
	{
		FVector3d BoxDims = StartBox.Max - StartBox.Min;
		int FlatDim = 0;
		for (int i = 0; i < 3; ++i)
		{
			if (BoxDims[i] == 0)
			{
				FlatDim = i;
				break;
			}
		}

		bool bIntersects = false;
		FVector IntersectionPoint;
		GizmoMath::RayPlaneIntersectionPoint(
			CubeGrid.GetFrame().FromFramePoint(StartBox.Min), 
			CubeGrid.GetFrame().GetAxis(FlatDim), 
			WorldRay.Origin, WorldRay.Direction, 
			bIntersects, IntersectionPoint);

		if (!bIntersects)
		{
			return false;
		}

		FVector3d GridSpaceIntersection = CubeGrid.ToGridPoint(IntersectionPoint);

		FVector3d FrameSpaceMin(
			FMath::Floor(GridSpaceIntersection.X) * CubeGrid.GetCurrentGridCellSize(),
			FMath::Floor(GridSpaceIntersection.Y) * CubeGrid.GetCurrentGridCellSize(),
			FMath::Floor(GridSpaceIntersection.Z) * CubeGrid.GetCurrentGridCellSize());
		FVector3d FrameSpaceMax(
			FMath::CeilToDouble(GridSpaceIntersection.X) * CubeGrid.GetCurrentGridCellSize(),
			FMath::CeilToDouble(GridSpaceIntersection.Y) * CubeGrid.GetCurrentGridCellSize(),
			FMath::CeilToDouble(GridSpaceIntersection.Z) * CubeGrid.GetCurrentGridCellSize());

		// Project the cell we got back onto the plane
		FrameSpaceMin[FlatDim] = StartBox.Min[FlatDim];
		FrameSpaceMax[FlatDim] = StartBox.Min[FlatDim];

		BoxOut = FAxisAlignedBox3d(FrameSpaceMin, FrameSpaceMax);
		if (bExpandOutputBoxWithStartBox)
		{
			BoxOut.Contain(StartBox);
		}

		return true;
	}
}

const FToolTargetTypeRequirements& UCubeGridToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool UCubeGridToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) <= 1;
}

UInteractiveTool* UCubeGridToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UCubeGridTool* NewTool = NewObject<UCubeGridTool>(SceneState.ToolManager);

	TObjectPtr<UToolTarget> Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTarget(Target); // May be null
	if (Target)
	{
		// if there is an element selection, use its frame origin as an initial pivot
		FFrame3d Frame;
		FAxisAlignedBox3d Bounds;
		bool bIsElementSelection;
		if (GetCurrentSelectionWorldFrameBounds(SceneState, Frame, Bounds, bIsElementSelection))
		{
			if (bIsElementSelection)
			{
				NewTool->SetInitialGridPivot(Frame.Origin);
			}
		}
	}
	NewTool->SetWorld(SceneState.World);

	return NewTool;
}



void UCubeGridTool::InvalidatePreview(bool bUpdateCornerLineSet)
{
	using namespace CubeGridToolLocals;

	// Do the line set first
	if (bUpdateCornerLineSet && Mode == EMode::Corner)
	{
		UpdateCornerModeLineSet();
	}

	// If we're guaranteed not to change the result, just reset the preview
	if (CurrentExtrudeAmount == 0 || (CurrentMesh->TriangleCount() == 0 && CurrentExtrudeAmount < 0))
	{
		Preview->CancelCompute();
		LastOpChangedTids = nullptr;
		if (bPreviewMayDiffer)
		{
			Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get());
			Preview->PreviewMesh->SetTransform(CurrentMeshTransform);
			bPreviewMayDiffer = false;
		}
		return;
	}

	Preview->InvalidateResult();
	bPreviewMayDiffer = true;
}

TUniquePtr<FDynamicMeshOperator> UCubeGridTool::MakeNewOperator()
{
	using namespace CubeGridToolLocals;

	// Figure out how far to extrude in grid space. This only becomes tricky if the selection
	// is no longer on grid because we changed grid power.
	double FrameSpaceExtrudeAmount = GetFrameSpaceExtrudeDist(*CubeGrid, Selection.Box.Min, CurrentExtrudeAmount, Selection.Direction);

	FCubeGrid::EFaceDirection ZDirectionInGrid = CurrentExtrudeAmount < 0 ? FCubeGrid::FlipDir(Selection.Direction) : Selection.Direction;

	FOrientedBox3d FrameSpaceBox = ConvertToOrientedBox(Selection.Box, ZDirectionInGrid);

	// Give the selection box depth
	FrameSpaceBox.Frame.Origin += (FrameSpaceExtrudeAmount / 2.0f) * FCubeGrid::DirToNormal(Selection.Direction);
	FrameSpaceBox.Extents.Z = FMath::Abs(FrameSpaceExtrudeAmount) / 2.0f;

	// Translate the oriented box from grid space to world space
	const FFrame3d& GridFrame = CubeGrid->GetFrame();
	FOrientedBox3d WorldBox(FFrame3d(GridFrame.FromFrame(FrameSpaceBox.Frame)), FrameSpaceBox.Extents);

	// Make the op.
	TUniquePtr<FCubeGridBooleanOp> Op = MakeUnique<FCubeGridBooleanOp>();
	Op->InputMesh = ComputeStartMesh;
	Op->InputTransform = CurrentMeshTransform;

	// If we didn't start with an existing mesh, and we are adding to an empty
	// starting mesh, set the transform such that it is in the (grid) minimum of
	// the selection. This frequently (though not always, in some corner-mode cases)
	// places the pivot in a handy corner for snapping.
	if (!Target && ComputeStartMesh->TriangleCount() == 0 && CurrentExtrudeAmount > 0
		&& ensure(bHaveSelection))
	{
		FFrame3d TransformFrame = CubeGrid->GetFrame();
		TransformFrame.Origin = TransformFrame.FromFramePoint(Selection.Box.Min);
		Op->InputTransform = TransformFrame.ToTransform();
	}
	
	Op->bKeepInputTransform = true;
	Op->WorldBox = WorldBox;
	Op->bSubtract = CurrentExtrudeAmount < 0;
	Op->bTrackChangedTids = true;
	Op->OpMeshMaterialID = OpMeshMaterialID;
	Op->OpMeshHeightUVOffset = OpMeshHeightUVOffset;
	if (Settings->bKeepSideGroups)
	{
		if (Op->bSubtract && !OpMeshSubtractSideGroups.IsEmpty())
		{
			Op->OpMeshSideGroups = OpMeshSubtractSideGroups;
		}
		else if (!Op->bSubtract && !OpMeshAddSideGroups.IsEmpty())
		{
			Op->OpMeshSideGroups = OpMeshAddSideGroups;
		}
	}

	Op->FaceUVOrientations = GetFaceUVOrientations(ZDirectionInGrid);
	Op->bWorldSpaceUVs = MaterialProperties->bWorldSpaceUVScale;
	Op->UVScale = MaterialProperties->UVScale;

	if (Mode == EMode::Corner)
	{
		Op->CornerInfo = MakeShared<FCubeGridBooleanOp::FCornerInfo>();
		for (int i = 0; i < 4; ++i)
		{
			Op->CornerInfo->WeldedAtBase[i] = !CornerSelectedFlags[i];
		}

		if (Op->bSubtract)
		{
			// If we're flipping the Z direction of our frame, we currently effectively do so by rotating around 
			// the X axis, which determines which way we need to flip these corner labels.
			Swap(Op->CornerInfo->WeldedAtBase[0], Op->CornerInfo->WeldedAtBase[3]);
			Swap(Op->CornerInfo->WeldedAtBase[1], Op->CornerInfo->WeldedAtBase[2]);
		}

		Op->bCrosswiseDiagonal = Settings->bCrosswiseDiagonal;
	}

	return Op;
}

void UCubeGridTool::SlideSelection(int32 Amount, bool bEmitChange)
{
	using namespace CubeGridToolLocals;

	FVector3d FrameSpaceDisplacement = FCubeGrid::DirToNormal(Selection.Direction)
		* GetFrameSpaceExtrudeDist(*CubeGrid, Selection.StartBox.Min, Amount, Selection.Direction);

	FSelection NewSelection = Selection;
	NewSelection.StartBox = FAxisAlignedBox3d(
		Selection.StartBox.Min + FrameSpaceDisplacement,
		Selection.StartBox.Max + FrameSpaceDisplacement);
	NewSelection.Box = FAxisAlignedBox3d(
		Selection.Box.Min + FrameSpaceDisplacement,
		Selection.Box.Max + FrameSpaceDisplacement);

	SetSelection(NewSelection, bEmitChange);
}

void UCubeGridTool::SetSelection(const FSelection& NewSelection, bool bEmitChange)
{
	using namespace CubeGridToolLocals;

	// Clear op/preview
	if (Mode != EMode::Corner)
	{
		CurrentExtrudeAmount = 0;
		InvalidatePreview();
	}

	if (bEmitChange && (!bHaveSelection || Selection != NewSelection))
	{
		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FCubeGridToolSelectionChange>(bHaveSelection, true, Selection, NewSelection),
			SelectionChangeTransactionName);
	}
	Selection = NewSelection;
	bHaveSelection = true;

	UpdateSelectionLineSet();
	if (Mode == EMode::Corner)
	{
		InvalidatePreview();
	}
}

void UCubeGridTool::ClearSelection(bool bEmitChange)
{
	using namespace CubeGridToolLocals;

	if (bEmitChange && bHaveSelection)
	{
		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FCubeGridToolSelectionChange>(bHaveSelection, false, Selection, Selection),
			SelectionChangeTransactionName);
	}
	bHaveSelection = false;

	UpdateSelectionLineSet();
}


void UCubeGridTool::Setup()
{
	using namespace CubeGridToolLocals;

	UInteractiveTool::Setup();

	GetToolManager()->DisplayMessage(PushPullModeMessage, EToolMessageLevel::UserNotification);

	ToolActions = NewObject<UCubeGridToolActions>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	Settings = NewObject<UCubeGridToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	// Change the default by setting it before restoring
	MaterialProperties->bWorldSpaceUVScale = true;
	MaterialProperties->RestoreProperties(this, PropertyCacheIdentifier);
	AddToolPropertySource(MaterialProperties);

	CurrentMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	CurrentMesh->EnableAttributes();
	CurrentMeshMaterials.Empty();
	if (Target)
	{
		*CurrentMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
		UE::ToolTarget::SetSourceObjectVisible(Target, false);

		CurrentMeshTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
		CurrentMeshMaterials = UE::ToolTarget::GetMaterialSet(Target).Materials;

		ToolActions->GridSourceActor = UE::ToolTarget::GetTargetActor(Target);
	}

	MeshSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>();
	MeshSpatial->SetMesh(CurrentMesh.Get(), true);

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(TargetWorld, this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	if (Target)
	{
		Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get());
	}
	Preview->PreviewMesh->SetTransform((FTransform)CurrentMeshTransform);
	Preview->OnOpCompleted.AddWeakLambda(this, [this](const FDynamicMeshOperator* UncastOp) {
		const FCubeGridBooleanOp* Op = static_cast<const FCubeGridBooleanOp*>(UncastOp);
		if (Op->InputMesh == ComputeStartMesh)
		{
			LastOpChangedTids = Op->ChangedTids;
			if (Op->bSubtract)
			{
				OpMeshSubtractSideGroups = Op->OpMeshSideGroups;
			}
			else
			{
				OpMeshAddSideGroups = Op->OpMeshSideGroups;
			}
			
		}
	});
	UpdateOpMaterials();

	CubeGrid = MakeShared<FCubeGrid>();
	if (bHasInitialGridPivot)
	{
		Settings->GridFrameOrigin = InitialGridPivot;
	}
	CubeGrid->SetGridFrame(FFrame3d(Settings->GridFrameOrigin, Settings->GridFrameOrientation.Quaternion()));
	CubeGrid->SetGridPowerMode(Settings->bPowerOfTwoBlockSizes ? FCubeGrid::EPowerMode::PowerOfTwo : FCubeGrid::EPowerMode::FiveAndTen);
	CubeGrid->SetGridPower(Settings->GridPower);
	CubeGrid->SetCurrentGridCellSize(Settings->CurrentBlockSize);
	Settings->BlockBaseSize = CubeGrid->GetBaseGridCellSize();

	FTransform GridTransform = CubeGrid->GetFrame().ToFTransform();

	GridGizmoTransformProxy = NewObject<UTransformProxy>(this);
	GridGizmoTransformProxy->SetTransform(GridTransform);
	GridGizmoTransformProxy->OnBeginTransformEdit.AddWeakLambda(this, [this](UTransformProxy* Proxy) 
		{
			bInGizmoDrag = true;
		});
	GridGizmoTransformProxy->OnTransformChanged.AddUObject(this, &UCubeGridTool::GridGizmoMoved);
	GridGizmoTransformProxy->OnEndTransformEdit.AddWeakLambda(this,
		[this](UTransformProxy* Proxy) {
			NotifyOfPropertyChangeByTool(Settings);
			UpdateCornerGeometrySet();
			bInGizmoDrag = false;
		});

	GridGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this);
	GridGizmo->SetActiveTarget(GridGizmoTransformProxy, GetToolManager());
	GridGizmoAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	GridGizmoAlignmentMechanic->Setup(this);
	GridGizmoAlignmentMechanic->InitializeDeformedMeshRayCast([this]() { return MeshSpatial.Get(); }, CurrentMeshTransform, nullptr);
	GridGizmoAlignmentMechanic->AddToGizmo(GridGizmo);

	GridGizmo->SetVisibility(false);

	LineSets = NewObject<UPreviewGeometry>();
	LineSets->CreateInWorld(TargetWorld, GridTransform);

	LineSets->AddLineSet(HoverLineSetID);
	LineSets->AddLineSet(SelectionLineSetID);
	LineSets->AddLineSet(CornerModeLineSetID);
	LineSets->SetAllLineSetsMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager(), /*bDepthTested*/ false));

	LineSets->AddLineSet(GridLineSetID);
	LineSets->SetLineSetMaterial(GridLineSetID, 
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager(), /*bDepthTested*/ true));

	UpdateGridLineSet();

	SelectedCornerRenderer.LineThickness = CornerLineThickness;

	ClickDragBehavior = NewObject<UClickDragInputBehavior>();
	ClickDragBehavior->Initialize(this);
	AddInputBehavior(ClickDragBehavior, this);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior, this);

	CtrlMiddleClickBehavior = NewObject<ULocalSingleClickInputBehavior>();
	CtrlMiddleClickBehavior->Initialize();
	CtrlMiddleClickBehavior->SetUseMiddleMouseButton();
	CtrlMiddleClickBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputState) {
		return FInputDeviceState::IsCtrlKeyDown(InputState);
	};
	CtrlMiddleClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FCubeGrid::FCubeFace Face;
		FInputRayHit OutResult;
		OutResult.bHit = GetHitGridFace(InputRay.WorldRay, Face);
		return OutResult;
	};
	CtrlMiddleClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos) {
		OnCtrlMiddleClick(ClickPos);
	};
	AddInputBehavior(CtrlMiddleClickBehavior);

	MiddleClickDragBehavior = NewObject<ULocalClickDragInputBehavior>();
	MiddleClickDragBehavior->Initialize();
	MiddleClickDragBehavior->SetUseMiddleMouseButton();
	MiddleClickDragBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputState) {
		return !FInputDeviceState::IsCtrlKeyDown(InputState);
	};
	MiddleClickDragBehavior->CanBeginClickDragFunc = [this](const FInputDeviceRay& ClickPos) {
		return CanBeginMiddleClickDrag(ClickPos);
	};
	MiddleClickDragBehavior->OnClickPressFunc = [this](const FInputDeviceRay& ClickPos) {
		PrepForSelectionChange();
		RayCastSelectionPlane((FRay3d)ClickPos.WorldRay, MiddleClickDragStart);
	};
	MiddleClickDragBehavior->OnClickDragFunc = [this](const FInputDeviceRay& ClickPos) {
		OnMiddleClickDrag(ClickPos);
	};
	MiddleClickDragBehavior->OnClickReleaseFunc = [this](const FInputDeviceRay&) {
		EndSelectionChange();
	};
	MiddleClickDragBehavior->OnTerminateFunc = [this]() {
		EndSelectionChange();
	};
	AddInputBehavior(MiddleClickDragBehavior);

	GridPowerWatcherIdx = Settings->WatchProperty(Settings->GridPower,
		[this](uint8 NewPower) {
			SetGridPowerClamped(Settings->GridPower);
		});

	auto ChangeCurrentBlockSize = [this](double BlockSizeIn)
	{
		ClearHover();

		CubeGrid->SetCurrentGridCellSize(BlockSizeIn);

		// Update our views of size. The current block size may end up slightly different due to 
		// rounding error, since it's actually the base block size that is authoratative in cubegrid.
		Settings->CurrentBlockSize = CubeGrid->GetCurrentGridCellSize();
		Settings->BlockBaseSize = CubeGrid->GetBaseGridCellSize();

		Settings->SilentUpdateWatcherAtIndex(CurrentBlockSizeWatcherIdx);
		Settings->SilentUpdateWatcherAtIndex(BlockBaseSizeWatcherIdx);
		NotifyOfPropertyChangeByTool(Settings);

		UpdateSelectionLineSet();
		UpdateGridLineSet();
	};

	Settings->WatchProperty(Settings->bPowerOfTwoBlockSizes,
		[ChangeCurrentBlockSize, this](bool bOn) {
			CubeGrid->SetGridPowerMode(Settings->bPowerOfTwoBlockSizes ? FCubeGrid::EPowerMode::PowerOfTwo : FCubeGrid::EPowerMode::FiveAndTen);
			
			// We reset grid power and cell size to default. The reason we do this is because a user is likely to
			// only change this setting if they want the cube sizes to match up to editor grid snapping values. This
			// requires the grid size to be set a particular way, and while a user could realize this and set it
			// appropriately, it feels very clunky to do so.
			SetGridPowerClamped(Settings->DEFAULT_GRID_POWER);
			ChangeCurrentBlockSize(Settings->DEFAULT_CURRENT_BLOCK_SIZE);
		});

	CurrentBlockSizeWatcherIdx = Settings->WatchProperty(Settings->CurrentBlockSize,
		[ChangeCurrentBlockSize](double NewSize) {
			ChangeCurrentBlockSize(NewSize);
		});

	BlockBaseSizeWatcherIdx = Settings->WatchProperty(Settings->BlockBaseSize,
		[this](double NewBaseSize) {
			ClearHover();

			CubeGrid->SetBaseGridCellSize(NewBaseSize);

			Settings->CurrentBlockSize = CubeGrid->GetCurrentGridCellSize();
			Settings->SilentUpdateWatcherAtIndex(CurrentBlockSizeWatcherIdx);
			NotifyOfPropertyChangeByTool(Settings);

			UpdateSelectionLineSet();
			UpdateGridLineSet();
		});
	

	Settings->WatchProperty(Settings->bShowGizmo,
		[this](bool bOn) {
			UpdateGizmoVisibility(bOn);
		});
	Settings->WatchProperty(Settings->bShowGrid,
		[this](bool bShow)
		{
			if (ULineSetComponent* LineSet = LineSets->FindLineSet(GridLineSetID))
			{
				LineSet->SetVisibility(bShow);
			}
		});
	Settings->WatchProperty(Settings->bCrosswiseDiagonal,
		[this](bool bOn) {
			InvalidatePreview();
		});
	Settings->WatchProperty(Settings->bKeepSideGroups,
		[this](bool bOn) {
			InvalidatePreview(false);
		});
	Settings->WatchProperty(Settings->PlaneTolerance,
		[this](double Tolerance) {
			InvalidatePreview(false);
		});

	MaterialProperties->WatchProperty(MaterialProperties->Material,
		[this](TWeakObjectPtr<UMaterialInterface> Material) {
			UpdateOpMaterials();
			InvalidatePreview(false);
		});
	MaterialProperties->WatchProperty(MaterialProperties->UVScale,
		[this](float UVScale) {
			InvalidatePreview(false);
		});
	MaterialProperties->WatchProperty(MaterialProperties->bShowWireframe,
		[this](float UVScale) {
			Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
		});
	MaterialProperties->WatchProperty(MaterialProperties->bWorldSpaceUVScale,
		[this](float UVScale) {
			InvalidatePreview(false);
		});

	GridFrameOriginWatcherIdx = Settings->WatchProperty(Settings->GridFrameOrigin,
		[this](FVector) {
			FTransform Transform(Settings->GridFrameOrientation, Settings->GridFrameOrigin, FVector::One());
			UpdateGridGizmo(Transform, /* bSilentlyUpdate */ true); // Silent to avoid undo/redo during drag in detail panel
			UpdateGridTransform(Transform, /*bUpdateDetailPanel */ false);
		});
	GridFrameOrientationWatcherIdx = Settings->WatchProperty(Settings->GridFrameOrientation,
		[this](FRotator) {
			FTransform Transform(Settings->GridFrameOrientation, Settings->GridFrameOrigin, FVector::One());
			UpdateGridGizmo(Transform, /* bSilentlyUpdate */ true); // Silent to avoid undo/redo during drag in detail panel
			UpdateGridTransform(Transform, /*bUpdateDetailPanel */ false);
		});

	MaterialProperties->SilentUpdateWatched();
	Settings->SilentUpdateWatched();

	UpdateComputeInputs();

	if (Target)
	{
		GetToolManager()->DisplayMessage(EditingAssetMessage,
			EToolMessageLevel::UserWarning);
	}
	else
	{
		GetToolManager()->DisplayMessage(CreatingAssetMessage,
			EToolMessageLevel::UserWarning);
	}
}

void UCubeGridTool::UpdateOpMaterials()
{
	TArray<UMaterialInterface*> PreviewMaterials = CurrentMeshMaterials;
	OpMeshMaterialID = PreviewMaterials.Find(MaterialProperties->Material.Get());
	if (OpMeshMaterialID == INDEX_NONE)
	{
		OpMeshMaterialID = PreviewMaterials.Add(MaterialProperties->Material.Get());
	}

	Preview->ConfigureMaterials(PreviewMaterials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
}

void UCubeGridTool::UpdateComputeInputs()
{
	ComputeStartMesh = MakeShared<FDynamicMesh3>(*CurrentMesh);
}

void UCubeGridTool::Shutdown(EToolShutdownType ShutdownType)
{
	using namespace CubeGridToolLocals;

	Settings->SaveProperties(this);
	MaterialProperties->SaveProperties(this, PropertyCacheIdentifier);
	OutputTypeProperties->SaveProperties(this);

	if (Mode == EMode::Corner)
	{
		ApplyCornerMode(true);
	}

	if (Target)
	{
		UE::ToolTarget::SetSourceObjectVisible(Target.Get(), true);
	}

	// CubeGrid might get used for long stretches at a time, and an accidental Esc hit could result in a fair
	// amount of wasted work. So if the user made any saveable changes, we'll make sure they definitely want to
	// discard their work. Unfortunately we can't currently give a classic "are you sure you want to quit" sort
	// of message because our tool can't actually prevent a shutdown. Hence we have this "do you want to save 
	// the changes instead" message.
	if (ShutdownType == EToolShutdownType::Cancel && bChangesMade && (Target || CurrentMesh->TriangleCount() > 0))
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
			LOCTEXT("AcceptChangesQuestion", "The tool is being cancelled, which normally discards all changes. "
				"Would you like to apply the changes instead?\n\n Selecting \"No\" or closing this window will "
				"discard the tool's current work."), 
			LOCTEXT("AcceptChangesTitle", "Accept changes instead?"));
		if (Ret == EAppReturnType::Yes)
		{
			ShutdownType = EToolShutdownType::Accept;
		}
	}

	if (ShutdownType == EToolShutdownType::Accept && bChangesMade)
	{
		OutputCurrentResults(/*bSetSelection =*/ true);
	}

	Preview->OnOpCompleted.RemoveAll(this);
	Preview->Shutdown();

	if (LineSets)
	{
		LineSets->Disconnect();
		LineSets = nullptr;
	}

	GridGizmoAlignmentMechanic->Shutdown();

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	// Probably unnecessary by this point, but just in case.
	ClearViewportButtonCustomization();
}

// Applies the tool's results to the target or outputs the mesh. Meant to be called during shutdown
// or when doing "Accept and Start New"
void UCubeGridTool::OutputCurrentResults(bool bSetSelection)
{
	if (CurrentMesh->TriangleCount() <= 0)
	{
		if (Target && Target->IsValid())
		{
			EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
				LOCTEXT("DestroyComponentQuestion", "The tool has entirely cut away the source mesh, which is not permitted. "
					"Do you actually want to delete the mesh component? Note that the actor will remain. Choosing No will leave "
					"the component unchanged instead."),
				LOCTEXT("DestroyComponentTitle", "Delete mesh components?"));
			if (Ret == EAppReturnType::No || Ret == EAppReturnType::Cancel)
			{
				return;
			}

			UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
			if (ensure(Component))
			{
				GetToolManager()->BeginUndoTransaction(LOCTEXT("DestroyComponentTransaction", "Delete Mesh Component"));
				UE::ToolTarget::GetTargetComponent(Target)->DestroyComponent();
				GetToolManager()->EndUndoTransaction();
			}
		}

		return;
	}

	bool bCreatingNewAsset = !Target && CurrentMesh->TriangleCount() > 0;
	if (Target)
	{
		if (Target->IsValid())
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("CubeGridToolEditTransactionName", "Cube Grid Edit"));
			FComponentMaterialSet OutputMaterialSet;
			OutputMaterialSet.Materials = CurrentMeshMaterials;

			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *CurrentMesh, true,
				FConversionToMeshDescriptionOptions(), &OutputMaterialSet);
			GetToolManager()->EndUndoTransaction();
			return;
		}
		else if (!Target->IsValid() && CurrentMesh->TriangleCount() > 0)
		{
			EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
				LOCTEXT("RecreateAssetQuestion", "The underlying asset that this tool was "
					"operating on seems to no longer be valid (it was likely forcibly removed). "
					"Would you like to recreate a new asset from the tool's current working "
					"mesh? Selecting \"No\" or closing this window will discard the tool's "
					"current work."),
				LOCTEXT("RecreateAssetTitle", "Recreate Mesh Asset?"));
			if (Ret == EAppReturnType::Yes)
			{
				bCreatingNewAsset = true;
			}
		}
	}

	if (bCreatingNewAsset)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("CubeGridToolCreateTransactionName", "Cube Grid Create New"));

		FCreateMeshObjectParams NewMeshObjectParams;
		NewMeshObjectParams.TargetWorld = TargetWorld;
		NewMeshObjectParams.Transform = (FTransform)CurrentMeshTransform;
		NewMeshObjectParams.BaseName = TEXT("CubeGridToolOutput");
		NewMeshObjectParams.Materials = CurrentMeshMaterials;
		NewMeshObjectParams.SetMesh(CurrentMesh.Get());
		OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
		FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
		if (Result.IsOK() && Result.NewActor != nullptr && bSetSelection)
		{
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}

		GetToolManager()->EndUndoTransaction();
	}
}

void UCubeGridTool::AcceptToolAndStartNew()
{
	using namespace CubeGridToolLocals;

	if (Mode == EMode::Corner)
	{
		ApplyCornerMode(true);
	}

	if (!bChangesMade && !Target)
	{
		return;
	}

	FText TransactionText = LOCTEXT("AcceptAndStartNewTransaction", "Cube Grid Accept and Start New");
	GetToolManager()->BeginUndoTransaction(TransactionText);

	if (bChangesMade)
	{
		OutputCurrentResults(/*bSetSelection =*/ false);
		GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridChangesMadeChange>(false), TransactionText);
		bChangesMade = false;
	}

	if (Target)
	{
		UE::ToolTarget::SetSourceObjectVisible(Target.Get(), true);
		GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridTargetResetChange>(Target), TransactionText);
		SetTarget(nullptr);
	}
	GetToolManager()->DisplayMessage(CreatingAssetMessage, EToolMessageLevel::UserWarning);

	// Clear out the current mesh
	if (CurrentMesh->TriangleCount() > 0)
	{
		// TODO: There are probably better ways to do this than a dynamic mesh change tracker. For
		// instance we could have a change that keeps a weak pointer to the mesh and keep a shared
		// pointer in the tool, so that the mesh is thrown away once the tool exits (and the change
		// is expired). Though we might need to be a bit more careful with retargeting the spatial, etc.
		// For now, we do this easy approach.
		TArray<int32> AllTids;
		for (int32 Tid : CurrentMesh->TriangleIndicesItr())
		{
			AllTids.Add(Tid);
		}
		FDynamicMeshChangeTracker ChangeTracker(CurrentMesh.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(AllTids, true /*bSaveVertices*/);
		FDynamicMeshEditor Editor(CurrentMesh.Get());
		Editor.RemoveTriangles(AllTids, true);
		GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridToolMeshChange>(ChangeTracker.EndChange()), TransactionText);
	}

	// Clear out any other structures/state
	MeshSpatial->Build();
	UpdateComputeInputs();
	bWaitingToApplyPreview = false;
	CurrentExtrudeAmount = 0;
	Preview->CancelCompute();
	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get());

	GetToolManager()->EndUndoTransaction();
}

void UCubeGridTool::OnTick(float DeltaTime)
{
	if (Preview)
	{
		Preview->Tick(DeltaTime);
	}

	if (PendingAction != ECubeGridToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = ECubeGridToolAction::NoAction;
	}

	if (bWaitingToApplyPreview && 
		// Note that the check for bPreviewMayDiffer is needed, because when we do a forced
		// reset, HaveValidResult() returns false, yet we may still want to do an application
		// to slide the selection.
		CanApplyPreview())
	{
		ApplyPreview();
	}
}

void UCubeGridTool::ApplyPreview()
{
	using namespace CubeGridToolLocals;

	bWaitingToApplyPreview = false;
	bPreviewMayDiffer = false;
	bBlockUntilPreviewUpdate = false;

	const FText TransactionText = LOCTEXT("CubeGridToolTransactionName", "Block Tool Change");
	GetToolManager()->BeginUndoTransaction(TransactionText);

	// See if we're actually making a change to the mesh.
	if (CurrentExtrudeAmount > 0 || (CurrentExtrudeAmount < 0 && LastOpChangedTids.IsValid() && LastOpChangedTids->Num() > 0))
	{
		FDynamicMeshChangeTracker ChangeTracker(CurrentMesh.Get());
		ChangeTracker.BeginChange();
		ChangeTracker.SaveTriangles(*LastOpChangedTids, true /*bSaveVertices*/);

		// Update current mesh
		bool bWasValid = Preview->ProcessCurrentMesh([this](const FDynamicMesh3& Mesh)
		{
			CurrentMesh->Copy(Mesh);
		});
		if (!ensure(bWasValid))
		{
			CurrentExtrudeAmount = 0;
			return;
		}

		// The transform might have changed if we added to an empty mesh
		if (!FTransform(CurrentMeshTransform).Equals(Preview->PreviewMesh->GetTransform()))
		{
			GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridMeshTransformChange>(
				CurrentMeshTransform, Preview->PreviewMesh->GetTransform()), TransactionText);

			CurrentMeshTransform = Preview->PreviewMesh->GetTransform();
			GridGizmoAlignmentMechanic->InitializeDeformedMeshRayCast([this]() { return MeshSpatial.Get(); }, 
				CurrentMeshTransform, nullptr);
		}

		// Mark our tool as having done something, if we haven't already.
		if (!bChangesMade)
		{
			bChangesMade = true;
			GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridChangesMadeChange>(), TransactionText);
		}

		MeshSpatial->Build();

		CurrentMeshMaterials.Reset();
		Preview->PreviewMesh->GetMaterials(CurrentMeshMaterials);

		UpdateComputeInputs();

		GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridToolMeshChange>(ChangeTracker.EndChange()), TransactionText);
	}

	if (bAdjustSelectionOnPreviewUpdate && CurrentExtrudeAmount != 0)
	{
		// Save UV offset before sliding selection
		OpMeshHeightUVOffset += GetFrameSpaceExtrudeDist(*CubeGrid, Selection.Box.Min, CurrentExtrudeAmount, Selection.Direction);

		// Change the selection to the new location. Note that this should happen only
		// after resetting bPreviewMayDiffer to avoid an extra preview reset when selection
		// changes.
		SlideSelection(CurrentExtrudeAmount, true);
	}

	// This actually also happens as a side effect of SlideSelection above, but here for clarity.
	CurrentExtrudeAmount = 0;

	GetToolManager()->EndUndoTransaction();
}

void UCubeGridTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace CubeGridToolLocals;

	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (Mode == EMode::Corner)
	{
		FOrientedBox3d OrientedBox = ConvertToOrientedBox(Selection.Box, Selection.Direction);

		SelectedCornerRenderer.BeginFrame(RenderAPI, CameraState);
		for (int i = 0; i < 4; ++i)
		{
			FVector WorldPosition = CubeGrid->GetFrame().FromFramePoint(OrientedBox.GetCorner(i));

			// Depending on whether we're in an orthographic view or not, we set the radius based on visual angle or based on ortho 
			// viewport width (divided into 90 segments like the FOV is divided into 90 degrees).
			float Radius = (CameraState.bIsOrthographic) ? CameraState.OrthoWorldCoordinateWidth * 0.5 / 90.0
				: (float)ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, (FVector3d)WorldPosition, 0.5);
			bool bDepthTested = false;
			SelectedCornerRenderer.DrawViewFacingCircle(
				WorldPosition,
				Radius,
				CornerCircleNumSteps,
				CornerSelectedFlags[i] ? SelectedCornerLineColor : UnselectedCornerLineColor,
				CornerLineThickness, bDepthTested);
		}
		SelectedCornerRenderer.EndFrame();
	}

	GridGizmoAlignmentMechanic->Render(RenderAPI);
}

void UCubeGridTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	using namespace CubeGridToolLocals;

	if (bHaveSelection && Canvas && CubeGrid 
		&& Settings && Settings->bShowSelectionMeasurements)
	{
		if (const FSceneView* SceneView = RenderAPI->GetSceneView())
		{
			DisplayLengthsOfEveryNonzeroBoxSide(Selection.Box, CubeGrid->GetFrame().ToFTransform(), 
				*Canvas, *SceneView);
		}
	}
}

void UCubeGridTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
}

void UCubeGridTool::ClearHover() 
{
	if (bHaveHoveredSelection)
	{
		UpdateHoverLineSet(false, HoveredSelectionBox);
	}
}

void  UCubeGridTool::GridGizmoMoved(UTransformProxy* Proxy, FTransform Transform)
{
	UpdateGridTransform(Transform, /* bUpdateDetailPanel */ true,
		// If we're here due to a drag, then we don't want to trigger the detail panel
		// rebuild (it will be triggered on drag end).
		!bInGizmoDrag);
}

void UCubeGridTool::OnCtrlMiddleClick(const FInputDeviceRay& ClickPos)
{
	// Get the selected face
	FCubeGrid::FCubeFace Face;
	if (!ensure(GetHitGridFace(ClickPos.WorldRay, Face)))
	{
		return;
	}

	if (!GridGizmo->IsVisible())
	{
		UpdateGizmoVisibility(true);
	}

	// Get the face's four corners in grid space
	FVector3d FaceMin = Face.GetMinCorner();
	FVector3d FaceMax = Face.GetMaxCorner();

	FVector3d Corners[4] = {
		FaceMin,
		FaceMin,
		FaceMax,
		FaceMax
	};

	int32 DifferingDimension = (FaceMin[0] == FaceMax[0]) ? 1 : 0;

	Corners[1][DifferingDimension] = FaceMax[DifferingDimension];
	Corners[3][DifferingDimension] = FaceMin[DifferingDimension];

	// Transform the ray to grid space and see which of the corners is closest.
	FVector3d GridSpaceRayOrigin = CubeGrid->ToGridPoint(ClickPos.WorldRay.Origin);
	FVector3d GridSpaceRayDirection = CubeGrid->GetFrame().ToFrameVector(FVector3d(ClickPos.WorldRay.Direction));
	GridSpaceRayDirection.Normalize();
	FRay3d GizmoSpaceRay(GridSpaceRayOrigin, GridSpaceRayDirection);

	double MinDistSquared = GizmoSpaceRay.DistSquared(Corners[0]);
	int32 ClosestCornerIndex = 0;
	for (int32 i = 1; i < 4; ++i)
	{
		double DistSquared = GizmoSpaceRay.DistSquared(Corners[i]);
		if (DistSquared < MinDistSquared)
		{
			MinDistSquared = DistSquared;
			ClosestCornerIndex = i;
		}
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("QuickAdjustGizmo", "Transform Gizmo"));
	
	// Adjust the selection if needed
	if (bHaveSelection)
	{
		FVector3d GridSpaceDisplacement = Corners[ClosestCornerIndex];
		FVector3d FrameSpaceDisplacement = GridSpaceDisplacement * CubeGrid->GetCurrentGridCellSize();

		FSelection NewSelection = Selection;
		NewSelection.StartBox = FAxisAlignedBox3d(
			Selection.StartBox.Min - FrameSpaceDisplacement,
			Selection.StartBox.Max - FrameSpaceDisplacement);
		NewSelection.Box = FAxisAlignedBox3d(
			Selection.Box.Min - FrameSpaceDisplacement,
			Selection.Box.Max - FrameSpaceDisplacement);

		SetSelection(NewSelection, true);
	}

	// Move the gizmo to that corner.
	UpdateGridGizmo(FTransform(Settings->GridFrameOrientation, CubeGrid->ToWorldPoint(Corners[ClosestCornerIndex]), FVector::One()));

	GetToolManager()->EndUndoTransaction();
}

// Tries to intersect the selected box. Used for middle mouse dragging the selection.
FInputRayHit UCubeGridTool::RayCastSelectionPlane(const FRay3d& WorldRay,
	FVector3d& HitPointOut)
{
	FVector3d Normal = CubeGrid->GetFrame().FromFrameVector(
		FCubeGrid::DirToNormal(Selection.Direction));

	FInputRayHit HitResult;
	FVector HitPoint;
	GizmoMath::RayPlaneIntersectionPoint(CubeGrid->GetFrame().FromFramePoint(Selection.Box.Min), Normal,
		WorldRay.Origin, WorldRay.Direction, HitResult.bHit, HitPoint);
	
	if (HitResult.bHit)
	{
		HitPointOut = HitPoint;
		HitResult = FInputRayHit(WorldRay.GetParameter(HitPointOut));
	}
	return HitResult;
}

FInputRayHit UCubeGridTool::CanBeginMiddleClickDrag(const FInputDeviceRay& ClickPos)
{
	FInputRayHit HitResult;
	if (!bHaveSelection)
	{
		return HitResult;
	}

	FVector3d WorldHitPoint;
	HitResult = RayCastSelectionPlane((FRay3d)ClickPos.WorldRay, WorldHitPoint);

	int FlatDim = FCubeGrid::DirToFlatDim(Selection.Direction);

	FVector3d FrameSpacePoint = CubeGrid->GetFrame().ToFramePoint(WorldHitPoint);
	FrameSpacePoint[FlatDim] = Selection.Box.Min[FlatDim];
	HitResult.bHit = Selection.Box.Contains(FrameSpacePoint);

	return HitResult;
}

void UCubeGridTool::OnMiddleClickDrag(const FInputDeviceRay& DragPos)
{
	if (!bHaveSelection)
	{
		return;
	}

	FVector3d MiddleClickDragEnd;
	RayCastSelectionPlane((FRay3d)DragPos.WorldRay, MiddleClickDragEnd);
	FVector3d DisplacementInGridFrame = CubeGrid->GetFrame().ToFrameVector(MiddleClickDragEnd - MiddleClickDragStart); 
	
	// Clamp the relevant dimension in the displacement vector
	DisplacementInGridFrame[FCubeGrid::DirToFlatDim(Selection.Direction)] = 0;

	// Make the displacement be a multiple of the current grid cell size
	DisplacementInGridFrame /= CubeGrid->GetCurrentGridCellSize();
	DisplacementInGridFrame = FVector3d(
		FMath::RoundToDouble(DisplacementInGridFrame.X), 
		FMath::RoundToDouble(DisplacementInGridFrame.Y),
		FMath::RoundToDouble(DisplacementInGridFrame.Z));
	DisplacementInGridFrame *= CubeGrid->GetCurrentGridCellSize();

	FAxisAlignedBox3d NewSelectionBox(
		PreviousSelection.Box.Min + DisplacementInGridFrame,
		PreviousSelection.Box.Max + DisplacementInGridFrame
	);

	// Adjust selection
	if (NewSelectionBox != Selection.Box)
	{
		Selection.StartBox = FAxisAlignedBox3d(
			PreviousSelection.StartBox.Min + DisplacementInGridFrame,
			PreviousSelection.StartBox.Max + DisplacementInGridFrame
		);

		Selection.Box = FAxisAlignedBox3d(
			PreviousSelection.Box.Min + DisplacementInGridFrame,
			PreviousSelection.Box.Max + DisplacementInGridFrame
		);

		UpdateSelectionLineSet();
		InvalidatePreview();
	}
}

void UCubeGridTool::PrepForSelectionChange()
{
	bPreviousHaveSelection = bHaveSelection;
	PreviousSelection = Selection;
}

void UCubeGridTool::EndSelectionChange()
{
	using namespace CubeGridToolLocals;

	if (bPreviousHaveSelection != bHaveSelection
		|| (bHaveSelection && PreviousSelection != Selection))
	{
		ResetMultiStepConsistencyData();

		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FCubeGridToolSelectionChange>(bPreviousHaveSelection, bHaveSelection,
				PreviousSelection, Selection),
			SelectionChangeTransactionName);
	}
}

void UCubeGridTool::ResetMultiStepConsistencyData()
{
	OpMeshHeightUVOffset = 0;
	OpMeshAddSideGroups.Empty();
	OpMeshSubtractSideGroups.Empty();
}

void UCubeGridTool::UpdateGizmoVisibility(bool bVisible)
{
	GridGizmo->SetVisibility(bVisible);
	LineSets->SetLineSetMaterial(CubeGridToolLocals::GridLineSetID,
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager(), /*bDepthTested*/ !bVisible));
	Settings->bShowGizmo = bVisible;
	Settings->SilentUpdateWatched();
}

void UCubeGridTool::UpdateGridGizmo(const FTransform& NewTransform, bool bSilentlyUpdate)
{
	if (bSilentlyUpdate)
	{
		GridGizmo->ReinitializeGizmoTransform(NewTransform);
	}
	else
	{
		GridGizmo->SetNewGizmoTransform(NewTransform);
	}
}

void UCubeGridTool::UpdateGridTransform(const FTransform& NewTransform, bool bUpdateDetailPanel, bool bTriggerDetailPanelRebuild)
{
	if (bUpdateDetailPanel)
	{
		Settings->GridFrameOrigin = NewTransform.GetTranslation();
		Settings->SilentUpdateWatcherAtIndex(GridFrameOriginWatcherIdx);

		Settings->GridFrameOrientation = NewTransform.GetRotation().Rotator();
		Settings->SilentUpdateWatcherAtIndex(GridFrameOrientationWatcherIdx);

		if (bTriggerDetailPanelRebuild)
		{
			NotifyOfPropertyChangeByTool(Settings);
		}
	}

	CubeGrid->SetGridFrame(FFrame3d(NewTransform));
	LineSets->SetTransform(NewTransform);

	if (Mode == EMode::Corner)
	{
		InvalidatePreview(false);

		if (!bInGizmoDrag)
		{
			// Don't need to update corner hit points while dragging since we can't drag and click corners at the same time.
			UpdateCornerGeometrySet();
		}
	}
}

bool UCubeGridTool::GetHitGridFace(const FRay& WorldRay, FCubeGrid::FCubeFace& FaceOut)
{
	using namespace CubeGridToolLocals;

	double BestHitT = TNumericLimits<double>::Max();

	// We always hit-test the ground plane...
	bool bHitPlane;
	FVector IntersectionPoint;
	GizmoMath::RayPlaneIntersectionPoint(CubeGrid->GetFrame().Origin, CubeGrid->GetFrame().Z(), 
		WorldRay.Origin, WorldRay.Direction, bHitPlane, IntersectionPoint);
	if (bHitPlane)
	{
		FVector3d ClampedGridPoint = CubeGrid->ToGridPoint(IntersectionPoint);
		ClampedGridPoint.Z = 0;
		FaceOut = FCubeGrid::FCubeFace(ClampedGridPoint,
			CubeGrid->ToGridPoint(WorldRay.Origin).Z >= 0 ? FCubeGrid::EFaceDirection::PositiveZ : FCubeGrid::EFaceDirection::NegativeZ,
			CubeGrid->GetGridPower());
		BestHitT = WorldRay.GetParameter(IntersectionPoint);
	}

	// ...However depending on the settings, we may give everything else priority, which we do by
	// keeping the plane hit distance maximal.
	if (!Settings->bHitGridGroundPlaneIfCloser)
	{
		BestHitT = TNumericLimits<double>::Max();
	}

	if (Settings->bHitUnrelatedGeometry)
	{
		FHitResult HitResult;
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, HitResult, WorldRay)
			&& HitResult.Distance < BestHitT)
		{
			BestHitT = HitResult.Distance;
			ConvertToFaceHit(*CubeGrid, Settings->FaceSelectionMode, 
				WorldRay, BestHitT, HitResult.ImpactNormal, FaceOut, Settings->PlaneTolerance);
		}
	}

	if (MeshSpatial)
	{
		FRay3d LocalRay(CurrentMeshTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
			CurrentMeshTransform.InverseTransformVector((FVector3d)WorldRay.Direction));

		int32 Tid;
		double LocalHitT = TNumericLimits<double>::Max();
		if (MeshSpatial->FindNearestHitTriangle(LocalRay, LocalHitT, Tid))
		{
			double HitT = WorldRay.GetParameter(CurrentMeshTransform.TransformPosition(LocalRay.PointAt(LocalHitT)));
			if (HitT < BestHitT)
			{
				BestHitT = HitT;
				ConvertToFaceHit(*CubeGrid, Settings->FaceSelectionMode,
					WorldRay, BestHitT, CurrentMeshTransform.TransformNormal(CurrentMesh->GetTriNormal(Tid)), 
					FaceOut, Settings->PlaneTolerance);
			}
		}
	}

	// We can't go just off of BestHitT because we keep it maximal for plane hits when 
	// bHitGridGroundPlaneIfCloser is false.
	return bHitPlane || BestHitT != TNumericLimits<double>::Max();
}

FInputRayHit UCubeGridTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit HitResult;
	HitResult.bHit = (Mode != EMode::FitGrid && !(bBlockUntilPreviewUpdate && bWaitingToApplyPreview));
	return HitResult;
}


void UCubeGridTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	using namespace CubeGridToolLocals;

	UpdateHoverLineSet(false, HoveredSelectionBox); // clear hover

	// Ctrl+drag extrude setting works in both corner mode and regular mode
	if (bMouseDragShouldPushPull)
	{
		MouseState = EMouseState::DraggingExtrudeDistance;
		if (bHaveSelection)
		{
			DragProjectionAxis = FRay3d(CubeGrid->GetFrame().FromFramePoint(Selection.Box.Center()),
				CubeGrid->GetFrame().FromFrameVector(FCubeGrid::DirToNormal(Selection.Direction)), true);

			FDistLine3Ray3d DistanceCalculator(UE::Geometry::FLine3d(DragProjectionAxis.Origin, DragProjectionAxis.Direction), (FRay3d)PressPos.WorldRay);
			DistanceCalculator.ComputeResult();
			DragProjectedStartParam = DistanceCalculator.LineParameter;
			DragStartExtrudeAmount = CurrentExtrudeAmount;
		}
		return;
	}

	// Deal with corner selection if in corner mode
	if (Mode == EMode::Corner)
	{
		MouseState = EMouseState::DraggingCornerSelection;
		for (int i = 0; i < 4; ++i)
		{
			PreDragCornerSelectedFlags[i] = CornerSelectedFlags[i];
		}
		AttemptToSelectCorner((FRay3d)PressPos.WorldRay);
		return;
	}

	// Otherwise, deal with selection
	PrepForSelectionChange();

	MouseState = EMouseState::DraggingRegularSelection;
	FCubeGrid::FCubeFace HitFace;
	if (bHaveSelection && bSelectionToggle)
	{
		// We're adding to existing selection
		if (GetCoplanarFrameSpaceSelectedBox(*CubeGrid, PressPos.WorldRay, Selection.StartBox, true, Selection.Box))
		{
			UpdateSelectionLineSet();
		}
	} 
	else if (GetHitGridFace(PressPos.WorldRay, HitFace))
	{
		// Reset start of the selection
		bHaveSelection = true;
		double GridScale = CubeGrid->GetCellSize(HitFace.GetSourceCubeGridPower());
		Selection.Box = FAxisAlignedBox3d(HitFace.GetMinCorner() * GridScale, HitFace.GetMaxCorner() * GridScale);
		Selection.StartBox = Selection.Box;
		Selection.Direction = HitFace.GetDirection();
		UpdateSelectionLineSet();
	}
	else
	{
		// Clear selection (the event emit, if needed, happens on click release) 
		bHaveSelection = false;
		Selection.Box = FAxisAlignedBox3d();
		UpdateSelectionLineSet();
	}
}

void UCubeGridTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	using namespace CubeGridToolLocals;

	if (!bHaveSelection)
	{
		return;
	}

	if (MouseState == EMouseState::DraggingExtrudeDistance)
	{
		if (!bHaveSelection || (Mode == EMode::Corner && !IsAnyCornerSelected(CornerSelectedFlags)))
		{
			return;
		}

		FDistLine3Ray3d DistanceCalculator(
			UE::Geometry::FLine3d(DragProjectionAxis.Origin, DragProjectionAxis.Direction), (FRay3d)DragPos.WorldRay);
		DistanceCalculator.ComputeResult();

		double ParamDelta = DistanceCalculator.LineParameter - DragProjectedStartParam;
		double CubeSize = CubeGrid->GetCurrentGridCellSize();
		int32 NewExtrudeDelta = FMath::RoundToInt(ParamDelta / (CubeSize * Settings->BlocksPerStep)) * Settings->BlocksPerStep;
		int32 NewExtrudeAmount = DragStartExtrudeAmount + NewExtrudeDelta;
		if (NewExtrudeAmount != CurrentExtrudeAmount)
		{
			CurrentExtrudeAmount = NewExtrudeAmount;
			InvalidatePreview();
		}
		return;
	}
	else if (MouseState == EMouseState::DraggingCornerSelection)
	{
		AttemptToSelectCorner((FRay3d)DragPos.WorldRay);
		return;
	}
	else // Grid selection
	{
		FCubeGrid::FCubeFace HitFace;
		double HitT = TNumericLimits<double>::Max();
		bool bHit = GetCoplanarFrameSpaceSelectedBox(*CubeGrid, DragPos.WorldRay, Selection.StartBox,
			true, Selection.Box);

		UpdateSelectionLineSet();
	}
}

void UCubeGridTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	using namespace CubeGridToolLocals;

	if (MouseState == EMouseState::DraggingCornerSelection)
	{
		bool bCornerSelectionChanged = false;
		for (int i = 0; i < 4; ++i)
		{
			if (PreDragCornerSelectedFlags[i] != CornerSelectedFlags[i])
			{
				bCornerSelectionChanged = true;
				break;
			}
		}
		if (bCornerSelectionChanged)
		{
			GetToolManager()->EmitObjectChange(this,
				MakeUnique<FCornerModeSelectedCornerChange>(PreDragCornerSelectedFlags, CornerSelectedFlags),
				LOCTEXT("CornerSelectionTransaction", "Corner Selection"));
		}
	}
	else if (MouseState == EMouseState::DraggingExtrudeDistance)
	{
		// Only apply result if we're not in corner mode, because in corner mode
		// we apply when exiting corner mode (that behavior is particularly important 
		// when using E/Q to set extrude distance, to allow different slopes to be
		// set).
		if (Mode != EMode::Corner && CurrentExtrudeAmount != 0)
		{
			bWaitingToApplyPreview = true;
			bBlockUntilPreviewUpdate = false;
			bAdjustSelectionOnPreviewUpdate = true;
		}
		else if (Mode == EMode::Corner && CurrentExtrudeAmount != DragStartExtrudeAmount)
		{
			GetToolManager()->EmitObjectChange(this, 
				MakeUnique<FCornerModeExtrudeAmountChange>(DragStartExtrudeAmount, CurrentExtrudeAmount), 
				CornerModeExtrudeAmountChangeTransactionName);
		}
	}
	else if (MouseState == EMouseState::DraggingRegularSelection)
	{
		EndSelectionChange();
	}

	MouseState = EMouseState::NotDragging;
}

void UCubeGridTool::OnTerminateDragSequence()
{
	if (MouseState == EMouseState::DraggingExtrudeDistance)
	{
		// Only apply result if we're not in corner mode
		if (Mode != EMode::Corner && CurrentExtrudeAmount != 0)
		{
			bWaitingToApplyPreview = true;
			bBlockUntilPreviewUpdate = false;
			bAdjustSelectionOnPreviewUpdate = true;
		}
	}

	MouseState = EMouseState::NotDragging;
}

void UCubeGridTool::AttemptToSelectCorner(const FRay3d& WorldRay)
{
	TArray<FGeometrySet3::FNearest> HitCorners;
	CornersGeometrySet.CollectPointsNearRay(WorldRay, HitCorners, [this](const FVector3d& Position1, const FVector3d& Position2) {
		double ToleranceScale = 3;
		if (CameraState.bIsOrthographic)
		{
			// We could just always use ToolSceneQueriesUtil::PointSnapQuery. But in ortho viewports, we happen to know
			// that the only points that we will ever give this function will be the closest points between a ray and
			// some geometry, meaning that the vector between them will be orthogonal to the view ray. With this knowledge,
			// we can do the tolerance computation more efficiently than PointSnapQuery can, since we don't need to project
			// down to the view plane.
			// As in PointSnapQuery, we convert our angle-based tolerance to one we can use in an ortho viewport (instead of
			// dividing our field of view into 90 visual angle degrees, we divide the plane into 90 units).
			float OrthoTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * CameraState.OrthoWorldCoordinateWidth / 90.0;
			OrthoTolerance *= ToleranceScale;
			return FVector3d::DistSquared(Position1, Position2) < OrthoTolerance * OrthoTolerance;
		}
		else
		{
			return ToolSceneQueriesUtil::PointSnapQuery(CameraState,
				Position1, Position2,
				ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * ToleranceScale);
		}
		});

	for (FGeometrySet3::FNearest Hit : HitCorners)
	{
		CornerSelectedFlags[Hit.ID] = !PreDragCornerSelectedFlags[Hit.ID];
	}

	if (HitCorners.Num() > 0)
	{
		InvalidatePreview();
	}
}

void UCubeGridTool::UpdateSelectionLineSet()
{
	using namespace CubeGridToolLocals;

	ULineSetComponent* LineSet = LineSets->FindLineSet(SelectionLineSetID);
	LineSet->Clear();
	if (bHaveSelection)
	{
		FVector3d CornerVector = FCubeGrid::DirToNormal(Selection.Direction)
			* GetFrameSpaceExtrudeDist(*CubeGrid, Selection.Box.Min, -Settings->BlocksPerStep, Selection.Direction);
		DrawGridRectangle(*LineSet, *CubeGrid, Selection.StartBox.Min, Selection.StartBox.Max, SelectionLineColor,
			SelectionMainLineThickness, SelectionLineDepthBias);
		DrawGridRectangle(*LineSet, *CubeGrid, Selection.Box.Min, Selection.Box.Max, SelectionLineColor,
			SelectionMainLineThickness, SelectionLineDepthBias);
		DrawGridSection(*LineSet, *CubeGrid, Selection.Box, SelectionLineColor, 
			SelectionGridLineThickness, SelectionLineDepthBias, &CornerVector);

		if (Mode == EMode::Corner)
		{
			// This isn't quite relevant to updating the selection line set, but it's a convenient place
			// to put this because if the selection set is changing, the geometry set probably needs
			// to be doing the same.
			UpdateCornerGeometrySet();
		}
	}
}

FInputRayHit UCubeGridTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit HitResult;
	HitResult.bHit = UpdateHover(PressPos.WorldRay);
	return HitResult;
}

bool UCubeGridTool::UpdateHover(const FRay& WorldRay)
{
	using namespace CubeGridToolLocals;

	if (Mode != EMode::PushPull)
	{
		UpdateHoverLineSet(false, HoveredSelectionBox);
		return false;
	}

	bool bHit;
	UE::Geometry::FAxisAlignedBox3d Box;
	if (bHaveSelection && bSelectionToggle)
	{
		bHit = GetCoplanarFrameSpaceSelectedBox(*CubeGrid, WorldRay, Selection.StartBox, false, Box);
	}
	else
	{
		FCubeGrid::FCubeFace HitFace;
		bHit = GetHitGridFace(WorldRay, HitFace);
		if (bHit)
		{
			double HoverScale = CubeGrid->GetCurrentGridCellSize();
			Box = FAxisAlignedBox3d(HitFace.GetMinCorner() * HoverScale,
				HitFace.GetMaxCorner() * HoverScale);
		}
	}

	UpdateHoverLineSet(bHit, Box);

	return bHit;
}

void UCubeGridTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
}

bool UCubeGridTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return UpdateHover(DevicePos.WorldRay);
}

// We could have not taken arguments here and done it the way we do selection, but we'd need
// to keep track of previous hover to avoid unnecessary updates
void UCubeGridTool::UpdateHoverLineSet(bool bNewHaveHover, const UE::Geometry::FAxisAlignedBox3d& NewHoveredBox)
{
	using namespace CubeGridToolLocals;

	ULineSetComponent* LineSet = LineSets->FindLineSet(HoverLineSetID);

	if (!bNewHaveHover)
	{
		if (bHaveHoveredSelection)
		{
			LineSet->Clear();
		}
	}
	else if (!bHaveHoveredSelection || NewHoveredBox != HoveredSelectionBox)
	{
		HoveredSelectionBox = NewHoveredBox;
		bHaveHoveredSelection = true;

		LineSet->Clear();
		DrawGridRectangle(*LineSet, *CubeGrid, HoveredSelectionBox.Min, HoveredSelectionBox.Max,
			HoverLineColor, HoverLineThickness, HoverLineDepthBias);
	}
}

void UCubeGridTool::OnEndHover()
{
}

void UCubeGridTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifierID)
	{
		bSelectionToggle = bIsOn;
	}
	else if (ModifierID == CtrlModifierID)
	{
		bMouseDragShouldPushPull = bIsOn;
	}
}

void UCubeGridTool::UpdateGridLineSet()
{
	using namespace CubeGridToolLocals;

	double CurrentGridScale = CubeGrid->GetCurrentGridCellSize();

	FAxisAlignedBox3d GridBox;
	GridBox.Contain(FVector3d(-50, -50, 0) * CurrentGridScale);
	GridBox.Contain(FVector3d(50, 50, 0) * CurrentGridScale);

	ULineSetComponent* LineSet = LineSets->FindLineSet(GridLineSetID);
	LineSet->Clear();
	DrawGridSection(*LineSet, *CubeGrid, GridBox,
		GridLineColor, GridLineThickness, GridLineDepthBias);
	LineSet->SetVisibility(Settings->bShowGrid);
}

void UCubeGridTool::UpdateCornerModeLineSet()
{
	using namespace CubeGridToolLocals;
	ULineSetComponent* LineSet = LineSets->FindLineSet(CornerModeLineSetID);

	LineSet->Clear();
	if (Mode == EMode::Corner && CurrentExtrudeAmount < 0)
	{
		// Since we're subtracting here, we need to do some flipping.
		FOrientedBox3d FrameSpaceBox = ConvertToOrientedBox(Selection.Box, FCubeGrid::FlipDir(Selection.Direction));
		bool CornerWelded[4];
		for (int i = 0; i < 4; ++i) {
			CornerWelded[i] = !CornerSelectedFlags[i];
		}
		Swap(CornerWelded[0], CornerWelded[3]);
		Swap(CornerWelded[1], CornerWelded[2]);

		// The choice of diagonal here lines up with the generator in CubeGridBooleanOp for
		// the bottom face.
		int DiagStartIdx = 0;
		if (CornerWelded[1] != CornerWelded[3] ||
			(!CornerWelded[1] && CornerWelded[0] && CornerWelded[2]))
		{
			DiagStartIdx = 1;
		}
		DiagStartIdx = Settings->bCrosswiseDiagonal ? 1 - DiagStartIdx : DiagStartIdx;
		
		bool bDiagonalWelded = CornerWelded[DiagStartIdx] && CornerWelded[DiagStartIdx + 2];
		int DeletedVert = -1;
		if (bDiagonalWelded)
		{
			if (CornerWelded[DiagStartIdx + 1])
			{
				DeletedVert = DiagStartIdx + 1;
			}
			else if (CornerWelded[(DiagStartIdx + 3) % 4])
			{
				DeletedVert = (DiagStartIdx + 3) % 4;
			}
		}

		FVector3d CornerExtrudeVector = CubeGrid->GetCurrentGridCellSize() * CurrentExtrudeAmount 
			* FCubeGrid::DirToNormal(Selection.Direction);

		for (int i = 0; i < 4; ++i)
		{
			FVector3d CurrentCorner = FrameSpaceBox.GetCorner(i);
			if (!CornerWelded[i])
			{
				FVector3d UpCorner = CurrentCorner + CornerExtrudeVector;
				LineSet->AddLine(CurrentCorner, UpCorner, CornerModeWireframeColor, 
					CornerModeWireframeThickness, static_cast<float>(CornerModeWireframeDepthBias));
				CurrentCorner = UpCorner;
			}

			int NextIdx = (i + 1) % 4;
			if (i == DeletedVert || NextIdx == DeletedVert)
			{
				continue;
			}

			FVector3d NextCorner = CornerWelded[NextIdx] ? FrameSpaceBox.GetCorner(NextIdx)
				: FrameSpaceBox.GetCorner(NextIdx) + CornerExtrudeVector;
			LineSet->AddLine(CurrentCorner, NextCorner, CornerModeWireframeColor,
				CornerModeWireframeThickness, static_cast<float>(CornerModeWireframeDepthBias));
		}

		FVector3d DiagCorner1 = CornerWelded[DiagStartIdx] ? FrameSpaceBox.GetCorner(DiagStartIdx)
			: FrameSpaceBox.GetCorner(DiagStartIdx) + CornerExtrudeVector;
		FVector3d DiagCorner2 = CornerWelded[DiagStartIdx + 2] ? FrameSpaceBox.GetCorner(DiagStartIdx + 2)
			: FrameSpaceBox.GetCorner(DiagStartIdx + 2) + CornerExtrudeVector;
		LineSet->AddLine(DiagCorner1, DiagCorner2, CornerModeWireframeColor,
			CornerModeWireframeThickness, static_cast<float>(CornerModeWireframeDepthBias));
	}
}

void UCubeGridTool::ApplyFlipSelection()
{
	if (!bHaveSelection)
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("FlipTransactionName", "Flip Selection"));

	FSelection NewSelection = Selection;
	NewSelection.Direction = FCubeGrid::FlipDir(Selection.Direction);
	SetSelection(NewSelection, true);
	ResetMultiStepConsistencyData();

	GetToolManager()->EndUndoTransaction();

	// TODO: We actually probably want some special handling here in Corner mode. For one thing,
	// we're keeping the selected corners the same, which ends up rotating rather than mirroring
	// the currently pushed/pulled portion (should fix this, but would need another undo transaction,
	// at which point we probably want full undo support for corner mode, rather than our current
	// approach of keeping corner selection and extrude distance...). For another, might a user want
	// a flip in corner mode to equate to a reversal of push vs pull, instead of a mirror of the same
	// operation (i.e. you flip a pull and you get a mirrored push rather than mirrored pull)? Not certain.
}

void UCubeGridTool::ApplySlide(int32 NumBlocks)
{
	if (!bHaveSelection)
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("SlideTransactionName", "Slide Selection"));
	SlideSelection(NumBlocks, true);
	ResetMultiStepConsistencyData();

	GetToolManager()->EndUndoTransaction();
}

void UCubeGridTool::ApplyPushPull(int32 NumBlocks)
{
	using namespace CubeGridToolLocals;

	if (!bHaveSelection || (Mode == EMode::Corner && !IsAnyCornerSelected(CornerSelectedFlags)))
	{
		return;
	}

	int32 PreviousExtrudeAmount = CurrentExtrudeAmount;
	CurrentExtrudeAmount += NumBlocks;

	InvalidatePreview();

	if (Mode == EMode::PushPull)
	{
		bWaitingToApplyPreview = true;
		bBlockUntilPreviewUpdate = false;
		bAdjustSelectionOnPreviewUpdate = true;
	}
	else if (Mode == EMode::Corner)
	{
		// In corner mode, we don't actually appy the change yet, but we transact the extrude amount.
		// (in regular mode we transact the mesh change once the actual change is applied)
		GetToolManager()->EmitObjectChange(this,
			MakeUnique<FCornerModeExtrudeAmountChange>(PreviousExtrudeAmount, CurrentExtrudeAmount),
			CornerModeExtrudeAmountChangeTransactionName);
	}
}

void UCubeGridTool::SetGridPowerClamped(int32 GridPower)
{
	Settings->GridPower = FMath::Clamp<int32>(GridPower, 0, Settings->MaxGridPower);
	CubeGrid->SetGridPower(Settings->GridPower);
	Settings->SilentUpdateWatcherAtIndex(GridPowerWatcherIdx);

	Settings->CurrentBlockSize = CubeGrid->GetCurrentGridCellSize();
	Settings->SilentUpdateWatcherAtIndex(CurrentBlockSizeWatcherIdx);
	
	NotifyOfPropertyChangeByTool(Settings);

	if (Mode == EMode::Corner)
	{
		InvalidatePreview(); // effective extrude distance is now different
		UpdateCornerModeLineSet();

		// Note: If we wanted to, we could adjust the extrude amount in corner mode such that the currently extruded
		// distance stays roughly the same. However, it's not clear whether that is desirable. On the one hand, it might
		// make it easier to reach certain far-from-zero values if they are not on a big grid boundary. On the other hand,
		// the behavior makes it harder to see how the ramp will change on the next E/Q press.
		// For now we decided against it. If we ever want it, we would use this code instead of the above (requires 
		// keeping track of GridPowerPrevious):
		if constexpr (false)
		{
			// Needs to match behavior in CubeGrid.cpp. We would probably expose it if we used this code block.
			auto GetMultiplierForGridPower = [](bool bPowerOfTwo, uint8 GridPower)
			{
				if (bPowerOfTwo)
				{
					// This is split up into two statements to avoid a static analysis warning about
					// shifting a 32 bit value and casting to a 64 bit value.
					uint32 ShiftedResult = static_cast<uint32>(1) << GridPower;
					return static_cast<double>(ShiftedResult);
				}
				else
				{
					// For FiveAndTen, we multiply by 2 half the time and by 5 the second half, rounding up for 2's.
					uint8 FloorHalfGridPower = GridPower / 2;
					uint32 TwoMultiplier = static_cast<uint32>(1) << (GridPower - FloorHalfGridPower);
					return TwoMultiplier * FMath::Pow(5.0, static_cast<double>(FloorHalfGridPower));
				}
			};

			// This would be made a member in the tool
			uint8 GridPowerPrevious = 0;

			double PreviousMultiplier = GetMultiplierForGridPower(Settings->bPowerOfTwoBlockSizes, GridPowerPrevious);
			double NewMultiplier = GetMultiplierForGridPower(Settings->bPowerOfTwoBlockSizes, Settings->GridPower);;

			double NewExtrudeAmount = CurrentExtrudeAmount * PreviousMultiplier / NewMultiplier;
			// Could consider allowing CurrentExtrudeAmount not to be an int (and moving it to the nearest int
			// on next manipulation).
			CurrentExtrudeAmount = FMath::RoundFromZero(NewExtrudeAmount);

			if (CurrentExtrudeAmount != NewExtrudeAmount)
			{
				InvalidatePreview();
				UpdateCornerModeLineSet();
			}
		}//end unused adjustment code
	}

	ClearHover();
	UpdateSelectionLineSet(); // Updates the grid drawn inside
	UpdateGridLineSet();
}

// Action support

void UCubeGridToolActions::PostAction(ECubeGridToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}

void UCubeGridTool::RequestAction(ECubeGridToolAction ActionType)
{
	if (PendingAction == ECubeGridToolAction::NoAction)
	{
		PendingAction = ActionType;
	}
}

void UCubeGridTool::ApplyAction(ECubeGridToolAction ActionType)
{
	switch (ActionType)
	{
	case ECubeGridToolAction::Push:
		ApplyPushPull(-Settings->BlocksPerStep);
		break;
	case ECubeGridToolAction::Pull:
		ApplyPushPull(Settings->BlocksPerStep);
		break;
	case ECubeGridToolAction::Flip:
		ApplyFlipSelection();
		break;
	case ECubeGridToolAction::SlideForward:
		ApplySlide(-Settings->BlocksPerStep);
		break;
	case ECubeGridToolAction::SlideBack:
		ApplySlide(Settings->BlocksPerStep);
		break;
	case ECubeGridToolAction::DecreaseGridPower:
		// cast is just to be explicit
		SetGridPowerClamped(static_cast<int32>(Settings->GridPower) - 1); 
		break;
	case ECubeGridToolAction::IncreaseGridPower:
		SetGridPowerClamped(Settings->GridPower + 1);
		break;

	case ECubeGridToolAction::CornerMode:
		StartCornerMode();
		break;
	//case ECubeGridToolAction::FitGrid:
	//	StartFitGrid();
	//	break;
	case ECubeGridToolAction::ResetFromActor:
		if (ToolActions->GridSourceActor)
		{
			FTransform TransformToUse = ToolActions->GridSourceActor->GetTransform();
			TransformToUse.SetScale3D(FVector::OneVector);
			UpdateGridGizmo(TransformToUse);
		}
		break;
	case ECubeGridToolAction::AcceptAndStartNew:
		AcceptToolAndStartNew();
		break;
	}
}

void UCubeGridTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	using namespace CubeGridToolLocals;

	int32 ActionID = (int32)EStandardToolActions::BaseClientDefinedActionID + 1;
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("PullBlock"),
		LOCTEXT("PullAction", "Pull Out Blocks"),
		FText(),
		EModifierKey::None, EKeys::E,
		[this]() { RequestAction(ECubeGridToolAction::Pull); });
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("PushBlock"),
		LOCTEXT("PushAction", "Push In Holes"),
		FText(),
		EModifierKey::None, EKeys::Q,
		[this]() { RequestAction(ECubeGridToolAction::Push); });
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("SlideBack"),
		LOCTEXT("SlideBackAction", "Slide Selection Back"),
		FText(),
		EModifierKey::Shift, EKeys::E,
		[this]() { RequestAction(ECubeGridToolAction::SlideBack); });
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("SlideForward"),
		LOCTEXT("SlideForwardAction", "Slide Selection Forward"),
		FText(),
		EModifierKey::Shift, EKeys::Q,
		[this]() { RequestAction(ECubeGridToolAction::SlideForward); });

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("DecreaseGridPower"),
		LOCTEXT("DecreaseGridPowerAction", "Decrease Grid Power"),
		FText(),
		// Note that we can't use Ctrl+Q on Mac because that is mapped to Cmd+Q which kills the editor.
		// At the same time we can't use Option+E because Mac consumes that for typing accented letters
#if PLATFORM_MAC
		EModifierKey::Alt, EKeys::A,
#else
		EModifierKey::Control, EKeys::Q,
#endif
		[this]() { RequestAction(ECubeGridToolAction::DecreaseGridPower); });
	ActionSet.RegisterAction(this, ActionID++,
		TEXT("IncreaseGridPower"),
		LOCTEXT("IncreaseGridPowerAction", "Increase Grid Power"),
		FText(),
#if PLATFORM_MAC
		EModifierKey::Alt, EKeys::D,
#else
		EModifierKey::Control, EKeys::E,
#endif
		[this]() { RequestAction(ECubeGridToolAction::IncreaseGridPower); });

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("AcceptAndStartNew"),
		LOCTEXT("AcceptAndStartNewAction", "Accept and Start New"),
		FText(),
		EModifierKey::None, EKeys::N,
		[this]() {RequestAction(ECubeGridToolAction::AcceptAndStartNew); });

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("ToggleGizmoVisibility"),
		LOCTEXT("ToggleGizmoVisibilityAction", "Toggle Gizmo Visibility"),
		FText(),
		EModifierKey::None, EKeys::R,
		[this]() {
			if (Mode != EMode::FitGrid)
			{
				UpdateGizmoVisibility(!GridGizmo->IsVisible());
			}
		});

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("ToggleCornerMode"),
		LOCTEXT("ToggleCornerModeAction", "Toggle Corner Mode"),
		FText(),
		EModifierKey::None, EKeys::Z,
		[this]() {
			if (Mode != EMode::Corner)
			{
				StartCornerMode();
			}
			else
			{
				ApplyCornerMode();
			}
		});

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("ToggleDiagonalMode"),
		LOCTEXT("ToggleDiagonalModeAction", "Toggle Diagonal Mode"),
		FText(),
		EModifierKey::None, EKeys::X,
		[this]() {
			if (Mode == EMode::Corner)
			{
				Settings->bCrosswiseDiagonal = !Settings->bCrosswiseDiagonal;
			}
		});

	ActionSet.RegisterAction(this, ActionID++,
		TEXT("FlipSelection"),
		LOCTEXT("FlipSelectionAction", "Flip Selection"),
		FText(),
		EModifierKey::None, EKeys::T,
		[this]() {
			ApplyFlipSelection();
		});
}

FBox UCubeGridTool::GetWorldSpaceFocusBox()
{
	using namespace CubeGridToolLocals;

	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	if (bHaveSelection)
	{
		FOrientedBox3d FrameSpaceBox = ConvertToOrientedBox(Selection.Box, Selection.Direction);

		// The resulting oriented box is flat in the z frame direction (which is aligned to Selection.Direction).
		// So we only need to contain the four corners of a z face instead of doing 8 corners.
		const FFrame3d& GridFrame = CubeGrid->GetFrame();
		int ZFaceIndex = 0;
		for (int CornerIndex = 0; CornerIndex < 4; ++CornerIndex)
		{
			Bounds.Contain(GridFrame.FromFramePoint(FrameSpaceBox.GetCorner(IndexUtil::BoxFaces[ZFaceIndex][CornerIndex])));
		}
	}

	return (FBox)Bounds;
}

bool UCubeGridTool::CanApplyPreview() const
{
	return !bPreviewMayDiffer || Preview->HaveValidResult();
}

void UCubeGridTool::StartCornerMode()
{
	using namespace CubeGridToolLocals;

	if (!bHaveSelection)
	{
		// TODO: Write out a message here and clear it at some point
		return;
	}
	if (Mode == EMode::Corner)
	{
		return; // Already in mode
	}

	// Clear/cancel stuff
	//if (Mode == EMode::FitGrid)
	//{
	//	CancelFitGrid();
	//}
	CurrentExtrudeAmount = 0;
	InvalidatePreview();

	// Clear selected corner render
	for (int i = 0; i < 4; ++i)
	{
		CornerSelectedFlags[i] = false;
		PreDragCornerSelectedFlags[i] = false;
	}

	UpdateCornerGeometrySet();
	Mode = EMode::Corner;

	SetToolPropertySourceEnabled(ToolActions, false);
	// Customize the tool accept/cancel buttons to the current activity.
	if (IToolHostCustomizationAPI* ButtonCustomizer = IToolHostCustomizationAPI::Find(GetToolManager()).GetInterface())
	{
		IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams Params;
		Params.Label = LOCTEXT("CornerModeActivityLabel","Corner Mode");
		Params.OverrideAcceptButtonText = LOCTEXT("DoneButton", "Done");
		Params.OverrideAcceptButtonTooltip = LOCTEXT("DoneButtonTooltip", "Apply the current change and exit corner mode.");
		Params.OverrideCancelButtonText = LOCTEXT("CancelButton", "Cancel");
		Params.OverrideCancelButtonTooltip = LOCTEXT("CancelButtonTooltip", "Cancel the current change and exit corner mode.");
		Params.CanAccept = [this]() { return true; };
		Params.OnAcceptCancelTriggered = [this](bool bAccept)
		{
			if (bAccept)
			{
				if (Mode == EMode::Corner)
				{
					ApplyCornerMode();
				}
				else if (Mode == EMode::FitGrid)
				{
					//AcceptFitGrid();
				}
			}
			else
			{
				RevertToDefaultMode();
			}
			return FReply::Handled();
		};

		ButtonCustomizer->RequestAcceptCancelButtonOverride(Params);
	}

	Settings->bInCornerMode = true;
	NotifyOfPropertyChangeByTool(Settings);

	GetToolManager()->BeginUndoTransaction(ModeChangeTransactionName);
	GetToolManager()->EmitObjectChange(this, MakeUnique<FCubeGridToolModeChange>(), ModeChangeTransactionName);
	GetToolManager()->EndUndoTransaction();

	GetToolManager()->DisplayMessage(CornerModeMessage, EToolMessageLevel::UserNotification);
}

void UCubeGridTool::UpdateCornerGeometrySet()
{
	using namespace CubeGridToolLocals;

	CornersGeometrySet.Reset();
	if (bHaveSelection)
	{
		FOrientedBox3d FrameSpaceBox = ConvertToOrientedBox(Selection.Box, Selection.Direction);
		for (int i = 0; i < 4; ++i)
		{
			CornersGeometrySet.AddPoint(i, CubeGrid->GetFrame().FromFramePoint(FrameSpaceBox.GetCorner(i)));
		}
	}
}

void UCubeGridTool::ApplyCornerMode(bool bDontWaitForTick)
{
	using namespace CubeGridToolLocals;

	if (CurrentExtrudeAmount != 0 && IsAnyCornerSelected(CornerSelectedFlags))
	{
		bWaitingToApplyPreview = true;
		bBlockUntilPreviewUpdate = true;
		bAdjustSelectionOnPreviewUpdate = false;

		if (bDontWaitForTick)
		{
			ApplyPreview();
		}
	}

	CornersGeometrySet.Reset();

	Mode = EMode::PushPull;
	GetToolManager()->DisplayMessage(PushPullModeMessage, EToolMessageLevel::UserNotification);
	SetToolPropertySourceEnabled(ToolActions, true);
	ClearViewportButtonCustomization();

	Settings->bInCornerMode = false;
	NotifyOfPropertyChangeByTool(Settings);

	UpdateCornerModeLineSet();
}

void UCubeGridTool::CancelCornerMode()
{
	using namespace CubeGridToolLocals;

	CornersGeometrySet.Reset();

	Mode = EMode::PushPull;
	GetToolManager()->DisplayMessage(PushPullModeMessage, EToolMessageLevel::UserNotification);
	SetToolPropertySourceEnabled(ToolActions, true);
	ClearViewportButtonCustomization();

	CurrentExtrudeAmount = 0;
	InvalidatePreview();

	Settings->bInCornerMode = false;
	NotifyOfPropertyChangeByTool(Settings);

	UpdateCornerModeLineSet();
}

void UCubeGridTool::UpdateUsingMeshChange(const FDynamicMeshChange& MeshChange, bool bRevert)
{
	MeshChange.Apply(CurrentMesh.Get(), bRevert);
	MeshSpatial->Build();
	UpdateComputeInputs();
	CurrentExtrudeAmount = 0;
	bPreviewMayDiffer = true;
	InvalidatePreview();
}

bool UCubeGridTool::IsInDefaultMode() const
{
	return Mode == EMode::PushPull;
}

bool UCubeGridTool::IsInCornerMode() const
{
	return Mode == EMode::Corner;
}

void UCubeGridTool::RevertToDefaultMode()
{
	if (Mode == EMode::Corner)
	{
		CancelCornerMode();
	}
	else if (Mode == EMode::FitGrid)
	{
		//CancelFitGrid();
	}
}

void UCubeGridTool::ClearViewportButtonCustomization()
{
	if (IToolHostCustomizationAPI* ButtonCustomizer = IToolHostCustomizationAPI::Find(GetToolManager()).GetInterface())
	{
		ButtonCustomizer->ClearButtonOverrides();
	}
}

void UCubeGridTool::SetChangesMade(bool bChangesMadeIn)
{
	bChangesMade = bChangesMadeIn;
}

void UCubeGridTool::SetCurrentMeshTransform(const FTransform& TransformIn)
{
	CurrentMeshTransform = TransformIn;
	InvalidatePreview();
}

// For use by Undo/Redo during corner mode
void UCubeGridTool::SetCurrentExtrudeAmount(int32 ExtrudeAmount)
{
	CurrentExtrudeAmount = ExtrudeAmount;
	InvalidatePreview();
}

// For use by Undo/Redo during corner mode
void UCubeGridTool::SetCornerSelection(bool CornerSelectedFlagsIn[4])
{
	for (int i = 0; i < 4; ++i)
	{
		CornerSelectedFlags[i] = CornerSelectedFlagsIn[i];
	}
	InvalidatePreview();
}

bool UCubeGridTool::CanCurrentlyNestedCancel()
{
	return Mode == EMode::Corner || bHaveSelection;
}

bool UCubeGridTool::ExecuteNestedCancelCommand()
{
	if (!IsInDefaultMode())
	{
		RevertToDefaultMode();
		return true;
	}
	else if (bHaveSelection)
	{
		ClearSelection(true);
		return true;
	}
	return false;
}

bool UCubeGridTool::CanCurrentlyNestedAccept()
{
	return Mode == EMode::Corner;
}

bool UCubeGridTool::ExecuteNestedAcceptCommand()
{
	if (Mode == EMode::Corner)
	{
		ApplyCornerMode();
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

