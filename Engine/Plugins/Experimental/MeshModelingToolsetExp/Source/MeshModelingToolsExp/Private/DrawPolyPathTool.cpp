// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawPolyPathTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

#include "ToolSceneQueriesUtil.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Generators/PolygonEdgeMeshGenerator.h"
#include "Distance/DistLine3Line3.h"
#include "ModelingObjectsCreationAPI.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Selection/ToolSelectionUtil.h"
#include "Operations/ExtrudeMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "MeshBoundaryLoops.h"
#include "ToolDataVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawPolyPathTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDrawPolyPathTool"


namespace DrawPolyPathToolLocals
{
	void ComputeArcLengths(const TArray<FFrame3d>& PathPoints, TArray<double>& ArcLengths)
	{
		double CurPathLength = 0;
		ArcLengths.SetNum(PathPoints.Num());
		ArcLengths[0] = 0.0f;
		for (int32 k = 1; k < PathPoints.Num(); ++k)
		{
			CurPathLength += Distance(PathPoints[k].Origin, PathPoints[k - 1].Origin);
			ArcLengths[k] = CurPathLength;
		}
	}

	/// Generate path mesh
	/// @return Offset to location of generated mesh 
	[[nodiscard]] FVector3d GeneratePathMesh(FDynamicMesh3& Mesh,
		const TArray<FFrame3d>& InPathPoints, 
		const TArray<double>& InOffsetScaleFactors, 
		double OffsetDistance, 
		bool bPathIsClosed, 
		bool bRampMode, 
		bool bSinglePolyGroup,
		bool bRoundedCorners,
		double CornerRadiusFraction,
		bool bLimitCornerRadius,
		int NumCornerArcPoints)
	{
		Mesh.Clear();

		TArray<FFrame3d> UsePathPoints = InPathPoints;
		TArray<double> UseOffsetScaleFactors = InOffsetScaleFactors;

		// re-center the input points at the origin
		FVector3d Center(0, 0, 0);
		if (UsePathPoints.Num())
		{
			FAxisAlignedBox3d PathBounds;
			for (const FFrame3d& Point : UsePathPoints)
			{
				PathBounds.Contain(Point.Origin);
			}
			Center = PathBounds.Center();
			for (FFrame3d& Point : UsePathPoints)
			{
				Point.Origin -= Center;
			}
		}

		if (bPathIsClosed && bRampMode)
		{
			// Duplicate vertices at the beginning/end of the path when generating a ramp
			const FFrame3d FirstPoint = InPathPoints[0];
			UsePathPoints.Add(FirstPoint);
			const double FirstScaleFactor = InOffsetScaleFactors[0];
			UseOffsetScaleFactors.Add(FirstScaleFactor);
		}

		const int NumPoints = UsePathPoints.Num();

		TArray<double> ArcLengths;
		ComputeArcLengths(UsePathPoints, ArcLengths);
		double PathLength = ArcLengths.Last();

		const double PathWidth = 2.0 * OffsetDistance;
		const double CornerRadius = CornerRadiusFraction * PathWidth;
		
		FPolygonEdgeMeshGenerator MeshGen(UsePathPoints, bPathIsClosed, UseOffsetScaleFactors, PathWidth, FVector3d::UnitZ(), bRoundedCorners, CornerRadius, bLimitCornerRadius, NumCornerArcPoints);
		MeshGen.bSinglePolyGroup = bSinglePolyGroup;
		MeshGen.UVWidth = PathLength;
		MeshGen.UVHeight = 2 * OffsetDistance;
		MeshGen.Generate();
		Mesh.Copy(&MeshGen);

		Mesh.EnableVertexUVs(FVector2f::Zero());

		if (bRampMode)
		{
			// Temporarily set vertex UVs to arclengths, for use in interpolating height in ramp mode

			if (bRoundedCorners)
			{
				// If we added arcs to the corners, recompute arc lengths
				const int N = Mesh.VertexCount() / 2;
				ArcLengths.Init(0.0, N);
				double CurPathLength = 0;
				for (int k = 1; k < N; ++k)
				{
					CurPathLength += Distance(Mesh.GetVertex(2 * k), Mesh.GetVertex(2 * (k - 1)));
					ArcLengths[k] = CurPathLength;
				}
				PathLength = ArcLengths.Last();
			}

			int NumMeshVertices = Mesh.VertexCount();
			ensure(NumMeshVertices == Mesh.MaxVertexID());
			ensure(NumMeshVertices == 2 * ArcLengths.Num());

			for (int k = 0; k < NumMeshVertices/2; ++k)
			{
				const float Alpha = static_cast<float>(ArcLengths[k] / PathLength);
				Mesh.SetVertexUV(2 * k, FVector2f(Alpha, static_cast<float>(k)));
				Mesh.SetVertexUV(2 * k + 1, FVector2f(Alpha, static_cast<float>(k)));
			}

			if (bPathIsClosed)
			{
				// Set last vertex positions to match first vertex locations so we can construct the vertical wall
				Mesh.SetVertex(NumMeshVertices - 2, Mesh.GetVertex(0));
				Mesh.SetVertex(NumMeshVertices - 1, Mesh.GetVertex(1));
			}
		}

		return Center;
	}

}	// namespace DrawPolyPathToolLocals


/*
 * ToolBuilder
 */
bool UDrawPolyPathToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UDrawPolyPathToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UDrawPolyPathTool* NewTool = NewObject<UDrawPolyPathTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	NewTool->SetInitialDrawFrame(ToolSetupUtil::GetDefaultWorldReferenceFrame(SceneState.ToolManager));
	return NewTool;
}

/*
* Tool methods
*/
void UDrawPolyPathTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UDrawPolyPathTool::Setup()
{
	UInteractiveTool::Setup();

	// register click behavior
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	ClickBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	AddInputBehavior(ClickBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	AddInputBehavior(HoverBehavior);

	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->CanUpdatePlaneFunc = [this]() { return CanUpdateDrawPlane(); };
	PlaneMechanic->Initialize(TargetWorld, DrawPlaneWorld);
	PlaneMechanic->UpdateClickPriority(ClickBehavior->GetPriority().MakeHigher());
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		DrawPlaneWorld = PlaneMechanic->Plane;
		UpdateSurfacePathPlane();
	});

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	// add properties
	TransformProps = NewObject<UDrawPolyPathProperties>(this);
	TransformProps->RestoreProperties(this);
	AddToolPropertySource(TransformProps);

	TransformProps->WatchProperty(TransformProps->WidthMode, [this](EDrawPolyPathWidthMode Mode)
	{
		if (State == EState::SettingWidth)
		{
			// Switch to the other mode of setting width.
			BeginSettingWidth();
		}
		else if (State != EState::DrawingPath)
		{
			UpdatePathPreview();
		}
	});
	TransformProps->WatchProperty(TransformProps->RadiusMode, [this](EDrawPolyPathRadiusMode Mode)
	{
		if (State == EState::SettingRadius)
		{
			// Switch to the other mode of setting radius.
			BeginSettingRadius();
		}
		else if (State != EState::DrawingPath)
		{
			UpdatePathPreview();
		}
	});
	TransformProps->WatchProperty(TransformProps->ExtrudeMode, [this](EDrawPolyPathExtrudeMode Mode)
	{
		if (State == EState::SettingHeight)
		{
			BeginSettingHeight();
		}
	});

	ExtrudeProperties = NewObject<UDrawPolyPathExtrudeProperties>();
	ExtrudeProperties->RestoreProperties(this);
	AddToolPropertySource(ExtrudeProperties);
	SetToolPropertySourceEnabled(ExtrudeProperties, false);

	// initialize material properties for new objects
	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	MaterialProperties->RestoreProperties(this);
	MaterialProperties->bShowExtendedOptions = false;
	AddToolPropertySource(MaterialProperties);

	// begin path draw
	InitializeNewSurfacePath();

	SetToolDisplayName(LOCTEXT("ToolName", "Path Extrude"));
}

void UDrawPolyPathTool::Shutdown(EToolShutdownType ShutdownType)
{
	// If we have a ready result, go ahead and create it, because especially with "fixed"
	// height mode, it's easy to think that Enter should result in its creation.
	// TODO: We could consider letting this tool accept nested accept/cancel commands, but
	// it's a bit more work.
	if (State == EState::SettingHeight && ShutdownType != EToolShutdownType::Cancel)
	{
		CurHeight = TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Flat ? 0.0 : TransformProps->ExtrudeHeight;
		OnCompleteExtrudeHeight();
	}
	else
	{
		bRestoreInputSelection = true;
	}

	if (bHasSavedWidth)
	{
		TransformProps->Width = SavedWidth;
		bHasSavedWidth = false;
	}
	if (bHasSavedRadius)
	{
		TransformProps->CornerRadius = SavedRadius;
		bHasSavedRadius = false;
	}
	if (bHasSavedExtrudeHeight)
	{
		TransformProps->ExtrudeHeight = SavedExtrudeHeight;
		SavedExtrudeHeight = false;
	}

	PlaneMechanic->Shutdown();
	PlaneMechanic = nullptr;

	OutputTypeProperties->SaveProperties(this);
	TransformProps->SaveProperties(this);
	ExtrudeProperties->SaveProperties(this);
	MaterialProperties->SaveProperties(this);

	ClearPreview();
}

void UDrawPolyPathTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (!ensure(Property))
	{
		// Not sure whether this would ever happen, but just in case
		return;
	}

	// We deal with these properties here instead of inside a watcher because we frequently update them
	// ourselves as we set them interactively, and we don't want to bother with doing silent watcher updates.
	// Instead we rely on the fact that user scrubbing of the properties calls OnPropertyModified whereas
	// programmatic changes generally don't.
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UDrawPolyPathProperties, Width)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UDrawPolyPathProperties, CornerRadius))
	{
		UpdatePathPreview();
	}
	else if (State == EState::SettingHeight
		&& Property->GetFName() == GET_MEMBER_NAME_CHECKED(UDrawPolyPathProperties, ExtrudeHeight)
		&& (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Fixed 
			|| TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed))
	{
		CurHeight = TransformProps->ExtrudeHeight;
		UpdateExtrudePreview();
	}
	else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UDrawPolyPathProperties, bRoundedCorners))
	{
		if (!TransformProps->bRoundedCorners && State == EState::SettingRadius)
		{
			GetToolManager()->EmitObjectChange(this,
				MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp, EState::SettingRadius),
				LOCTEXT("CancelRadiusTransactionName", "Cancel Setting Corner Radius"));
			OnCompleteRadius();
		}
		else if(State != EState::DrawingPath)
		{
			UpdatePathPreview();
		}
	}
}

bool UDrawPolyPathTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	if (SurfacePathMechanic != nullptr)
	{
		FFrame3d HitPoint;
		if (SurfacePathMechanic->IsHitByRay(FRay3d(Ray), HitPoint))
		{
			OutHit.Distance = FRay3d(Ray).GetParameter(HitPoint.Origin);
			OutHit.ImpactPoint = (FVector)HitPoint.Origin;
			OutHit.ImpactNormal = (FVector)HitPoint.Z();
			return true;
		}
		return false;
	}
	else if (CurveDistMechanic != nullptr)
	{
		OutHit.ImpactPoint = Ray.PointAt(100);
		OutHit.Distance = 100;
		return true;
	}
	else if (ExtrudeHeightMechanic != nullptr)
	{
		OutHit.ImpactPoint = Ray.PointAt(100);
		OutHit.Distance = 100;
		return true;
	}

	return false;
}

void UDrawPolyPathTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifierID && bIgnoreSnappingToggle != bIsOn)
	{
		bIgnoreSnappingToggle = bIsOn;
		if (SurfacePathMechanic)
		{
			SurfacePathMechanic->bSnapToWorldGrid = !bIgnoreSnappingToggle;
		}
	}
}

bool UDrawPolyPathTool::IsInputSelectionValidOnOutput()
{
	return bRestoreInputSelection;
}

FInputRayHit UDrawPolyPathTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FHitResult OutHit;
	if (HitTest(ClickPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}

	// background capture, if nothing else is hit
	return FInputRayHit(TNumericLimits<float>::Max());
}


void UDrawPolyPathTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	switch(State)
	{
		case EState::DrawingPath:
			if (SurfacePathMechanic != nullptr 
				&& SurfacePathMechanic->TryAddPointFromRay((FRay3d)ClickPos.WorldRay))
			{
				if (SurfacePathMechanic->IsDone())
				{
					bPathIsClosed = SurfacePathMechanic->LoopWasClosed();
					GetToolManager()->EmitObjectChange(this, 
						MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp, EState::DrawingPath), 
						LOCTEXT("FinishPathTransactionName", "Finish Path"));
					OnCompleteSurfacePath();
				}
				else
				{
					GetToolManager()->EmitObjectChange(this, 
						MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp, EState::DrawingPath),
						LOCTEXT("AddToPathTransactionName", "Add Point to Path"));
				}
			}
			break;
		case EState::SettingWidth:
			if (TransformProps->Width == 0)
			{
				// This can happen accidentally when the user has snapping turned on and it snaps to 0. 
				// We'll ignore that click and show an error message.
				GetToolManager()->DisplayMessage(
					LOCTEXT("ZeroWidthPathError", "Cannot set path width to 0."),
					EToolMessageLevel::UserError);
			}
			else
			{
				GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserError);

				GetToolManager()->EmitObjectChange(this,
					MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp, EState::SettingWidth),
					LOCTEXT("BeginWidthTransactionName", "Set Path Width"));
				OnCompleteWidth();
			}
			break;
		case EState::SettingRadius:
			GetToolManager()->EmitObjectChange(this, 
				MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp, EState::SettingRadius), 
				LOCTEXT("BeginRadiusTransactionName", "Set Corner Radius"));
			OnCompleteRadius();
			break;
		case EState::SettingHeight:
			if (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Interactive
				|| TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive)
			{
				CurHeight = TransformProps->ExtrudeHeight;
			}
			OnCompleteExtrudeHeight();
			break;
	}
}




FInputRayHit UDrawPolyPathTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	if (HitTest(PressPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}

	// background capture, if nothing else is hit
	return FInputRayHit(TNumericLimits<float>::Max());
}


bool UDrawPolyPathTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	switch (State)
	{
	case EState::DrawingPath:
		if (ensure(SurfacePathMechanic != nullptr))
		{
			SurfacePathMechanic->UpdatePreviewPoint((FRay3d)DevicePos.WorldRay);
		}
		break;
	case EState::SettingWidth:
		if (CurveDistMechanic != nullptr)
		{
			CurveDistMechanic->UpdateCurrentDistance(DevicePos.WorldRay);
			if (TransformProps->WidthMode == EDrawPolyPathWidthMode::Interactive)
			{
				double CurveDistance = CurveDistMechanic->CurrentDistance;
				if (!bIgnoreSnappingToggle)
				{
					CurveDistance = ToolSceneQueriesUtil::SnapDistanceToWorldGridSize(this, CurveDistance);
				}
				TransformProps->Width = CurveDistance * 2;
				UpdatePathPreview();
			}
		}
		break;
	case EState::SettingRadius:
		if (CurveDistMechanic != nullptr)
		{
			CurveDistMechanic->UpdateCurrentDistance(DevicePos.WorldRay);
			if (TransformProps->RadiusMode == EDrawPolyPathRadiusMode::Interactive)
			{
				double CurveDistance = CurveDistMechanic->CurrentDistance;
				if (!bIgnoreSnappingToggle)
				{
					CurveDistance = ToolSceneQueriesUtil::SnapDistanceToWorldGridSize(this, CurveDistance);
				}
				TransformProps->CornerRadius = FMath::Clamp(CurveDistance / TransformProps->Width, 0.0, 2.0);
				UpdatePathPreview();
			}
		}
		break;
	case EState::SettingHeight:
		if ((TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Interactive
			|| TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive)
			&& ensure(ExtrudeHeightMechanic != nullptr))
		{
			ExtrudeHeightMechanic->UpdateCurrentDistance(DevicePos.WorldRay);
			
			CurHeight = ExtrudeHeightMechanic->CurrentHeight;
			TransformProps->ExtrudeHeight = ExtrudeHeightMechanic->CurrentHeight;
			UpdateExtrudePreview();
		}
		break;
	}

	return true;
}






void UDrawPolyPathTool::OnTick(float DeltaTime)
{
	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Tick(DeltaTime);
	}
}


void UDrawPolyPathTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Render(RenderAPI);
	}

	if (ExtrudeHeightMechanic != nullptr)
	{
		ExtrudeHeightMechanic->Render(RenderAPI);
	}
	if (CurveDistMechanic != nullptr)
	{
		CurveDistMechanic->Render(RenderAPI);
	}
	if (SurfacePathMechanic != nullptr)
	{
		SurfacePathMechanic->Render(RenderAPI);
	}

	if (CurPolyLoop.Num() > 0)
	{
		FToolDataVisualizer LineRenderer;
		LineRenderer.LineColor = LinearColors::DarkOrange3f();
		LineRenderer.LineThickness = 4.0f;
		LineRenderer.bDepthTested = false;

		LineRenderer.BeginFrame(RenderAPI);

		int32 NumPoints = CurPolyLoop.Num();
		for (int32 k = 0; k < NumPoints; ++k)
		{
			LineRenderer.DrawLine( CurPolyLoop[k], CurPolyLoop[ (k+1) % NumPoints ] );
		}
		if (SecondPolyLoop.Num() > 0)
		{
			NumPoints = SecondPolyLoop.Num();
			for (int32 k = 0; k < NumPoints; ++k)
			{
				LineRenderer.DrawLine( SecondPolyLoop[k], SecondPolyLoop[ (k+1) % NumPoints ] );
			}
		}

		LineRenderer.EndFrame();
	}

}




void UDrawPolyPathTool::InitializeNewSurfacePath()
{
	State = EState::DrawingPath;

	CurveDistMechanic = nullptr;
	CurveDistMechanic = nullptr;

	SurfacePathMechanic = NewObject<UCollectSurfacePathMechanic>(this);
	SurfacePathMechanic->Setup(this);
	double SnapTol = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	SurfacePathMechanic->SpatialSnapPointsFunc = [this, SnapTol](FVector3d Position1, FVector3d Position2)
	{
		return ToolSceneQueriesUtil::PointSnapQuery(this->CameraState, Position1, Position2, SnapTol);
	};
	SurfacePathMechanic->SetDoubleClickOrCloseLoopMode();
	SurfacePathMechanic->bSnapToWorldGrid = !bIgnoreSnappingToggle;

	UpdateSurfacePathPlane();

	ShowStartupMessage();
}


bool UDrawPolyPathTool::CanUpdateDrawPlane() const
{
 	return (State == EState::DrawingPath && SurfacePathMechanic != nullptr && SurfacePathMechanic->HitPath.Num() == 0);
}

void UDrawPolyPathTool::UpdateSurfacePathPlane()
{
	if (SurfacePathMechanic != nullptr)
	{
		SurfacePathMechanic->InitializePlaneSurface(DrawPlaneWorld);
	}
}


void UDrawPolyPathTool::OnCompleteSurfacePath()
{
	check(SurfacePathMechanic != nullptr);

	CurPathPoints = SurfacePathMechanic->HitPath;
	int NumPoints = CurPathPoints.Num();
	// align frames
	FVector3d PlaneNormal = DrawPlaneWorld.Z();
	CurPathPoints[0].ConstrainedAlignAxis(0, UE::Geometry::Normalized(CurPathPoints[1].Origin - CurPathPoints[0].Origin), PlaneNormal);
	CurPathPoints[NumPoints-1].ConstrainedAlignAxis(0, UE::Geometry::Normalized(CurPathPoints[NumPoints-1].Origin - CurPathPoints[NumPoints-2].Origin), PlaneNormal);
	double DistOffsetDelta = 0.01;
	OffsetScaleFactors.SetNum(NumPoints);
	OffsetScaleFactors[0] = OffsetScaleFactors[NumPoints-1] = 1.0;

	// Set local frames for path points. If the path is closed, we will adjust the first and last frames for continuity,
	// otherwise we will leave them as set above.
	int LastPointIndex = bPathIsClosed ? NumPoints : NumPoints - 1;
	int FirstPointIndex = bPathIsClosed ? 0 : 1;
	for (int j = FirstPointIndex; j < LastPointIndex; ++j)
	{
		int NextJ = (j + 1) % NumPoints;
		int PrevJ = (j - 1 + NumPoints) % NumPoints;
		FVector3d Prev(CurPathPoints[PrevJ].Origin), Next(CurPathPoints[NextJ].Origin), Cur(CurPathPoints[j].Origin);
		FLine3d Line1(FLine3d::FromPoints(Prev, Cur)), Line2(FLine3d::FromPoints(Cur, Next));
		Line1.Origin += DistOffsetDelta * PlaneNormal.Cross(Line1.Direction);
		Line2.Origin += DistOffsetDelta * PlaneNormal.Cross(Line2.Direction);

		if (FMath::Abs(Line1.Direction.Dot(Line2.Direction)) > 0.999 )
		{
			CurPathPoints[j].ConstrainedAlignAxis(0, UE::Geometry::Normalized(Next-Prev), PlaneNormal);
			OffsetScaleFactors[j] = 1.0;
		}
		else
		{
			FDistLine3Line3d LineDist(Line1, Line2);
			LineDist.GetSquared();
			FVector3d OffsetPoint = 0.5 * (LineDist.Line1ClosestPoint + LineDist.Line2ClosestPoint);
			OffsetScaleFactors[j] = Distance(OffsetPoint, Cur) / DistOffsetDelta;
			FVector3d TangentDir = UE::Geometry::Normalized(OffsetPoint - Cur).Cross(PlaneNormal);
			CurPathPoints[j].ConstrainedAlignAxis(0, TangentDir, PlaneNormal);
		}
	}

	CurPolyLine.Reset();
	for (const FFrame3d& Point : SurfacePathMechanic->HitPath)
	{
		CurPolyLine.Add(Point.Origin);
	}

	SurfacePathMechanic = nullptr;

	InitializePreviewMesh();

	// Progress to next state
	BeginSettingWidth();
}


void UDrawPolyPathTool::BeginSettingWidth()
{
	// Note that even when the width is constant, we still wait for a click from the user because we 
	// want them to have a chance to edit it in the detail panel, or to switch to interactive width mode.
	
	State = EState::SettingWidth;

	if (TransformProps->WidthMode == EDrawPolyPathWidthMode::Interactive)
	{
		bHasSavedWidth = true;
		SavedWidth = TransformProps->Width;
	}
	else if(bHasSavedWidth)
	{
		TransformProps->Width = SavedWidth;
		bHasSavedWidth = false;
	}

	CurveDistMechanic = NewObject<USpatialCurveDistanceMechanic>(this);
	CurveDistMechanic->Setup(this);
	CurveDistMechanic->InitializePolyCurve(CurPolyLine, FTransform3d::Identity);

	ExtrudeHeightMechanic = nullptr;

	UpdatePathPreview();

	if (TransformProps->WidthMode == EDrawPolyPathWidthMode::Interactive)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("InteractiveSetWidthInstructions", "Set the width of the path by clicking on the drawing plane. Hold Shift to ignore snapping."),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("FixedSetWidthInstructions", "Click in viewport to accept fixed path width, or change it in details panel."),
			EToolMessageLevel::UserNotification);
	}
}

void UDrawPolyPathTool::OnCompleteWidth()
{
	if (TransformProps->bRoundedCorners)
	{
		BeginSettingRadius();
	}
	else
	{
		// Skip radius setting
		OnCompleteRadius();
	}
}


void UDrawPolyPathTool::BeginSettingRadius()
{
	// Note that even when the radius is constant, we still wait for a click from the user because we 
	// want them to have a chance to edit it in the detail panel, or to switch to interactive radius mode.

	State = EState::SettingRadius;

	if (TransformProps->RadiusMode == EDrawPolyPathRadiusMode::Interactive)
	{
		bHasSavedRadius = true;
		SavedRadius = TransformProps->CornerRadius;
	}
	else if (bHasSavedRadius)
	{
		TransformProps->CornerRadius = SavedRadius;
		bHasSavedRadius = false;
	}

	CurveDistMechanic = NewObject<USpatialCurveDistanceMechanic>(this);
	CurveDistMechanic->Setup(this);
	CurveDistMechanic->InitializePolyCurve(CurPolyLine, FTransform3d::Identity);

	ExtrudeHeightMechanic = nullptr;

	UpdatePathPreview();

	if (TransformProps->RadiusMode == EDrawPolyPathRadiusMode::Interactive)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("InteractiveSetRadiusInstructions", "Set the radius of the corners by clicking on the drawing plane. Hold Shift to ignore snapping."),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("FixedSetRadiusInstructions", "Click in viewport to accept fixed corner radius, or change it in details panel."),
			EToolMessageLevel::UserNotification);
	}
}

void UDrawPolyPathTool::OnCompleteRadius()
{
	BeginSettingHeight();
}


void UDrawPolyPathTool::OnCompleteExtrudeHeight()
{
	ExtrudeHeightMechanic = nullptr;

	ClearPreview();

	EmitNewObject();

	InitializeNewSurfacePath();
	CurrentCurveTimestamp++;
}


void UDrawPolyPathTool::UpdatePathPreview()
{
	if (EditPreview == nullptr)
	{
		return;
	}

	FDynamicMesh3 PathMesh;
	FVector3d MeshCenter = GeneratePathMesh(PathMesh);
	EditPreview->SetTransform(FTransform3d(MeshCenter));

	if (State == EState::SettingHeight)
	{
		EditPreview->InitializeExtrudeType(MoveTemp(PathMesh), DrawPlaneWorld.Z(), nullptr, false);
		UpdateExtrudePreview();
	}
	else
	{
		EditPreview->ReplaceMesh(MoveTempIfPossible(PathMesh));
	}
}


FVector3d UDrawPolyPathTool::GeneratePathMesh(FDynamicMesh3& Mesh) 
{
	CurPolyLoop.Reset();
	SecondPolyLoop.Reset();

	const bool bRampMode = (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed) || (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive);
	constexpr bool bLimitCornerRadius = true;
	FVector3d MeshCenter = DrawPolyPathToolLocals::GeneratePathMesh(Mesh, 
		CurPathPoints, 
		OffsetScaleFactors, 
		TransformProps->Width/2,
		bPathIsClosed, 
		bRampMode, 
		TransformProps->bSinglePolyGroup, 
		// Treat radius 0 corners as not rounded, rather than placing a bunch of vertices
		// in the same place.
		TransformProps->bRoundedCorners && TransformProps->CornerRadius > 0,
		TransformProps->CornerRadius,
		bLimitCornerRadius,
		TransformProps->RadialSlices);

	FMeshNormals::QuickRecomputeOverlayNormals(Mesh);

	FMeshBoundaryLoops Loops(&Mesh, true);
	if (Loops.Loops.Num() > 0)
	{
		Loops.Loops[0].GetVertices<FVector3d>(CurPolyLoop);
		for (FVector3d& Pt : CurPolyLoop)
		{
			Pt += MeshCenter;
		}
		if (Loops.Loops.Num() > 1)
		{
			Loops.Loops[1].GetVertices<FVector3d>(SecondPolyLoop);
			for (FVector3d& Pt : SecondPolyLoop)
			{
				Pt += MeshCenter;
			}
		}
	}

	return MeshCenter;
}

void UDrawPolyPathTool::BeginSettingHeight()
{
	State = EState::SettingHeight;

	if (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Interactive
		|| TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive)
	{
		BeginInteractiveExtrudeHeight();
	}
	else
	{
		BeginConstantExtrudeHeight();
	}
}

void UDrawPolyPathTool::BeginInteractiveExtrudeHeight()
{
	bHasSavedExtrudeHeight = true;
	SavedExtrudeHeight = TransformProps->ExtrudeHeight;

	// begin extrude
	ExtrudeHeightMechanic = NewObject<UPlaneDistanceFromHitMechanic>(this);
	ExtrudeHeightMechanic->Setup(this);

	ExtrudeHeightMechanic->WorldHitQueryFunc = [this](const FRay& WorldRay, FHitResult& HitResult)
	{
		return ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, HitResult, WorldRay);
	};
	ExtrudeHeightMechanic->WorldPointSnapFunc = [this](const FVector3d& WorldPos, FVector3d& SnapPos)
	{
		return !bIgnoreSnappingToggle && ToolSceneQueriesUtil::FindWorldGridSnapPoint(this, WorldPos, SnapPos);
	};
	ExtrudeHeightMechanic->CurrentHeight = 1.0f;  // initialize to something non-zero...prob should be based on polygon bounds maybe?

	FDynamicMesh3 PathMesh;
	FVector3d MeshCenter = GeneratePathMesh(PathMesh);
	EditPreview->SetTransform(FTransform(MeshCenter));
	EditPreview->InitializeExtrudeType(MoveTemp(PathMesh), DrawPlaneWorld.Z(), nullptr, false);

	FDynamicMesh3 TmpMesh;
	EditPreview->MakeExtrudeTypeHitTargetMesh(TmpMesh, false);

	FFrame3d UseFrame = DrawPlaneWorld; 
	UseFrame.Origin = MeshCenter;
	FTransform3d MeshToFrame(FQuat(UseFrame.Rotation.Inverse()));
	ExtrudeHeightMechanic->Initialize(MoveTemp(TmpMesh), UseFrame, MeshToFrame);

	ShowExtrudeMessage();
}

void UDrawPolyPathTool::BeginConstantExtrudeHeight()
{
	State = EState::SettingHeight;

	if (bHasSavedExtrudeHeight)
	{
		TransformProps->ExtrudeHeight = SavedExtrudeHeight;
		bHasSavedExtrudeHeight = false;
	}

	if (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Flat)
	{
		CurHeight = 0.0;
	}
	else if (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::Fixed || TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed)
	{
		CurHeight = TransformProps->ExtrudeHeight;
	}
	else
	{
		ensure(false);
	}

	FDynamicMesh3 PathMesh;
	FVector3d Center = GeneratePathMesh(PathMesh);
	EditPreview->SetTransform(FTransform(Center));
	EditPreview->InitializeExtrudeType(MoveTemp(PathMesh), DrawPlaneWorld.Z(), nullptr, false);
	UpdateExtrudePreview();

	ExtrudeHeightMechanic = nullptr;

	GetToolManager()->DisplayMessage(
		LOCTEXT("FixedSetHeightInstructions", "Click in viewport to accept fixed path height, or change it in details panel."),
		EToolMessageLevel::UserNotification);
}

// This should only be used after doing EditPreview->InitializeExtrudeType, when setting height
void UDrawPolyPathTool::UpdateExtrudePreview()
{
	EditPreview->UpdateExtrudeType([&](FDynamicMesh3& Mesh) { GenerateExtrudeMesh(Mesh); }, true);
}


void UDrawPolyPathTool::InitializePreviewMesh()
{
	if (EditPreview == nullptr)
	{
		EditPreview = NewObject<UPolyEditPreviewMesh>(this);
		EditPreview->CreateInWorld(TargetWorld, FTransform::Identity);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(EditPreview, nullptr);
		if ( MaterialProperties->Material == nullptr )
		{
			EditPreview->SetMaterial(
				ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.8f, 0.75f, 0.0f), GetToolManager()));
		}
		else
		{
			EditPreview->SetMaterial(MaterialProperties->Material.Get());
		}
	}
}

void UDrawPolyPathTool::ClearPreview()
{
	if (EditPreview != nullptr)
	{
		EditPreview->Disconnect();
		EditPreview = nullptr;
	}

	CurPolyLoop.Reset();
	SecondPolyLoop.Reset();
}



void UDrawPolyPathTool::GenerateExtrudeMesh(FDynamicMesh3& PathMesh)
{
	if (CurHeight == 0)
	{
		// This behavior should match whatever we do in DrawPolygonTool. Currently a flat mesh should
		// just be a one-sided strip, not a degenerately thin closed shape.
		return;
	}

	FExtrudeMesh Extruder(&PathMesh);
	const FVector3d ExtrudeDir = DrawPlaneWorld.Z();

	const bool bRampMode = (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed) || (TransformProps->ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive);

	if (bRampMode)
	{
		const double RampStartRatio = TransformProps->RampStartRatio;
		const double StartHeight = FMathd::Max(0.1, RampStartRatio * FMathd::Abs(CurHeight)) * FMathd::Sign(CurHeight);
		const double EndHeight = CurHeight;
		Extruder.ExtrudedPositionFunc = [&PathMesh, StartHeight, EndHeight, &ExtrudeDir](const FVector3d& P, const FVector3f& N, int32 VID) {
			FVector2f UV = PathMesh.GetVertexUV(VID);
			double UseHeight = FMathd::Lerp(StartHeight, EndHeight, UV.X);
			return P + UseHeight * ExtrudeDir;
		};
	}
	else
	{
		Extruder.ExtrudedPositionFunc = [this, &ExtrudeDir](const FVector3d& P, const FVector3f& N, int32 VID) {
			return P + CurHeight * ExtrudeDir;
		};
	}

	const FAxisAlignedBox3d Bounds = PathMesh.GetBounds();
	Extruder.UVScaleFactor = 1.0 / Bounds.MaxDim();
	Extruder.IsPositiveOffset = (CurHeight >= 0);
	Extruder.Apply();

	FMeshNormals::QuickRecomputeOverlayNormals(PathMesh);
}


void UDrawPolyPathTool::EmitNewObject()
{
	FDynamicMesh3 PathMesh;
	FVector3d MeshCenter = GeneratePathMesh(PathMesh);
	GenerateExtrudeMesh(PathMesh);
	PathMesh.DiscardVertexUVs();  // throw away arc lengths

	FFrame3d MeshTransform = DrawPlaneWorld; // The desired frame for the final output mesh
	FVector3d WorldCenter = PathMesh.GetBounds().Center() + MeshCenter;
	MeshTransform.Origin = MeshTransform.ToPlane(WorldCenter, 2);
	
	// Transform the mesh from its MeshCenter-offset space to the MeshTransform frame space
	// The below is equivalent to applying: MeshTransform.Rotation^-1 (Mesh + MeshCenter - MeshTransform.Origin)
	FFrame3d LocalCenterMeshTransform = MeshTransform;
	LocalCenterMeshTransform.Origin -= MeshCenter;
	MeshTransforms::WorldToFrameCoords(PathMesh, LocalCenterMeshTransform);
	UE::Geometry::FMeshTangentsf::ComputeDefaultOverlayTangents(PathMesh);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("CreatePolyPathTransactionName", "Create PolyPath"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = MeshTransform.ToFTransform();
	NewMeshObjectParams.BaseName = TEXT("Path");
	NewMeshObjectParams.Materials.Add(MaterialProperties->Material.Get());
	NewMeshObjectParams.SetMesh(&PathMesh);
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();

	if (bHasSavedWidth)
	{		
		TransformProps->Width = SavedWidth;
		bHasSavedWidth = false;
	}
	if (bHasSavedRadius)
	{
		TransformProps->CornerRadius = SavedRadius;
		bHasSavedRadius = false;
	}
	if (bHasSavedExtrudeHeight)
	{
		TransformProps->ExtrudeHeight = SavedExtrudeHeight;
		bHasSavedExtrudeHeight = false;
	}

	CurPolyLoop.Reset();
	SecondPolyLoop.Reset();
}



void UDrawPolyPathTool::ShowStartupMessage()
{
	GetToolManager()->DisplayMessage(
		LOCTEXT("StartDrawInstructions", "Draw a path on the drawing plane, set its width, and extrude it. Left-click to place path vertices, and click on the last or first vertex to complete the path. Hold Shift to ignore snapping while drawing."),
		EToolMessageLevel::UserNotification);
}


void UDrawPolyPathTool::ShowExtrudeMessage()
{
	GetToolManager()->DisplayMessage(
		LOCTEXT("InteractiveSetHeightInstructions", "Set the height of the extrusion by positioning the mouse over the extrusion volume, or over objects to snap to their heights. Hold Shift to ignore snapping."),
		EToolMessageLevel::UserNotification);
}



void UDrawPolyPathTool::UndoCurrentOperation(EState DestinationState)
{
	switch (State)
	{
	case EState::DrawingPath:
		if (ensure(DestinationState == EState::DrawingPath))
		{
			SurfacePathMechanic->PopLastPoint();
			if (SurfacePathMechanic->HitPath.Num() == 0)
			{
				CurrentCurveTimestamp++;
			}
		}
		break;
	case EState::SettingWidth:
		if (ensure(DestinationState == EState::DrawingPath))
		{
			CurveDistMechanic = nullptr;
			ClearPreview();
			InitializeNewSurfacePath();
			SurfacePathMechanic->HitPath = CurPathPoints;
		}
		break;
	case EState::SettingRadius:
		if (ensure(DestinationState == EState::SettingWidth))
		{
			BeginSettingWidth();
		}
		break;
	case EState::SettingHeight:
		if (DestinationState == EState::SettingRadius)
		{
			TransformProps->bRoundedCorners = true;
			BeginSettingRadius();
		}
		else if (ensure(DestinationState == EState::SettingWidth))
		{
			BeginSettingWidth();
		}
		break;
	}
}


void FDrawPolyPathStateChange::Revert(UObject* Object)
{
	Cast<UDrawPolyPathTool>(Object)->UndoCurrentOperation(PreviousState);
	bHaveDoneUndo = true;
}
bool FDrawPolyPathStateChange::HasExpired(UObject* Object) const
{
	return bHaveDoneUndo || (Cast<UDrawPolyPathTool>(Object)->CheckInCurve(CurveTimestamp) == false);
}
FString FDrawPolyPathStateChange::ToString() const
{
	return TEXT("FDrawPolyPathStateChange");
}




#undef LOCTEXT_NAMESPACE

