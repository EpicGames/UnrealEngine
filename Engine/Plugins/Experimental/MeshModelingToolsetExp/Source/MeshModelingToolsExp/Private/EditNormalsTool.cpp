// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditNormalsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolTargetManager.h"

#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Polygroups/PolygroupUtil.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "MeshDescriptionToDynamicMesh.h"

#include "AssetUtils/MeshDescriptionUtil.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"

#include "GroupTopology.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selections/GeometrySelectionUtil.h"

#include "Drawing/PreviewGeometryActor.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"

#include "Selection/GeometrySelectionVisualization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditNormalsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UEditNormalsTool"

/*
 * ToolBuilder
 */


UMultiSelectionMeshEditingTool* UEditNormalsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UEditNormalsTool* EditNormalsTool = NewObject<UEditNormalsTool>(SceneState.ToolManager);
	return EditNormalsTool;
}

void UEditNormalsToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	const TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);
	if (UEditNormalsTool* EditNormalsTool = Cast<UEditNormalsTool>(NewTool))
	{
		if (Targets.Num() == 1) // Can only have a selection when there is one target
		{
			UE::Geometry::FGeometrySelection Selection;
			if (UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Targets[0], Selection))
			{
				EditNormalsTool->SetGeometrySelection(MoveTemp(Selection));
			}
		}
	}
}

bool UEditNormalsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMultiSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}



/*
 * Tool
 */

UEditNormalsToolProperties::UEditNormalsToolProperties()
{
	bFixInconsistentNormals = false;
	bInvertNormals = false;
	bRecomputeNormals = true;
	NormalCalculationMethod = ENormalCalculationMethod::AreaAngleWeighting;
	SplitNormalMethod = ESplitNormalMethod::UseExistingTopology;
	SharpEdgeAngleThreshold = 60;
	bAllowSharpVertices = false;
}


UEditNormalsTool::UEditNormalsTool()
{
}


void UEditNormalsTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponent
	for (int32 ComponentIdx = 0, NumTargets = Targets.Num(); ComponentIdx < NumTargets; ComponentIdx++)
	{
		UE::ToolTarget::HideSourceObject(Targets[ComponentIdx]);
	}

	BasicProperties = NewObject<UEditNormalsToolProperties>(this, TEXT("Mesh Normals Settings"));
	BasicProperties->RestoreProperties(this);
	BasicProperties->bToolHasSelection = !InputGeometrySelection.IsEmpty();
	AddToolPropertySource(BasicProperties);

	// initialize the PreviewMesh+BackgroundCompute object
	UpdateNumPreviews();

	// if editing normals on a single object, user can select from available polygroup layers in from-polygroup mode
	if (Previews.Num() == 1)
	{
		PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
		PolygroupLayerProperties->RestoreProperties(this, TEXT("EditNormalsTool"));
		PolygroupLayerProperties->InitializeGroupLayers(OriginalDynamicMeshes[0].Get());
		PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
		AddToolPropertySource(PolygroupLayerProperties);
		UpdateActiveGroupLayer();

		BasicProperties->WatchProperty(BasicProperties->SplitNormalMethod, [&](ESplitNormalMethod NewMethod)
		{
			SetToolPropertySourceEnabled(PolygroupLayerProperties, NewMethod == ESplitNormalMethod::FaceGroupID);
		});
		SetToolPropertySourceEnabled(PolygroupLayerProperties, BasicProperties->SplitNormalMethod == ESplitNormalMethod::FaceGroupID);

		if (InputGeometrySelection.IsEmpty() == false)
		{
			GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
			GeometrySelectionVizProperties->RestoreProperties(this);
			AddToolPropertySource(GeometrySelectionVizProperties);
			GeometrySelectionVizProperties->Initialize(this);
			GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(InputGeometrySelection.ElementType);
			GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(InputGeometrySelection.TopologyType);
			GeometrySelectionVizProperties->bEnableShowEdgeSelectionVertices = true;
			// TODO Enable this but note we need to compute a ROI which only includes triangles incident to the
			//      polygroup feature eg do not include all triangles in the groups incident to a polygroup edge
			//GeometrySelectionVizProperties->bEnableShowTriangleROIBorder = true;

			// Setup input geometry selection visualization
			FTransform ApplyTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);

			// Compute group topology if the selection has Polygroup topology, and do nothing otherwise
			// Currently it is only possible to make a polygroup geometry selection using polygroup set stored directly in the mesh
			FGroupTopology GroupTopology(OriginalDynamicMeshes[0].Get(), InputGeometrySelection.TopologyType == EGeometryTopologyType::Polygroup);

			// Compute the overlay selection and a proxy triangle vertex selection used to make edge selections behave like
			// vertex selections. TODO if we added Triangle/Vertex ROI visualization (or overlay element visualization) it
			// would be clearer to users how those selection types affect different overlay elements and maybe we can remove
			// this conversion, see also :EdgeSelectionsBehaveLikeVertexSelections
			if (InputGeometrySelection.TopologyType == EGeometryTopologyType::Polygroup)
			{
				ConvertPolygroupSelectionToIncidentOverlaySelection(
					*OriginalDynamicMeshes[0],
					GroupTopology,
					InputGeometrySelection,
					EditTriangles,
					EditVertices,
					&TriangleVertexGeometrySelection);
			}
			else
			{
				ConvertTriangleSelectionToOverlaySelection(
					*OriginalDynamicMeshes[0],
					InputGeometrySelection,
					EditTriangles,
					EditVertices,
					&TriangleVertexGeometrySelection);
			}

			GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
			GeometrySelectionViz->CreateInWorld(GetTargetWorld(), ApplyTransform);
			InitializeGeometrySelectionVisualization(
				GeometrySelectionViz,
				GeometrySelectionVizProperties,
				*OriginalDynamicMeshes[0],
				InputGeometrySelection,
				&GroupTopology,
				!TriangleVertexGeometrySelection.IsEmpty() ? &TriangleVertexGeometrySelection : nullptr);
		}
	}

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Edit Normals"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Configure or Recalculate Normals on a Mesh (disables autogenerated Normals)"),
		EToolMessageLevel::UserNotification);
}


void UEditNormalsTool::SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn)
{
	InputGeometrySelection = MoveTemp(SelectionIn);
}


void UEditNormalsTool::UpdateNumPreviews()
{
	int32 CurrentNumPreview = Previews.Num();
	int32 TargetNumPreview = Targets.Num();
	if (TargetNumPreview < CurrentNumPreview)
	{
		for (int32 PreviewIdx = CurrentNumPreview - 1; PreviewIdx >= TargetNumPreview; PreviewIdx--)
		{
			Previews[PreviewIdx]->Cancel();
		}
		Previews.SetNum(TargetNumPreview);
		OriginalDynamicMeshes.SetNum(TargetNumPreview);
	}
	else
	{
		OriginalDynamicMeshes.SetNum(TargetNumPreview);
		for (int32 PreviewIdx = CurrentNumPreview; PreviewIdx < TargetNumPreview; PreviewIdx++)
		{
			UEditNormalsOperatorFactory *OpFactory = NewObject<UEditNormalsOperatorFactory>();
			OpFactory->Tool = this;
			OpFactory->ComponentIndex = PreviewIdx;
			OriginalDynamicMeshes[PreviewIdx] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
			*OriginalDynamicMeshes[PreviewIdx] = UE::ToolTarget::GetDynamicMeshCopy(Targets[PreviewIdx]);

			UMeshOpPreviewWithBackgroundCompute* Preview = Previews.Add_GetRef(NewObject<UMeshOpPreviewWithBackgroundCompute>(OpFactory, "Preview"));
			Preview->Setup(GetTargetWorld(), OpFactory);
			ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Targets[PreviewIdx]);
			Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

			const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[PreviewIdx]);
			Preview->ConfigureMaterials(MaterialSet.Materials,
				ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
			);

			Preview->SetVisibility(true);
		}
	}
}


void UEditNormalsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	BasicProperties->SaveProperties(this);

	if (GeometrySelectionViz)
	{
		GeometrySelectionViz->Disconnect();
	}

	if (GeometrySelectionVizProperties)
	{
		GeometrySelectionVizProperties->SaveProperties(this);
	}

	if (PolygroupLayerProperties)
	{
		PolygroupLayerProperties->SaveProperties(this, TEXT("EditNormalsTool"));
	}

	// Restore (unhide) the source meshes
	for (int32 ComponentIdx = 0, NumTargets = Targets.Num(); ComponentIdx < NumTargets; ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	TArray<FDynamicMeshOpResult> Results;
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Results.Add(Preview->Shutdown());
	}
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Results);
	}
}

TUniquePtr<FDynamicMeshOperator> UEditNormalsOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FEditNormalsOp> NormalsOp = MakeUnique<FEditNormalsOp>();
	NormalsOp->bFixInconsistentNormals = Tool->BasicProperties->bFixInconsistentNormals;
	NormalsOp->bInvertNormals = Tool->BasicProperties->bInvertNormals;
	NormalsOp->bRecomputeNormals = Tool->BasicProperties->bRecomputeNormals;
	NormalsOp->SplitNormalMethod = Tool->BasicProperties->SplitNormalMethod;
	NormalsOp->bAllowSharpVertices = Tool->BasicProperties->bAllowSharpVertices;
	NormalsOp->NormalCalculationMethod = Tool->BasicProperties->NormalCalculationMethod;
	NormalsOp->NormalSplitThreshold = Tool->BasicProperties->SharpEdgeAngleThreshold;
	NormalsOp->EditTriangles = Tool->EditTriangles;
	NormalsOp->EditVertices = Tool->EditVertices;

	const FTransform LocalToWorld = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Tool->Targets[ComponentIndex]);
	NormalsOp->OriginalMesh = Tool->OriginalDynamicMeshes[ComponentIndex];

	// use custom polygroup layer if available
	if (ComponentIndex == 0 && Tool->OriginalDynamicMeshes.Num() == 1 && Tool->ActiveGroupSet.IsValid())
	{
		NormalsOp->MeshPolygroups = Tool->ActiveGroupSet;
	}

	NormalsOp->SetTransform(LocalToWorld);

	return NormalsOp;
}



void UEditNormalsTool::OnTick(float DeltaTime)
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}

	if (GeometrySelectionViz)
	{
		UpdateGeometrySelectionVisualization(GeometrySelectionViz, GeometrySelectionVizProperties);
	}
}



#if WITH_EDITOR
void UEditNormalsTool::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateNumPreviews();
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}
#endif

void UEditNormalsTool::OnPropertyModified(UObject* ModifiedObject, FProperty* ModifiedProperty)
{
	Super::OnPropertyModified(ModifiedObject, ModifiedProperty);

	if (ModifiedObject == GeometrySelectionVizProperties)
	{
		return;
	}

	UpdateNumPreviews();
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}



bool UEditNormalsTool::HasAccept() const
{
	return true;
}

bool UEditNormalsTool::CanAccept() const
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}


void UEditNormalsTool::OnSelectedGroupLayerChanged()
{
	UpdateActiveGroupLayer();
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}


void UEditNormalsTool::UpdateActiveGroupLayer()
{
	// do not update if more than one mesh
	if (OriginalDynamicMeshes.Num() > 1) return;

	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(OriginalDynamicMeshes[0].Get());
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*OriginalDynamicMeshes[0], SelectedName);
		ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(OriginalDynamicMeshes[0].Get(), FoundAttrib);
	}
}



void UEditNormalsTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("EditNormalsToolTransactionName", "Edit Normals Tool"));

	check(Results.Num() == Targets.Num());
	
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		// disable auto-generated normals StaticMesh build setting
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]));
		if (StaticMeshComponent != nullptr)
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			if (ensure(StaticMesh != nullptr))
			{
				StaticMesh->Modify();
				UE::MeshDescription::FStaticMeshBuildSettingChange SettingsChange;
				SettingsChange.AutoGeneratedNormals = UE::MeshDescription::EBuildSettingBoolChange::Disable;
				UE::MeshDescription::ConfigureBuildSettings(StaticMesh, 0, SettingsChange);
			}
		}

		const FDynamicMesh3* NewDynamicMesh = Results[ComponentIdx].Mesh.Get();
		if (NewDynamicMesh)
		{
			// Note we could use BasicProperties->WillTopologyChange(); to be faster in the simple cases topology has not changed,
			// or use UE::ToolTarget::CommitDynamicMeshNormalsUpdate(Targets[ComponentIdx], NewDynamicMesh, false); in that case,
			// but doing so avoids recomputing tangents in some cases where it *should* be done, so for more consistent behavior
			// we always commit the full mesh update here.
			// See JIRA UE-225381 for more info on the inconsistent behavior case.
			UE::ToolTarget::CommitMeshDescriptionUpdateViaDynamicMesh(Targets[ComponentIdx], *NewDynamicMesh, true /*bHaveModifiedTopology*/);
		}
	}

	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE
