// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditPivotTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "MeshAdapterTransforms.h"
#include "MeshDescriptionAdapter.h"
#include "DynamicMesh/MeshTransforms.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "ToolSceneQueriesUtil.h"
#include "Physics/ComponentCollisionUtil.h"
#include "PhysicsEngine/BodySetup.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include "Components/PrimitiveComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/World.h"

#include "Selection/StoredMeshSelectionUtil.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditPivotTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UEditPivotTool"

/*
 * ToolBuilder
 */

UMultiSelectionMeshEditingTool* UEditPivotToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UEditPivotTool>(SceneState.ToolManager);
}

void UEditPivotToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	const TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);

	if (Targets.Num() == 1)
	{
		if (UEditPivotTool* NewPivotTool = Cast<UEditPivotTool>(NewTool))
		{
			// if there is an element selection, use its frame origin as an initial pivot
			FFrame3d Frame;
			FAxisAlignedBox3d Bounds;
			bool bIsElementSelection;
			if (GetCurrentSelectionWorldFrameBounds(SceneState, Frame, Bounds, bIsElementSelection))
			{
				if (bIsElementSelection)
				{
					FTransform LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
					LocalToWorld.SetTranslation(Frame.Origin);
					NewPivotTool->SetInitialPivot(LocalToWorld);
				}
			}
		}
	}
}

const FToolTargetTypeRequirements& UEditPivotToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


void UEditPivotToolActionPropertySet::PostAction(EEditPivotToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}




/*
 * Tool
 */

UEditPivotTool::UEditPivotTool()
{
}


void UEditPivotTool::Setup()
{
	UInteractiveTool::Setup();

	UClickDragInputBehavior* ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
	ClickDragBehavior->Initialize(this);
	AddInputBehavior(ClickDragBehavior);

	TransformProps = NewObject<UEditPivotToolProperties>();
	AddToolPropertySource(TransformProps);

	EditPivotActions = NewObject<UEditPivotToolActionPropertySet>(this);
	EditPivotActions->Initialize(this);
	AddToolPropertySource(EditPivotActions);

	ResetActiveGizmos();
	SetActiveGizmos_Single(false);
	UpdateSetPivotModes(true);

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(ActiveGizmos[0].TransformGizmo);

	Precompute();

	if (bHasCustomInitialPivot)
	{
		ActiveGizmos[0].TransformGizmo->SetNewGizmoTransform(InitialPivot);
	}

	FText AllTheWarnings = LOCTEXT("EditPivotWarning", "WARNING: This Tool will Modify the selected StaticMesh Assets! If you do not wish to modify the original Assets, please make copies in the Content Browser first!");

	// detect and warn about any meshes in selection that correspond to same source data
	bool bSharesSources = GetMapToSharedSourceData(MapToFirstOccurrences);
	if (bSharesSources)
	{
		AllTheWarnings = FText::Format(FTextFormat::FromString("{0}\n\n{1}"), AllTheWarnings, LOCTEXT("EditPivotSharedAssetsWarning", "WARNING: Multiple selected meshes share the same source Asset! Each Asset can only have one baked pivot, some results will be incorrect."));
	}

	bool bHasISMCs = false;
	for (int32 k = 0; k < Targets.Num(); ++k)
	{
		if (Cast<UInstancedStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[k])) != nullptr)
		{
			bHasISMCs = true;
		}
	}
	if (bHasISMCs)
	{
		AllTheWarnings = FText::Format(FTextFormat::FromString("{0}\n\n{1}"), AllTheWarnings, LOCTEXT("EditPivotISMCWarning", "WARNING: Some selected objects are Instanced Components. Pivot of Instances will be modified, instead of Asset."));
	}

	GetToolManager()->DisplayMessage(AllTheWarnings, EToolMessageLevel::UserWarning);

	SetToolDisplayName(LOCTEXT("ToolName", "Edit Pivot"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "This tool edits the Pivot (Origin) of the input assets. Hold Ctrl while using the gizmo to align to scene. Enable Snap Dragging and click+drag to place gizmo directly into clicked position."),
		EToolMessageLevel::UserNotification);
}



void UEditPivotTool::OnShutdown(EToolShutdownType ShutdownType)
{
	DragAlignmentMechanic->Shutdown();

	FFrame3d CurPivotFrame(ActiveGizmos[0].TransformProxy->GetTransform());

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UpdateAssets(CurPivotFrame);
	}
}


void VertexIteration(const FMeshDescription* Mesh, TFunctionRef<void(int32, const FVector&)> ApplyFunc)
{
	TArrayView<const FVector3f> VertexPositions = Mesh->GetVertexPositions().GetRawArray();

	for (const FVertexID VertexID : Mesh->Vertices().GetElementIDs())
	{
		const FVector Position = (FVector)VertexPositions[VertexID];
		ApplyFunc(VertexID.GetValue(), Position);
	}
}


void UEditPivotTool::Precompute()
{
	ObjectBounds = FAxisAlignedBox3d::Empty();
	WorldBounds = FAxisAlignedBox3d::Empty();

	int NumComponents = Targets.Num();
	if (NumComponents == 1)
	{
		Transform = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);

		const FMeshDescription* Mesh = UE::ToolTarget::GetMeshDescription(Targets[0]);
		VertexIteration(Mesh, [&](int32 VertexID, const FVector& Position) {
			ObjectBounds.Contain((FVector3d)Position);
			WorldBounds.Contain(Transform.TransformPosition((FVector3d)Position));
		});
	}
	else
	{
		Transform = FTransform3d::Identity;
		for (int k = 0; k < NumComponents; ++k)
		{
			FTransform3d CurTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[k]);
			const FMeshDescription* Mesh = UE::ToolTarget::GetMeshDescription(Targets[k]);
			VertexIteration(Mesh, [&](int32 VertexID, const FVector& Position) {
				ObjectBounds.Contain(CurTransform.TransformPosition((FVector3d)Position));
				WorldBounds.Contain(CurTransform.TransformPosition((FVector3d)Position));
			});
		}
	}
}



void UEditPivotTool::RequestAction(EEditPivotToolActions ActionType)
{
	if (PendingAction == EEditPivotToolActions::NoAction)
	{
		PendingAction = ActionType;
	}
}


void UEditPivotTool::OnTick(float DeltaTime)
{
	if (PendingAction != EEditPivotToolActions::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = EEditPivotToolActions::NoAction;
	}
}

void UEditPivotTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
}

void UEditPivotTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
}


void UEditPivotTool::UpdateSetPivotModes(bool bEnableSetPivot)
{
	for (FEditPivotTarget& Target : ActiveGizmos)
	{
		Target.TransformProxy->bSetPivotMode = bEnableSetPivot;
	}
}



void UEditPivotTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
}


void UEditPivotTool::ApplyAction(EEditPivotToolActions ActionType)
{
	switch (ActionType)
	{
		case EEditPivotToolActions::Center:
		case EEditPivotToolActions::Bottom:
		case EEditPivotToolActions::Top:
		case EEditPivotToolActions::Left:
		case EEditPivotToolActions::Right:
		case EEditPivotToolActions::Front:
		case EEditPivotToolActions::Back:
			SetPivotToBoxPoint(ActionType);
			break;

		case EEditPivotToolActions::WorldOrigin:
			SetPivotToWorldOrigin();
			break;
	}
}


void UEditPivotTool::SetPivotToBoxPoint(EEditPivotToolActions ActionPoint)
{
	FAxisAlignedBox3d UseBox = (EditPivotActions->bUseWorldBox) ? WorldBounds : ObjectBounds;
	FVector3d Point = UseBox.Center();

	if (ActionPoint == EEditPivotToolActions::Bottom || ActionPoint == EEditPivotToolActions::Top)
	{
		Point.Z = (ActionPoint == EEditPivotToolActions::Bottom) ? UseBox.Min.Z : UseBox.Max.Z;
	}
	else if (ActionPoint == EEditPivotToolActions::Left || ActionPoint == EEditPivotToolActions::Right)
	{
		Point.Y = (ActionPoint == EEditPivotToolActions::Left) ? UseBox.Min.Y : UseBox.Max.Y;
	}
	else if (ActionPoint == EEditPivotToolActions::Back || ActionPoint == EEditPivotToolActions::Front)
	{
		Point.X = (ActionPoint == EEditPivotToolActions::Front) ? UseBox.Min.X : UseBox.Max.X;
	}

	FTransform NewTransform;
	if (EditPivotActions->bUseWorldBox == false)
	{
		FFrame3d LocalFrame(Point);
		LocalFrame.Transform(Transform);
		NewTransform = LocalFrame.ToFTransform();
	}
	else
	{
		NewTransform = FTransform((FVector)Point);
	}

	ActiveGizmos[0].TransformGizmo->SetNewGizmoTransform(NewTransform);
}


void UEditPivotTool::SetPivotToWorldOrigin()
{
	ActiveGizmos[0].TransformGizmo->SetNewGizmoTransform(FTransform());
}


void UEditPivotTool::SetActiveGizmos_Single(bool bLocalRotations)
{
	check(ActiveGizmos.Num() == 0);

	FEditPivotTarget Transformable;
	Transformable.TransformProxy = NewObject<UTransformProxy>(this);
	Transformable.TransformProxy->bRotatePerObject = bLocalRotations;

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		Transformable.TransformProxy->AddComponent(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]));
	}
	Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager()->GetPairedGizmoManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this
	);
	Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy, GetToolManager());

	Transformable.TransformGizmo->bUseContextCoordinateSystem = false;
	Transformable.TransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	ActiveGizmos.Add(Transformable);
}


void UEditPivotTool::ResetActiveGizmos()
{
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	ActiveGizmos.Reset();
}



// does not make sense that CanBeginClickDragSequence() returns a RayHit? Needs to be an out-argument...
FInputRayHit UEditPivotTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if ((!TransformProps->bSnapDragPosition && !TransformProps->bSnapDragRotation) || ActiveGizmos.Num() == 0)
	{
		return FInputRayHit();
	}

	FHitResult Result;
	bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, PressPos.WorldRay);

	if (!bWorldHit)
	{
		return FInputRayHit();
	}
	return FInputRayHit(Result.Distance, Result.ImpactNormal);
}

void UEditPivotTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	FInputRayHit HitPos = CanBeginClickDragSequence(PressPos);
	check(HitPos.bHit);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("TransformToolTransformTxnName", "SnapDrag"));

	FEditPivotTarget& ActiveTarget = ActiveGizmos[0];
	USceneComponent* GizmoComponent = ActiveTarget.TransformGizmo->GetGizmoActor()->GetRootComponent();
	StartDragTransform = GizmoComponent->GetComponentToWorld();

	// Apply the drag logic as well so that the snap-drag position/orientation update is applied on first click
	OnClickDrag(PressPos);
}


void UEditPivotTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	bool bRotate = TransformProps->bSnapDragRotation;
	float NormalSign = (TransformProps->RotationMode == EEditPivotSnapDragRotationMode::AlignFlipped) ? -1.0f : 1.0f;

	FHitResult Result;
	bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, DragPos.WorldRay);
	if (bWorldHit == false)
	{
		return;
	}


	FVector HitPos = Result.ImpactPoint;
	FVector TargetNormal = (NormalSign) * Result.Normal;

	FQuaterniond AlignRotation = (bRotate) ?
		FQuaterniond(FVector3d::UnitZ(), (FVector3d)TargetNormal) : FQuaterniond(StartDragTransform.GetRotation());

	FTransform NewTransform = StartDragTransform;
	NewTransform.SetRotation((FQuat)AlignRotation);
	if (TransformProps->bSnapDragPosition)
	{
		NewTransform.SetTranslation(HitPos);
	}

	FEditPivotTarget& ActiveTarget = ActiveGizmos[0];
	ActiveTarget.TransformGizmo->SetNewGizmoTransform(NewTransform);
}


void UEditPivotTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnTerminateDragSequence();
}

void UEditPivotTool::OnTerminateDragSequence()
{
	FEditPivotTarget& ActiveTarget = ActiveGizmos[0];
	USceneComponent* GizmoComponent = ActiveTarget.TransformGizmo->GetGizmoActor()->GetRootComponent();
	FTransform EndDragtransform = GizmoComponent->GetComponentToWorld();

	TUniquePtr<FComponentWorldTransformChange> Change = MakeUnique<FComponentWorldTransformChange>(StartDragTransform, EndDragtransform);
	GetToolManager()->EmitObjectChange(GizmoComponent, MoveTemp(Change),
		LOCTEXT("TransformToolTransformTxnName", "SnapDrag"));

	GetToolManager()->EndUndoTransaction();
}




void UEditPivotTool::UpdateAssets(const FFrame3d& NewPivotWorldFrame)
{
	// Make sure mesh descriptions are deserialized before we open transaction.
	// This is to avoid potential stability issues related to creation/load of
	// mesh descriptions inside a transaction.
	// TODO: this may not be necessary anymore. Also may not be the most efficient
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		bool bTargetSupportsLODs = false;
		TArray<EMeshLODIdentifier> LODs = UE::ToolTarget::GetMeshDescriptionLODs(Targets[ComponentIdx], bTargetSupportsLODs, !TransformProps->bApplyToAllLODs);
		for (EMeshLODIdentifier LOD : LODs)
		{
			UE::ToolTarget::GetMeshDescription(Targets[ComponentIdx], FGetMeshParameters(bTargetSupportsLODs, LOD));
		}
	}

	// If the targets have any convex hulls, we need to update those first in a separate transaction to work around a crash
	// in the handling of convex DDC data
	constexpr bool bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction = false;
	bool bNeedSeparateTransactionForSimpleCollision = false;
	if constexpr (bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction)
	{
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			UToolTarget* Target = Targets[ComponentIdx];
			UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
			if (UE::Geometry::ComponentTypeSupportsCollision(Component))
			{
				if (UBodySetup* BodySetup = UE::Geometry::GetBodySetup(Component))
				{
					if (BodySetup->AggGeom.ConvexElems.Num() > 0)
					{
						bNeedSeparateTransactionForSimpleCollision = true;
						break;
					}
				}
			}
		}
	}

	// First pre-compute all the transforms we will need to apply
	TArray<FTransformSequence3d> BakeTransforms;
	BakeTransforms.SetNum(Targets.Num());
	TArray<FTransform> ComponentTransforms;
	FTransform NewWorldTransform = NewPivotWorldFrame.ToFTransform();
	FTransform NewWorldInverse = NewPivotWorldFrame.ToInverseFTransform();
	TArray<FTransform> OriginalTransforms;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		OriginalTransforms.Add((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]));
	}
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UToolTarget* Target = Targets[ComponentIdx];
		const UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);

		const UInstancedStaticMeshComponent* InstancedComponent = Cast<const UInstancedStaticMeshComponent>(Component);
		if (InstancedComponent != nullptr)
		{
			// no bake for instancedcomponents, nothing needs to be precomputed here
			ComponentTransforms.Add(FTransform::Identity);
		}
		else if (MapToFirstOccurrences[ComponentIdx] == ComponentIdx)
		{
			FTransformSRT3d ToBake(OriginalTransforms[ComponentIdx] * NewWorldInverse);
			// to preserve scale, after baking the first transform, we will also need to bake an inverse scale transform
			// this bake will need to be applied separately only in cases where FTransform3d cannot correctly represent the combination of the two transforms
			// TODO: we could skip the extra bake step if the bake functions could take a general matrix
			FTransformSRT3d SeparateBakeScale = FTransformSRT3d::Identity();
			bool bNeedSeparateScale = false;
			// ScaledNewWorldTransform is the pivot widget's transform with the scale of the original component transform
			FTransform ScaledNewWorldTransform = NewWorldTransform;
			// Note: this section only needed if we want to preserve the original mesh scale
			// Basically our goal is, given the original actor transform A = Ta Ra Sa, and gizmo transform G = Tg Rg:
			//   Try to keep the mesh in the same place w/ new actor transform, Tg Rg Sa (the gizmo transform w/ original actor scale)
			//   To keep the mesh unmoved, we then have:
			//    (New actor transform: Tg Rg Sa) * (Baked transform: Sa^-1 Rg^-1 Tg^-1 Ta Ra Sa) = (Original actor transform: Ta Ra Sa)
			//   This cannot in general be represented as a single transform, so it requires us to bake the Sa^-1 separately ...
			//   but in special cases where Rg^-1 Ra Sa == Sa Rg^-1 Ra, the baked scales cancel and we can instead bake a single transform with no scale
			{
				FVector3d Scale = (FVector3d)OriginalTransforms[ComponentIdx].GetScale3D();
				FQuaterniond Rotation = ToBake.GetRotation();
				bool AxisZero[3]{ FMath::IsNearlyZero(Rotation.X, DOUBLE_KINDA_SMALL_NUMBER),
								  FMath::IsNearlyZero(Rotation.Y, DOUBLE_KINDA_SMALL_NUMBER), 
								  FMath::IsNearlyZero(Rotation.Z, DOUBLE_KINDA_SMALL_NUMBER) };
				int Zeros = (int)AxisZero[0] + (int)AxisZero[1] + (int)AxisZero[2];
				bool bEqXY = FMath::IsNearlyEqual(Scale.X, Scale.Y);
				bool bEqYZ = FMath::IsNearlyEqual(Scale.Y, Scale.Z);
				bool bCanCombinedScales = 
					Zeros == 3 || // no rotation (quaternion x,y,z == axis scaled by sin(angle/2) == 0 when angle is 0)
					(bEqXY && bEqYZ) || // uniform scale
					(Zeros == 2 && (// rotation is around a major axis && 
						(!AxisZero[0] && bEqYZ) || // (scales on dimensions-moved-by-rotation are equal)
						(!AxisZero[1] && FMath::IsNearlyEqual(Scale.X, Scale.Z)) ||
						(!AxisZero[2] && bEqXY)
					));
				bNeedSeparateScale = !bCanCombinedScales;
				FVector3d InvScale = FTransformSRT3d::GetSafeScaleReciprocal(OriginalTransforms[ComponentIdx].GetScale3D());
				if (!bNeedSeparateScale)
				{
					ToBake.SetScale(FVector3d::One()); // clear scale; it will be fully captured by the new transform
					ToBake.SetTranslation(ToBake.GetTranslation() * InvScale);
					ScaledNewWorldTransform.SetScale3D(OriginalTransforms[ComponentIdx].GetScale3D());
				}
				else if (InvScale.X != 0 && InvScale.Y != 0 && InvScale.Z != 0) // Scale was inverted correctly
				{
					// non-uniform scale is incompatible with new pivot orientation; need to bake additional counter-scale into the mesh
					SeparateBakeScale.SetScale(InvScale);
					ScaledNewWorldTransform.SetScale3D(OriginalTransforms[ComponentIdx].GetScale3D());
				}
				// else do nothing -- scale is not invertible and must be baked
			}

			FTransformSequence3d& Sequence = BakeTransforms[ComponentIdx];
			Sequence.Append(ToBake);
			if (bNeedSeparateScale)
			{
				Sequence.Append(SeparateBakeScale);
			}
			ComponentTransforms.Add(ScaledNewWorldTransform);
		}
		else
		{
			// try to invert baked transform -- note that this may not be a correct inverse if there is rotation + non-uniform scale,
			// but it's the best we can do given we can not bake a separate custom scale when this is not the first occurrence
			int32 FirstOccurrence = MapToFirstOccurrences[ComponentIdx];
			FTransform ApproxBaked = OriginalTransforms[FirstOccurrence] * NewWorldInverse;
			FTransform ComponentTransform = ApproxBaked.Inverse() * OriginalTransforms[ComponentIdx];
			ComponentTransforms.Add(ComponentTransform);
			BakeTransforms[ComponentIdx] = BakeTransforms[FirstOccurrence];
		}
	}

	// Pre-computed transforms must be 1:1 with Targets
	check(ComponentTransforms.Num() == Targets.Num());
	check(BakeTransforms.Num() == Targets.Num());

	auto UpdateSimpleCollision = [](UPrimitiveComponent* Component, const FTransformSequence3d& ToBakeSeq)
	{
		// try to transform simple collision
		if (UE::Geometry::ComponentTypeSupportsCollision(Component))
		{
			for (const FTransformSRT3d& ToBake : ToBakeSeq.GetTransforms())
			{
				UE::Geometry::TransformSimpleCollision(Component, ToBake);
			}
		}
	};

	// If necessary, make a first transaction to bake simple collision updates separately (works around crash in undo/redo of transactions that update both hulls and static meshes together)
	if (bNeedSeparateTransactionForSimpleCollision)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("EditPivotToolSimpleCollisionTransactionName", "Edit Pivot Part 1 (Simple Collision)"));
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			UToolTarget* Target = Targets[ComponentIdx];
			UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
			Component->Modify();

			UpdateSimpleCollision(Component, BakeTransforms[ComponentIdx]);
		}
		GetToolManager()->EndUndoTransaction();
	}

	GetToolManager()->BeginUndoTransaction(bNeedSeparateTransactionForSimpleCollision ?
		LOCTEXT("EditPivotToolGeometryTransactionName", "Edit Pivot Part 2 (Visible Geometry)") :
		LOCTEXT("EditPivotToolTransactionName", "Edit Pivot"));

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UToolTarget* Target = Targets[ComponentIdx];
		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
		Component->Modify();

		UInstancedStaticMeshComponent* InstancedComponent = Cast<UInstancedStaticMeshComponent>(Component);
		if (InstancedComponent != nullptr)
		{
			// For ISMC, we will not bake in a mesh transform, instead we will update the instance transforms relative to the new pivot
			// TODO: this could be optional, and another alternative would be to bake the pivot to the mesh and then update
			//   all the instance transforms so they stay in the same position?

			// save world transforms
			int32 NumInstances = InstancedComponent->GetInstanceCount();
			TArray<FTransform> WorldTransforms;
			WorldTransforms.SetNum(NumInstances);
			for (int32 k = 0; k < NumInstances; ++k)
			{
				InstancedComponent->GetInstanceTransform(k, WorldTransforms[k], true);
			}

			// update position to new pivot
			InstancedComponent->SetWorldTransform(NewWorldTransform);

			// restore world transforms, which will compute new local transforms such that the instances do not move in the world
			for (int32 k = 0; k < NumInstances; ++k)
			{
				InstancedComponent->UpdateInstanceTransform(k, WorldTransforms[k], true, true, false);
			}
		}
		else if (MapToFirstOccurrences[ComponentIdx] == ComponentIdx)
		{
			bool bTargetSupportsLODs = false;
			TArray<EMeshLODIdentifier> LODs = UE::ToolTarget::GetMeshDescriptionLODs(Targets[ComponentIdx], bTargetSupportsLODs, !TransformProps->bApplyToAllLODs);
			for (EMeshLODIdentifier LOD : LODs)
			{
				FMeshDescription SourceMesh(UE::ToolTarget::GetMeshDescriptionCopy(Target, FGetMeshParameters(bTargetSupportsLODs, LOD)));
				FMeshDescriptionEditableTriangleMeshAdapter EditableMeshDescAdapter(&SourceMesh);

				for (const FTransformSRT3d& ToBake : BakeTransforms[ComponentIdx].GetTransforms())
				{
					MeshAdapterTransforms::ApplyTransform(EditableMeshDescAdapter, ToBake);
				}

				// todo: support vertex-only update
				UE::ToolTarget::CommitMeshDescriptionUpdate(Target, &SourceMesh, nullptr /*no material set changes*/, FCommitMeshParameters(bTargetSupportsLODs, LOD));
			}

			// transform simple collision geometry
			if (!bNeedSeparateTransactionForSimpleCollision)
			{
				UpdateSimpleCollision(Component, BakeTransforms[ComponentIdx]);
			}

			Component->SetWorldTransform(ComponentTransforms[ComponentIdx]);
		}
		else
		{
			Component->SetWorldTransform(ComponentTransforms[ComponentIdx]);
		}

		AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
		if (TargetActor)
		{
			TargetActor->MarkComponentsRenderStateDirty();
			TargetActor->UpdateComponentTransforms();
		}
	}

	// hack to ensure user sees the updated pivot immediately: request re-select of the original selection
	FSelectedObjectsChangeList NewSelection;
	NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
	for (int OrigMeshIdx = 0; OrigMeshIdx < Targets.Num(); OrigMeshIdx++)
	{
		AActor* OwnerActor = UE::ToolTarget::GetTargetActor(Targets[OrigMeshIdx]);
		if (OwnerActor)
		{
			NewSelection.Actors.Add(OwnerActor);
		}
	}
	GetToolManager()->RequestSelectionChange(NewSelection);

	GetToolManager()->EndUndoTransaction();
}





#undef LOCTEXT_NAMESPACE

