// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditUVIslandsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "SegmentTypes.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/World.h"
#include "Selections/MeshConnectedComponents.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"

#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditUVIslandsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UEditUVIslandsTool"

/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UEditUVIslandsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UEditUVIslandsTool* DeformTool = NewObject<UEditUVIslandsTool>(SceneState.ToolManager);
	DeformTool->SetWorld(SceneState.World);
	return DeformTool;
}

bool UEditUVIslandsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSurfacePointToolBuilder::CanBuildTool(SceneState) && 		
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
		[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) > 0;
}

/*
* Tool methods
*/

UEditUVIslandsTool::UEditUVIslandsTool()
{
}


void UEditUVIslandsTool::Setup()
{
	UMeshSurfacePointTool::Setup();

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	DynamicMeshComponent->SetupAttachment(PreviewMeshActor->GetRootComponent());
	DynamicMeshComponent->RegisterComponent();
	DynamicMeshComponent->SetWorldTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(DynamicMeshComponent, Target);
	WorldTransform = FTransform3d(DynamicMeshComponent->GetComponentTransform());

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	// enable secondary triangle buffers. Will default to existing material unless we set override.
	DynamicMeshComponent->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, &Topology, TriangleID);
	});

	// dynamic mesh configuration settings
	DynamicMeshComponent->SetTangentsType(EDynamicMeshComponentTangentsMode::AutoCalculated);
	DynamicMeshComponent->SetMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));
	FMeshNormals::QuickComputeVertexNormals(*DynamicMeshComponent->GetMesh());
	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshChanged.Add(
		FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UEditUVIslandsTool::OnDynamicMeshComponentChanged));

	// set up SelectionMechanic
	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	SelectionMechanic->Setup(this);
	SelectionMechanic->Properties->bSelectEdges = false;
	SelectionMechanic->Properties->bSelectVertices = false;
	SelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::None;
	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UEditUVIslandsTool::OnSelectionModifiedEvent);

	// initialize AABBTree
	MeshSpatial.SetMesh(DynamicMeshComponent->GetMesh());
	PrecomputeTopology();

	UVTranslateScale = 1.0 / DynamicMeshComponent->GetMesh()->GetBounds().MaxDim();

	// hide input StaticMeshComponent
	UE::ToolTarget::HideSourceObject(Target);

	// init state flags flags
	bInDrag = false;

	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAxisX | ETransformGizmoSubElements::TranslateAxisY
		| ETransformGizmoSubElements::TranslatePlaneXY | ETransformGizmoSubElements::RotateAxisZ
		| ETransformGizmoSubElements::ScaleAxisX | ETransformGizmoSubElements::ScaleAxisY
		| ETransformGizmoSubElements::ScalePlaneXY | ETransformGizmoSubElements::ScaleUniform,
		this);

	// Things are probably pretty broken if we're unable to get a transform gizmo... But we do this check in PolyEd,
	// so might as well be safe here too.
	if (ensure(TransformGizmo))
	{
		// Hook up callbacks
		TransformProxy = NewObject<UTransformProxy>(this);
		TransformProxy->OnBeginTransformEdit.AddUObject(this, &UEditUVIslandsTool::OnGizmoTransformBegin);
		TransformProxy->OnTransformChanged.AddUObject(this, &UEditUVIslandsTool::OnGizmoTransformUpdate);
		TransformProxy->OnEndTransformEdit.AddUObject(this, &UEditUVIslandsTool::OnGizmoTransformEnd);
		
		TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());
		TransformGizmo->bUseContextCoordinateSystem = false;
		TransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
		TransformGizmo->SetVisibility(false);
	}



	MaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	const FDynamicMesh3* TargetMesh = DynamicMeshComponent->GetMesh();
	TArray<FString> UVChannelNamesList;
	for (int32 k = 0; k < TargetMesh->Attributes()->NumUVLayers(); ++k)
	{
		UVChannelNamesList.Add(FString::Printf(TEXT("UV %d"), k));
	}
	MaterialSettings->UpdateUVChannels(0, UVChannelNamesList);
	MaterialSettings->RestoreProperties(this);
	AddToolPropertySource(MaterialSettings);
	MaterialSettings->GetOnModified().AddLambda([this](UObject*, FProperty*)
	{
		OnMaterialSettingsChanged();
	});
	OnMaterialSettingsChanged();

	SetToolDisplayName(LOCTEXT("ToolName", "UV Transform"));
	GetToolManager()->DisplayMessage(LOCTEXT("UEditUVIslandsToolStartupMessage", "Click on a UV Island to select it, and then use the Gizmo to translate/rotate/scale the UVs"), EToolMessageLevel::UserNotification);
}

void UEditUVIslandsTool::Shutdown(EToolShutdownType ShutdownType)
{
	MaterialSettings->SaveProperties(this);

	SelectionMechanic->Shutdown();

	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);

		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
			GetToolManager()->BeginUndoTransaction(LOCTEXT("EditUVIslandsToolTransactionName", "Edit UVs"));
			DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				FConversionToMeshDescriptionOptions ConversionOptions;
				ConversionOptions.bUpdateNormals = ConversionOptions.bUpdatePositions = false;
				ConversionOptions.bUpdateUVs = true;
				UE::ToolTarget::CommitDynamicMeshUpdate(Target, ReadMesh, true, ConversionOptions);
			});
			GetToolManager()->EndUndoTransaction();
		}

		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	TransformGizmo = nullptr;
	TransformProxy = nullptr;
}




void UEditUVIslandsTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	//ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
	//	TEXT("NextTransformType"),
	//	LOCTEXT("NextTransformType", "Next Transform Type"),
	//	LOCTEXT("NextTransformTypeTooltip", "Cycle to next transform type"),
	//	EModifierKey::None, EKeys::Q,
	//	[this]() { NextTransformTypeAction(); });
}


void UEditUVIslandsTool::OnMaterialSettingsChanged()
{
	MaterialSettings->UpdateMaterials();

	UMaterialInterface* OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	if (OverrideMaterial != nullptr)
	{
		DynamicMeshComponent->SetSecondaryRenderMaterial(OverrideMaterial);
	}
	else
	{
		DynamicMeshComponent->ClearSecondaryRenderMaterial();
	}
}



FDynamicMeshAABBTree3& UEditUVIslandsTool::GetSpatial()
{
	if (bSpatialDirty)
	{
		MeshSpatial.Build();
		bSpatialDirty = false;
	}
	return MeshSpatial;
}

void UEditUVIslandsTool::OnSelectionModifiedEvent()
{
	bSelectionStateDirty = true;
	if (!SelectionMechanic->GetActiveSelection().IsEmpty())
	{
		FFrame3d UseFrame = Topology.GetIslandFrame(
			SelectionMechanic->GetActiveSelection().GetASelectedGroupID(), GetSpatial());
		UseFrame.Transform(WorldTransform);

		if (TransformGizmo)
		{
			TransformGizmo->ReinitializeGizmoTransform(UseFrame.ToFTransform());
			TransformGizmo->SetDisplaySpaceTransform(UseFrame.ToFTransform());
		}
	}
}



bool UEditUVIslandsTool::HitTest(const FRay& WorldRay, FHitResult& OutHit)
{
	return SelectionMechanic->TopologyHitTest(WorldRay, OutHit);
}

FInputRayHit UEditUVIslandsTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	// disable this for now
	return FInputRayHit();
	//return UMeshSurfacePointTool::CanBeginClickDragSequence(PressPos);
}

void UEditUVIslandsTool::OnBeginDrag(const FRay& WorldRay)
{
}

void UEditUVIslandsTool::OnUpdateDrag(const FRay& Ray)
{
	check(false);
}

void UEditUVIslandsTool::OnEndDrag(const FRay& Ray)
{
	check(false);
}

void UEditUVIslandsTool::OnCancelDrag()
{
	check(false);
}

void UEditUVIslandsTool::OnGizmoTransformBegin(UTransformProxy*)
{
	bInDrag = true;
	SelectionMechanic->ClearHighlight();
	UpdateUVTransformFromSelection( SelectionMechanic->GetActiveSelection() );
	InitialGizmoFrame = FFrame3d(TransformProxy->GetTransform());
	InitialGizmoScale = TransformProxy->GetTransform().GetScale3D();
	BeginChange();
}

void UEditUVIslandsTool::OnGizmoTransformUpdate(UTransformProxy*, FTransform)
{
	if (bInDrag)
	{
		ComputeUpdate_Gizmo();
	}
}

void UEditUVIslandsTool::OnGizmoTransformEnd(UTransformProxy*)
{
	bInDrag = false;
	SelectionMechanic->NotifyMeshChanged(false);

	if (TransformGizmo)
	{
		TransformGizmo->SetNewChildScale(FVector::OneVector);
	}

	// close change record
	EndChange();
}



bool UEditUVIslandsTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (ActiveVertexChange == nullptr && !bInDrag)
	{
		SelectionMechanic->UpdateHighlight(DevicePos.WorldRay);
	}
	return true;
}


void UEditUVIslandsTool::OnEndHover()
{
	SelectionMechanic->ClearHighlight();
}




void UEditUVIslandsTool::UpdateUVTransformFromSelection(const FGroupTopologySelection& Selection)
{
	ActiveIslands.Reset();
	ActiveIslands.SetNum(Selection.SelectedGroupIDs.Num());
	int k = 0;
	for (int32 IslandID : Selection.SelectedGroupIDs)
	{
		FEditIsland& IslandInfo = ActiveIslands[k++];

		FGroupTopologySelection TempSelection;
		TempSelection.SelectedGroupIDs.Add(IslandID);
		IslandInfo.LocalFrame = Topology.GetIslandFrame(IslandID, GetSpatial());
		IslandInfo.Triangles = Topology.GetGroupTriangles(IslandID);

		TSet<int32> UVs;
		for (int32 tid : IslandInfo.Triangles)
		{
			if (Topology.UVOverlay->IsSetTriangle(tid))
			{
				FIndex3i Tri = Topology.UVOverlay->GetTriangle(tid);
				UVs.Add(Tri.A);
				UVs.Add(Tri.B);
				UVs.Add(Tri.C);
			}
		}

		IslandInfo.UVBounds = FAxisAlignedBox2d::Empty();
		for (int32 uvid : UVs)
		{
			IslandInfo.UVs.Add(uvid);
			FVector2f InitialUV = Topology.UVOverlay->GetElement(uvid);
			IslandInfo.InitialPositions.Add(InitialUV);
			IslandInfo.UVBounds.Contain(FVector2d(InitialUV));
		}
		IslandInfo.UVOrigin = IslandInfo.UVBounds.Center();
	}
}





void UEditUVIslandsTool::ComputeUpdate_Gizmo()
{
	if (SelectionMechanic->HasSelection() == false)
	{
		return;
	}

	FFrame3d CurFrame = FFrame3d(TransformProxy->GetTransform());
	FVector3d CurScale = TransformProxy->GetTransform().GetScale3D();
	FVector3d TranslationDelta = CurFrame.Origin - InitialGizmoFrame.Origin;
	FQuaterniond RotateDelta = CurFrame.Rotation - InitialGizmoFrame.Rotation;
	FVector3d CurScaleDelta = CurScale - InitialGizmoScale;
	double DeltaU = UVTranslateScale * InitialGizmoFrame.X().Dot(TranslationDelta);
	double DeltaV = UVTranslateScale * InitialGizmoFrame.Y().Dot(TranslationDelta);
	FVector2d UVTranslate(-DeltaU, -DeltaV);
	double RotateAngleDeg = VectorUtil::PlaneAngleSignedD(InitialGizmoFrame.X(), CurFrame.X(), InitialGizmoFrame.Z());
	FMatrix2d UVRotate = FMatrix2d::RotationDeg(-RotateAngleDeg);
	FVector2d UVScale(1.0/CurScale.X, 1.0/CurScale.Y);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->GetUVLayer(0);
	bool bHaveTransformation = (TranslationDelta.SquaredLength() > 0.0001 || RotateDelta.SquaredLength() > 0.0001 || CurScaleDelta.SquaredLength() > 0.0001);
	
	for (FEditIsland& Island : ActiveIslands)
	{
		int32 NumUVs = Island.UVs.Num();
		FVector2d OriginTranslate = Island.UVOrigin + UVTranslate;
		for ( int32 k = 0; k < NumUVs; ++k )
		{
			int32 uvid = Island.UVs[k];
			FVector2f InitialUV = Island.InitialPositions[k];
			if (bHaveTransformation)
			{
				FVector2d LocalUV = FVector2d(InitialUV) - Island.UVOrigin;
				FVector2d NewUV = (UVRotate * (UVScale * LocalUV)) + OriginTranslate;
				UVOverlay->SetElement(uvid, FVector2f(NewUV) );
			}
			else
			{
				UVOverlay->SetElement(uvid, InitialUV);
			}
		}
	}

	DynamicMeshComponent->FastNotifyUVsUpdated();
	GetToolManager()->PostInvalidation();
}



void UEditUVIslandsTool::OnTick(float DeltaTime)
{
	if (bSelectionStateDirty)
	{
		// update color highlights
		DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();

		if (TransformGizmo)
		{
			TransformGizmo->SetVisibility(SelectionMechanic->HasSelection());
		}

		bSelectionStateDirty = false;
	}
}




FUVGroupTopology::FUVGroupTopology(const FDynamicMesh3* Mesh, uint32 UVLayerIndex, bool bAutoBuild)
	: FGroupTopology(Mesh, false)
{
	if (Mesh->HasAttributes() && UVLayerIndex < (uint32)Mesh->Attributes()->NumUVLayers())
	{
		UVOverlay = Mesh->Attributes()->GetUVLayer(UVLayerIndex);

		if (bAutoBuild)
		{
			CalculateIslandGroups();
			RebuildTopology();
		}
	}
}


void FUVGroupTopology::CalculateIslandGroups()
{
	if (UVOverlay == nullptr)
	{
		return;
	}

	FMeshConnectedComponents UVComponents(Mesh);
	UVComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
		return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
	});

	int32 UVGroupCounter = 1;
	TriIslandGroups.SetNumUninitialized(Mesh->MaxTriangleID());
	for (const FMeshConnectedComponents::FComponent& Component : UVComponents)
	{
		for (int32 tid : Component.Indices)
		{
			TriIslandGroups[tid] = UVGroupCounter;
		}
		UVGroupCounter++;
	}
}


FFrame3d FUVGroupTopology::GetIslandFrame(int32 GroupID, FDynamicMeshAABBTree3& AABBTree)
{
	FFrame3d Frame = GetGroupFrame(GroupID);
	IMeshSpatial::FQueryOptions QueryOptions([&](int32 TriangleID) { return GetGroupID(TriangleID) == GroupID; });
	Frame.Origin = AABBTree.FindNearestPoint(Frame.Origin, QueryOptions);

	const TArray<int32>& Triangles = GetGroupTriangles(GroupID);

	// Accumulate gradients of UV.X over triangles and align frame X with that direction.
	// Probably should weight with a falloff from frame origin?
	FVector3d AccumX = FVector3d::Zero();
	for (int32 TriangleID : Triangles)
	{
		FVector3d A, B, C;
		Mesh->GetTriVertices(TriangleID, A, B, C);
		FVector2f fi, fj, fk;
		UVOverlay->GetTriElements(TriangleID, fi, fj, fk);

		FVector3d GradX = VectorUtil::TriGradient<double>(A, B, C, fi.X, fj.X, fk.X);
		AccumX += UE::Geometry::Normalized(GradX);
	}
	UE::Geometry::Normalize(AccumX);
	Frame.AlignAxis(0, AccumX);

	return Frame;
}


void UEditUVIslandsTool::PrecomputeTopology()
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	Topology = FUVGroupTopology(Mesh, 0, true);

	// update selection mechanic
	SelectionMechanic->Initialize(DynamicMeshComponent, &Topology,
		[this]() { return &GetSpatial(); }
		);
}




void UEditUVIslandsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	//DynamicMeshComponent->bExplicitShowWireframe = TransformProps->bShowWireframe;
	DynamicMeshComponent->bExplicitShowWireframe = false;

	SelectionMechanic->Render(RenderAPI);
}




//
// Change Tracking
//


void UEditUVIslandsTool::UpdateChangeFromROI(bool bFinal)
{
	if (ActiveVertexChange == nullptr)
	{
		return;
	}

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TArray<int32> ModifiedUVs;
	for (FEditIsland& Island : ActiveIslands)
	{
		ModifiedUVs.Append(Island.UVs);
	}
	ActiveVertexChange->SaveOverlayUVs(Mesh, ModifiedUVs, !bFinal);
}


void UEditUVIslandsTool::BeginChange()
{
	if (ActiveVertexChange == nullptr)
	{
		ActiveVertexChange = new FMeshVertexChangeBuilder(EMeshVertexChangeComponents::OverylayUVs);
		UpdateChangeFromROI(false);
	}
}


void UEditUVIslandsTool::EndChange()
{
	if (ActiveVertexChange != nullptr)
	{
		UpdateChangeFromROI(true);
		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(ActiveVertexChange->Change), LOCTEXT("UVEditChange", "UV Edit"));
	}

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}


void UEditUVIslandsTool::OnDynamicMeshComponentChanged()
{
	SelectionMechanic->NotifyMeshChanged(false);
}




#undef LOCTEXT_NAMESPACE

