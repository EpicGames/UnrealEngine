// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtractSplineTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "DynamicMeshEditor.h"
#include "Selection/SelectClickedAction.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "Drawing/MeshDebugDrawing.h"
#include "ModelingObjectsCreationAPI.h"

#include "Changes/ToolCommandChangeSequence.h"

#include "CuttingOps/PlaneCutOp.h"

#include "Containers/SparseSet.h"
#include "Misc/MessageDialog.h"

#include "ModelingToolTargetUtil.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "TransformTypes.h"
#include "Components/SplineComponent.h"
#include "Selection/ToolSelectionUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "VertexConnectedComponents.h"

#include "Selection/PolygonSelectionMechanic.h"


#include "CurveOps/GenerateCrossSectionOp.h"

#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExtractSplineTool)

#define LOCTEXT_NAMESPACE "UExtractSplineTool"

namespace ExtractSplineToolLocals
{
	const FString& CutlinePointSetID(TEXT("CutlinePointSet"));
	const FString& CutlineLineSetID(TEXT("CutlineLineSet"));

	USplineComponent* CreateNewSplineInActor(UInteractiveToolManager* ToolManager, AActor* Actor, bool bTransact = false, bool bSetAsRoot = false)
	{
		FCreateComponentParams Params;
		Params.HostActor = Actor;
		Params.BaseName = ""; //Use the autogenerated name instead
		Params.bSetAsRoot = bSetAsRoot;
		Params.bTransact = bTransact;
		Params.ComponentClass = USplineComponent::StaticClass();

		FCreateComponentResult Result = UE::Modeling::CreateNewComponentOnActor(ToolManager, MoveTemp(Params));

		if (Result.IsOK())
		{
			return StaticCast<USplineComponent*>(Result.NewComponent);
		}
		else
		{
			return nullptr;
		}	
	};

}

UExtractSplineTool* UExtractSplineToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UExtractSplineTool* NewTool = NewObject<UExtractSplineTool>(SceneState.ToolManager);
	return NewTool;
}

void UExtractSplineTool::Setup()
{
	UInteractiveTool::Setup();

	Settings = NewObject<UExtractSplineToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	// Convert input target to dynamic mesh
	OriginalMesh = MakeShared<FDynamicMesh3>();
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	OriginalMesh->EnableAttributes();

	Topology = MakeShared<FGroupTopology, ESPMode::ThreadSafe>(OriginalMesh.Get(), false);
	Topology->RebuildTopology();


	// set initial cut plane (also attaches gizmo/proxy)
	FBox CombinedBounds; CombinedBounds.Init();
	FVector ComponentOrigin, ComponentExtents;
	UE::ToolTarget::GetTargetActor(Target)->GetActorBounds(false, ComponentOrigin, ComponentExtents);
	CombinedBounds += FBox::BuildAABB(ComponentOrigin, ComponentExtents);
	

	CutPlaneWorld.Origin = (FVector3d)CombinedBounds.GetCenter();
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), CutPlaneWorld);
	PlaneMechanic->CanUpdatePlaneFunc = [this]() {return Settings->ExtractionMode == EExtractSplineMode::PlaneCut; };
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		CutPlaneWorld = PlaneMechanic->Plane;
		InvalidatePreviews();
		});
	PlaneMechanic->SetPlaneCtrlClickBehaviorTarget->InvisibleComponentsToHitTest.Add(UE::ToolTarget::GetTargetComponent(Target));


	// set up SelectionMechanic
	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false; // We'll do this ourselves later
	SelectionMechanic->Setup(this);

	SelectionMechanic->Properties->RestoreProperties(this);

	SelectionMechanic->Properties->bCanSelectFaces = true;
	SelectionMechanic->Properties->bCanSelectVertices = false;
	SelectionMechanic->Properties->bCanSelectEdges = true;
	SelectionMechanic->Properties->bDisplayPolygroupReliantControls = true;
	SelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::None;

	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UExtractSplineTool::InvalidatePreviews);
	SelectionMechanic->SetShouldAddToSelectionFunc([]() {return false; });
	SelectionMechanic->SetShouldRemoveFromSelectionFunc([]() {return false; });

	SelectionMechanic->Initialize(OriginalMesh.Get(),
		(FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target),
		GetTargetWorld(),
		Topology.Get(),
		[this]() { return &GetSpatial(); }
	);
	AddToolPropertySource(SelectionMechanic->Properties);


	SetupPreviews();
	InvalidatePreviews();
	RegeneratePreviewSplines();
	SetInteractionMode();
}

void UExtractSplineTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset();
	}

	if (PlaneMechanic)
	{
		PlaneMechanic->Shutdown();
	}

	if (SelectionMechanic)
	{
		SelectionMechanic->Properties->SaveProperties(this);
		RemoveToolPropertySource(SelectionMechanic->Properties);
		SelectionMechanic->Shutdown();
	}

	if (Preview)
	{
		Preview->Shutdown();
	}

	if (CutlineGeometry)
	{
		CutlineGeometry->Disconnect();
		CutlineGeometry = nullptr;
	}

	Super::Shutdown(ShutdownType);
}

void UExtractSplineTool::OnTick(float DeltaTime)
{
	SelectionMechanic->Tick(DeltaTime);
	PlaneMechanic->Tick(DeltaTime);
	Preview->Tick(DeltaTime);
}

bool UExtractSplineTool::CanAccept() const
{
	return CutLoops.Num() > 0 || CutSpans.Num() > 0;	
}

void UExtractSplineTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (PlaneMechanic && Settings->ExtractionMode == EExtractSplineMode::PlaneCut)
	{
		PlaneMechanic->Render(RenderAPI);
	}

	if (SelectionMechanic && Settings->ExtractionMode == EExtractSplineMode::PolygroupLoops)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}

void UExtractSplineTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	SetInteractionMode();
	InvalidatePreviews();
}

void UExtractSplineTool::SetInteractionMode()
{
	PlaneMechanic->bShowGrid = (Settings->ExtractionMode == EExtractSplineMode::PlaneCut);
	SelectionMechanic->SetIsEnabled(Settings->ExtractionMode == EExtractSplineMode::PolygroupLoops);
	SetToolPropertySourceEnabled(SelectionMechanic->Properties, Settings->ExtractionMode == EExtractSplineMode::PolygroupLoops);
}


void UExtractSplineTool::SetupPreviews()
{
	Factory = NewObject<UGenerateCrossSectionOpFactory>();

	GetCutPlane(Factory->LocalPlaneOrigin, Factory->LocalPlaneNormal);
	Factory->bSimplifyAlongNewEdges = true;
	Factory->OriginalMesh = OriginalMesh;
	Factory->TargetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(Factory, "Preview");
	Preview->Setup(GetTargetWorld(), Factory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	// set initial preview to un-processed mesh, so stuff doesn't just disappear if the first cut takes a while
	Preview->PreviewMesh->UpdatePreview(OriginalMesh.Get());
	Preview->PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));

	Preview->OnOpCompleted.AddWeakLambda(this,
		[this](const UE::Geometry::FDynamicMeshOperator* Op)
		{
			const UE::Geometry::FGenerateCrossSectionOp* GenerateCrossSectionOp = (const UE::Geometry::FGenerateCrossSectionOp*)(Op);
			CutLoops = GenerateCrossSectionOp->GetCutLoops();
			CutSpans = GenerateCrossSectionOp->GetCutSpans();

			RegeneratePreviewSplines();
		}
	);

	// Set up all the components we need to visualize things.
	CutlineGeometry = NewObject<UPreviewGeometry>();
	CutlineGeometry->CreateInWorld(GetTargetWorld(), (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));

	// These visualize the current spline edges that would be extracted
	CutlineGeometry->AddPointSet(ExtractSplineToolLocals::CutlinePointSetID);
	CutlineGeometry->AddLineSet(ExtractSplineToolLocals::CutlineLineSetID);

	UpdateVisibility();
}

void UExtractSplineTool::InvalidatePreviews()
{
	switch (Settings->ExtractionMode)
	{
	case EExtractSplineMode::PlaneCut:
		GetCutPlane(Factory->LocalPlaneOrigin, Factory->LocalPlaneNormal);
		Preview->InvalidateResult();
		break;
	case EExtractSplineMode::OpenBoundary:
		GatherSplineDataFromMeshBoundaries();
		RegeneratePreviewSplines();
		break;
	case EExtractSplineMode::PolygroupLoops:
		GatherSplineDataFromPolygroupSelection();
		RegeneratePreviewSplines();
		break;
	default:
		ensure(false);
	}
}

void UExtractSplineTool::GetCutPlane(FVector& Origin, FVector& Normal)
{
	FTransform LocalToWorld = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);

	// for all plane computation, change LocalToWorld to not have any zero scale dims
	FVector LocalToWorldScale = LocalToWorld.GetScale3D();
	for (int i = 0; i < 3; i++)
	{
		float DimScale = FMathf::Abs(LocalToWorldScale[i]);
		float Tolerance = KINDA_SMALL_NUMBER;
		if (DimScale < Tolerance)
		{
			LocalToWorldScale[i] = Tolerance * FMathf::SignNonZero(LocalToWorldScale[i]);
		}
	}
	LocalToWorld.SetScale3D(LocalToWorldScale);

	Origin = LocalToWorld.InverseTransformPosition((FVector)CutPlaneWorld.Origin);
	FVector3d WorldNormal = CutPlaneWorld.GetAxis(2);
	UE::Geometry::FTransformSRT3d L2WForNormal(LocalToWorld);
	Normal = (FVector)L2WForNormal.InverseTransformNormal(WorldNormal);

}

void UExtractSplineTool::UpdateVisibility()
{
	// We're not actually using the operation for its dynamic mesh output... so we're only going to display the original mesh
	UE::ToolTarget::SetSourceObjectVisible(Target, true);
	Preview->SetVisibility(false);
}

void UExtractSplineTool::GenerateAsset()
{
	using namespace ExtractSplineToolLocals;

	FVector3d Origin, Normal;
	GetCutPlane(Origin, Normal);

	auto CreateSpline = [this, &Normal](AActor* HostActor, int Index, const TArray<TArray<FVector3d>>& PointLists, bool bIsClosed)
	{
		FTransform LocalToWorldNoTranslation = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
		LocalToWorldNoTranslation.SetTranslation(FVector::Zero());


		FVector3d Center = FVector3d::Zero();
		for (const FVector3d& Vertex : PointLists[Index])
		{
			Center += LocalToWorldNoTranslation.TransformVector(Vertex);
		}
		Center /= PointLists[Index].Num();
		

		USplineComponent* OutputSpline = nullptr;
		OutputSpline = CreateNewSplineInActor(GetToolManager(), HostActor, true, false);

		OutputSpline->Modify();
		OutputSpline->SetRelativeTransform(FTransform(Center), false, nullptr, ETeleportType::ResetPhysics);
		OutputSpline->ClearSplinePoints();
		OutputSpline->SetClosedLoop(bIsClosed);
		OutputSpline->bSplineHasBeenEdited = true;

		for (int SplineIndex = 0; SplineIndex < PointLists[Index].Num(); ++SplineIndex)
		{
			FVector3d Vertex = LocalToWorldNoTranslation.TransformVector(PointLists[Index][SplineIndex]);			
			Vertex -= Center;
			OutputSpline->AddSplinePointAtIndex(Vertex, SplineIndex, ESplineCoordinateSpace::Local, false);
			OutputSpline->SetUpVectorAtSplinePoint(SplineIndex, Normal, ESplineCoordinateSpace::Local, false);
			OutputSpline->SetSplinePointType(SplineIndex, ESplinePointType::Linear, false);
		}

		OutputSpline->UpdateSpline();
	};

	auto CreateLoopSpline = [this, &CreateSpline](AActor* HostActor, int LoopIndex)
	{
		CreateSpline(HostActor, LoopIndex, CutLoops, true);
	};

	auto CreateSpanSpline = [this, &CreateSpline](AActor* HostActor, int SpanIndex)
	{
		CreateSpline(HostActor, SpanIndex, CutSpans, false);
	};


	GetToolManager()->BeginUndoTransaction(LOCTEXT("ExtractSplineTransactionName", "Extract Spline"));

	FCreateActorParams CreateActorParams;
	CreateActorParams.TargetWorld = TargetWorld.Get();
	CreateActorParams.BaseName = "SplineActor";
	CreateActorParams.Transform = FTransform(UE::ToolTarget::GetLocalToWorldTransform(Target).GetTranslation());
	CreateActorParams.TemplateAsset = nullptr; // With no template, we create a basic AActor instance
	FCreateActorResult Result = UE::Modeling::CreateNewActor(GetToolManager(), MoveTemp(CreateActorParams));
	AActor* NewActor = Result.NewActor;

	if (NewActor)
	{
		for (int LoopIndex = 0; LoopIndex < CutLoops.Num(); ++LoopIndex)
		{
			CreateLoopSpline(NewActor, LoopIndex);
		}

		for (int SpanIndex = 0; SpanIndex < CutSpans.Num(); ++SpanIndex)
		{
			CreateSpanSpline(NewActor, SpanIndex);
		}
	}

	GetToolManager()->EndUndoTransaction();
}

void UExtractSplineTool::RegeneratePreviewSplines()
{
	UPointSetComponent* PointSet = CutlineGeometry->FindPointSet(ExtractSplineToolLocals::CutlinePointSetID);
	ULineSetComponent* LineSet = CutlineGeometry->FindLineSet(ExtractSplineToolLocals::CutlineLineSetID);

	PointSet->Clear();
	LineSet->Clear();

	for (int LoopIndex = 0; LoopIndex < CutLoops.Num(); ++LoopIndex)
	{
		for (int SplineIndex = 0; SplineIndex < CutLoops[LoopIndex].Num(); ++SplineIndex)
		{
			PointSet->AddPoint(CutLoops[LoopIndex][SplineIndex], FColor::Green, 5.0f, 0.1f);
			LineSet->AddLine(CutLoops[LoopIndex][SplineIndex], CutLoops[LoopIndex][(SplineIndex + 1) % CutLoops[LoopIndex].Num()], FColor::Green, 2.0f, 1.0f);
		}
	}

	for (int SpanIndex = 0; SpanIndex < CutSpans.Num(); ++SpanIndex)
	{
		for (int SplineIndex = 0; SplineIndex < CutSpans[SpanIndex].Num()-1; ++SplineIndex)
		{
			PointSet->AddPoint(CutSpans[SpanIndex][SplineIndex], FColor::Blue, 5.0f, 0.1f);
			LineSet->AddLine(CutSpans[SpanIndex][SplineIndex], CutSpans[SpanIndex][(SplineIndex + 1)], FColor::Blue, 2.0f, 1.0f);
		}
	}
}

void UExtractSplineTool::GatherSplineDataFromMeshBoundaries()
{
	CutLoops.Empty();
	CutSpans.Empty();

	UE::Geometry::FMeshBoundaryLoops BoundaryLoops(OriginalMesh.Get());	
	BoundaryLoops.SpanBehavior = UE::Geometry::FMeshBoundaryLoops::ESpanBehaviors::Compute;
	BoundaryLoops.FailureBehavior = UE::Geometry::FMeshBoundaryLoops::EFailureBehaviors::ConvertToOpenSpan;
	BoundaryLoops.Compute();

	for (UE::Geometry::FEdgeLoop EdgeLoop: BoundaryLoops.Loops)
	{
		TArray<FVector3d> LoopVertices;
		EdgeLoop.GetVertices(LoopVertices);
		CutLoops.Add(LoopVertices);
	}

	for (UE::Geometry::FEdgeSpan EdgeSpan : BoundaryLoops.Spans)
	{		
		UE::Geometry::FPolyline3d Polyline;
		EdgeSpan.GetPolyline(Polyline);		
		CutSpans.Add(Polyline.GetVertices());
	}
}

UE::Geometry::FDynamicMeshAABBTree3& UExtractSplineTool::GetSpatial()
{
	if (!MeshSpatial)
	{
		MeshSpatial = MakeUnique<UE::Geometry::FDynamicMeshAABBTree3>(OriginalMesh.Get());
	}
	return *MeshSpatial;
}

void UExtractSplineTool::GatherSplineDataFromPolygroupSelection()
{

	// Basic algorithim:
	// Starting with an edge from the set, sequentially pull edges out of the set which connect to the last selected edge. 
	// Once an ordered list of boundary edges is generated, generate a list of spline points from these polygroup edges which connect into a continous polyline.
	auto ProcessSelectedEdges = [this](TSparseSet<int32>& ToProcessEdges)
	{
		if (ToProcessEdges.IsEmpty())
		{
			return;
		}

		// It's possible a set of edges here will produce more than one spline. This occurs when edges form disjoint connectivity sets within the set. 
		// We handle that by assuming each disjoint set will be a loop and once a loop is closed, moving onto a new starting edge until our processing set is finally empty.
		while (ToProcessEdges.Num() > 0)
		{
			TArray<int32> OrderedLoopEdges;
			int32 StartingEdge;
			ToProcessEdges.CompactStable();
			StartingEdge = ToProcessEdges[FSetElementId::FromInteger(0)];

			ensure(ToProcessEdges.Contains(StartingEdge));
			ToProcessEdges.Remove(StartingEdge);
			int32 NextEdge, PrevEdge = -1;
			PrevEdge = StartingEdge;
			do
			{
				OrderedLoopEdges.Add(PrevEdge);

				TArray<int32> NextEdgeCandidates;
				Topology->FindEdgeNbrEdges(PrevEdge, NextEdgeCandidates);
				NextEdgeCandidates.Remove(PrevEdge);
				NextEdge = -1;
				for (int32 Edge : NextEdgeCandidates)
				{
					if (ToProcessEdges.Contains(Edge))
					{
						NextEdge = Edge;
						ToProcessEdges.Remove(Edge);
						break;
					}
				}
				if (NextEdge == FDynamicMesh3::InvalidID)
				{
					break;
				}
				if (ToProcessEdges.Num() == 0)
				{
					OrderedLoopEdges.Add(NextEdge);
					break;
				}

				PrevEdge = NextEdge;
			} while (ToProcessEdges.Num() > 0);


			// Once we're done building an ordered list of Polygroup boundaries, next we need to convert them into spline points.
			FPolygonVertices SplineVertices;

			int TrailingMeshEdge = -1;
			int32 LastVertex = -1;
			for (int EdgeIndex = 0; EdgeIndex < OrderedLoopEdges.Num(); ++EdgeIndex)
			{
				TArray<int32> MeshVertices;
				MeshVertices = Topology->GetGroupEdgeVertices(OrderedLoopEdges[EdgeIndex]);
				if (EdgeIndex == 0)
				{
					LastVertex = MeshVertices.Last();
				}

				if (EdgeIndex > 0)
				{
					if (MeshVertices[0] == TrailingMeshEdge && MeshVertices.Last() != TrailingMeshEdge)
					{
						Algo::Reverse(MeshVertices);
					}
					ensure(MeshVertices.Last() == TrailingMeshEdge);
				}
				TrailingMeshEdge = MeshVertices[0];

				if (OrderedLoopEdges.Num() > 1)
				{
					MeshVertices.RemoveAt(0);
				}

				TArray<FVector3d> MeshPositions;
				for (int MeshVertex : MeshVertices)
				{
					MeshPositions.Add(OriginalMesh->GetVertexRef(MeshVertex));
				}
				MeshPositions.Append(SplineVertices);
				SplineVertices = MeshPositions;
			}

			if (OrderedLoopEdges.Num() == 1 || TrailingMeshEdge != LastVertex)
			{
				CutSpans.Add(SplineVertices);
			}
			else
			{
				CutLoops.Add(SplineVertices);
			}

		}
	};


	CutLoops.Empty();
	CutSpans.Empty();

	

	FGroupTopologySelection Selection = SelectionMechanic->GetActiveSelection();
	if (Selection.IsEmpty())
	{
		return;
	}

	// Use sparse set so we can remove multiple and compact once
	TSparseSet<int32> MultipleCountedEdges;
	TSparseSet<int32> ToProcessEdges;
	if (!Selection.SelectedGroupIDs.IsEmpty())
	{
		// Use this to develop equivalence relations between groups.
		UE::Geometry::FSizedDisjointSet DisjointSet;
		int32* MaxElement = Algo::MinElement(Selection.SelectedGroupIDs, TGreater<>());

		// Make sure we add 1 here, or the indices will be wrong later. The selection seems to be one-indexed for some reason?
		DisjointSet.Init((*MaxElement)+1, [&Selection](int32 ElementToTest) {
			return Selection.SelectedGroupIDs.Contains(ElementToTest);
		});

		// First We need to figure out which groups are connected to each other via shared edges. We'll use a FSizeDisjointSet to handle the grouping logic.
		for (int32 Group : Selection.SelectedGroupIDs)
		{
			TArray<int32> NbrGroups = Topology->GetGroupNbrGroups(Group);
			for (int32 NbrGroup : NbrGroups)
			{
				if (Selection.SelectedGroupIDs.Contains(NbrGroup))
				{
					DisjointSet.Union(Group, NbrGroup);
				}
			}
		}

		TArray<int32> CompactIdxToGroupID;
		DisjointSet.CompactedGroupIndexToGroupID(&CompactIdxToGroupID, nullptr);

		// Now we can, per connected component, add all group edges together into  processing set. 
		for (int32 ConnectedGroupID : CompactIdxToGroupID)
		{
			for (int32 Group : Selection.SelectedGroupIDs)
			{
				if (DisjointSet.Find(Group) != ConnectedGroupID)
				{
					continue;
				}

				Topology->ForGroupEdges(Group, [&ToProcessEdges, &MultipleCountedEdges](const UE::Geometry::FGroupTopology::FGroupEdge& GroupEdge, int GroupEdgeID)
					{
						if (ToProcessEdges.Contains(GroupEdgeID))
						{
							// We'll keep track of any multi-counted edges here, in case of weird non-manifold topology cases where an edge might have more than two neighbors.
							// TODO: Check to see what happens if selected groups don't have "simple" boundary loops of edges.
							MultipleCountedEdges.Add(GroupEdgeID);
						}
						ToProcessEdges.Add(GroupEdgeID);
					});
			}
			ToProcessEdges = ToProcessEdges.Difference(MultipleCountedEdges);

			// Convert all selected edges into spline ready loops, per the function defined above.
			ProcessSelectedEdges(ToProcessEdges);
		}
	}
	else
	{
		// Manual copy since SelectedEdgeIDs may be a different set type based on compile switches
		ToProcessEdges.Empty(Selection.SelectedEdgeIDs.Num());
		for (const int32 Id : Selection.SelectedEdgeIDs)
		{
			ToProcessEdges.Add(Id);
		}

		ProcessSelectedEdges(ToProcessEdges);
	}



}

#undef LOCTEXT_NAMESPACE