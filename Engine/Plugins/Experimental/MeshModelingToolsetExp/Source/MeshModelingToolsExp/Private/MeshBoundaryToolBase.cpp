// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBoundaryToolBase.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "InteractiveToolManager.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Selection/PolygonSelectionMechanic.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBoundaryToolBase)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshBoundaryToolBase"

void UMeshBoundaryToolBase::Setup()
{
	USingleSelectionTool::Setup();

	if (!Target)
	{
		return;
	}

	// create mesh to operate on
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

	// initialize hit query
	MeshSpatial.SetMesh(OriginalMesh.Get());

	// initialize topology
	Topology = MakeUnique<FBasicTopology>(OriginalMesh.Get(), false);
	bool bTopologyOK = Topology->RebuildTopology();

	// Set up selection mechanic to find and select edges
	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	SelectionMechanic->Setup(this);
	SelectionMechanic->Properties->bSelectEdges = true;
	SelectionMechanic->Properties->bSelectFaces = false;
	SelectionMechanic->Properties->bSelectVertices = false;
	SelectionMechanic->Initialize(OriginalMesh.Get(),
		(FTransform3d)Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform(),
		GetTargetWorld(),
		Topology.Get(),
		[this]() { return &MeshSpatial; }
	);
	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UMeshBoundaryToolBase::OnSelectionChanged);
}

void UMeshBoundaryToolBase::OnShutdown(EToolShutdownType ShutdownType)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->Shutdown();
	}
}

void UMeshBoundaryToolBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}

#undef LOCTEXT_NAMESPACE

