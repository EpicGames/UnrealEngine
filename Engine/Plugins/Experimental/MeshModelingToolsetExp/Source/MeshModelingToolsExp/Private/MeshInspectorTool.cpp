// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshInspectorTool.h"
#include "Engine/Engine.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"

#include "Components/DynamicMeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Properties/MeshStatisticsProperties.h"
#include "Properties/MeshAnalysisProperties.h"

#include "Drawing/LineSetComponent.h"
#include "ToolDataVisualizer.h"
#include "Polygroups/PolygroupUtil.h"
#include "Util/ColorConstants.h"

#include "PrimitiveDrawingUtils.h" // for FPrimitiveDrawInterface

#include "ToolSetupUtil.h"
#include "AssetUtils/MeshDescriptionUtil.h"
#include "ModelingToolTargetUtil.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"

#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"  // for GEngine->GetSmallFont()
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshInspectorTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshInspectorTool"





void UMeshInspectorMaterialProperties::UpdateUVChannels(int32 UVChannelIndex, const TArray<FString>& UVChannelNames, bool bUpdateSelection)
{
	UVChannelNamesList = UVChannelNames;
	if (bUpdateSelection)
	{
		UVChannel = 0 <= UVChannelIndex && UVChannelIndex < UVChannelNames.Num() ? UVChannelNames[UVChannelIndex] : TEXT("");
	}
}


const TArray<FString>& UMeshInspectorMaterialProperties::GetUVChannelNamesFunc() const
{
	return UVChannelNamesList;
}

void UMeshInspectorMaterialProperties::RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier)
{
	Super::RestoreProperties(RestoreToTool, CacheIdentifier);
	Setup();
}

void UMeshInspectorMaterialProperties::Setup()
{
	UMaterial* CheckerMaterialBase = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/CheckerMaterial"));
	if (CheckerMaterialBase != nullptr)
	{
		CheckerMaterial = UMaterialInstanceDynamic::Create(CheckerMaterialBase, this);
		if (CheckerMaterial != nullptr)
		{
			CheckerMaterial->SetScalarParameterValue("Density", CheckerDensity);
			CheckerMaterial->SetScalarParameterValue("UVChannel", static_cast<float>(UVChannelNamesList.IndexOfByKey(UVChannel)));
		}
	}
}

void UMeshInspectorMaterialProperties::UpdateMaterials()
{
	if (MaterialMode == EMeshInspectorMaterialMode::Checkerboard)
	{
		if (CheckerMaterial != nullptr)
		{
			CheckerMaterial->SetScalarParameterValue("Density", CheckerDensity);
			CheckerMaterial->SetScalarParameterValue("UVChannel", static_cast<float>(UVChannelNamesList.IndexOfByKey(UVChannel)));
		}
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::FlatShaded)
	{
		ActiveCustomMaterial = UMaterialInstanceDynamic::Create(ToolSetupUtil::GetDefaultSculptMaterial(nullptr), this);
		ActiveCustomMaterial->SetScalarParameterValue(TEXT("FlatShading"), 1.0f);
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::Grey)
	{
		ActiveCustomMaterial = UMaterialInstanceDynamic::Create(ToolSetupUtil::GetImageBasedSculptMaterial(nullptr, ToolSetupUtil::ImageMaterialType::DefaultBasic), this);
		ActiveCustomMaterial->SetScalarParameterValue(TEXT("FlatShading"), 0.0f);
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::Transparent)
	{
		ActiveCustomMaterial = ToolSetupUtil::GetTransparentSculptMaterial(nullptr, TransparentMaterialColor, Opacity, bTwoSided);
		ActiveCustomMaterial->SetVectorParameterValue(TEXT("Color"), Color);
		ActiveCustomMaterial->SetScalarParameterValue(TEXT("Opacity"), Opacity);
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::VertexColor)
	{
		ActiveCustomMaterial = ToolSetupUtil::GetVertexColorMaterial(nullptr);
		ActiveCustomMaterial->SetScalarParameterValue(TEXT("FlatShading"), (bFlatShading) ? 1.0f : 0.0f);
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::TangentNormal)
	{
		ActiveCustomMaterial = UMaterialInstanceDynamic::Create(ToolSetupUtil::GetImageBasedSculptMaterial(nullptr, ToolSetupUtil::ImageMaterialType::TangentNormalFromView), this);
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::GroupColor)
	{
		ActiveCustomMaterial = ToolSetupUtil::GetVertexColorMaterial(nullptr);
		ActiveCustomMaterial->SetScalarParameterValue(TEXT("FlatShading"), (bFlatShading) ? 1.0f : 0.0f);
	}
}


UMaterialInterface* UMeshInspectorMaterialProperties::GetActiveOverrideMaterial() const
{
	if (MaterialMode == EMeshInspectorMaterialMode::Checkerboard)
	{
		return CheckerMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::Override)
	{
		return OverrideMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::FlatShaded)
	{
		return ActiveCustomMaterial;
	} 
	else if (MaterialMode == EMeshInspectorMaterialMode::Grey)
	{
		return ActiveCustomMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::Transparent)
	{
		return ActiveCustomMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::VertexColor)
	{
		return ActiveCustomMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::TangentNormal)
	{
		return ActiveCustomMaterial;
	}
	else if (MaterialMode == EMeshInspectorMaterialMode::GroupColor)
	{
		return ActiveCustomMaterial;
	}

	return nullptr;
}


/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UMeshInspectorToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UDynamicMeshProvider::StaticClass(),
		USceneComponentBackedTarget::StaticClass(),
		UMaterialProvider::StaticClass()
		});
	return TypeRequirements;
}

USingleSelectionMeshEditingTool* UMeshInspectorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshInspectorTool>(SceneState.ToolManager);
}

/*
 * Tool
 */
UMeshInspectorTool::UMeshInspectorTool()
{
}

void UMeshInspectorTool::Setup()
{
	UInteractiveTool::Setup();

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);
	LocalToWorldTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	PreviewMesh->SetTransform(LocalToWorldTransform);

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);
	DefaultMaterial = PreviewMesh->GetMaterial(0);

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	FDynamicMesh3 InputMeshWithTangents = UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams);
	PreviewMesh->ReplaceMesh(MoveTemp(InputMeshWithTangents));

	DrawnLineSet = NewObject<ULineSetComponent>(PreviewMesh->GetRootComponent(), "MeshInspectorToolLineSet");

	DrawnLineSet->SetupAttachment(PreviewMesh->GetRootComponent());

	DrawnLineSet->RegisterComponent();

	Precompute();

	UE::ToolTarget::HideSourceObject(Target);

	// initialize our properties
	Settings = NewObject<UMeshInspectorProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this, TEXT("MeshGroupPaintTool"));
	PolygroupLayerProperties->InitializeGroupLayers(PreviewMesh->GetMesh());
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
	UpdateActiveGroupLayer();
	AddToolPropertySource(PolygroupLayerProperties);


	MaterialSettings = NewObject<UMeshInspectorMaterialProperties>(this);
	const FDynamicMesh3* TargetMesh = PreviewMesh->GetPreviewDynamicMesh();
	TArray<FString> UVChannelNamesList;
	for (int32 k = 0; k < TargetMesh->Attributes()->NumUVLayers(); ++k)
	{
		UVChannelNamesList.Add(FString::Printf(TEXT("UV %d"), k));
	}
	MaterialSettings->UpdateUVChannels(0, UVChannelNamesList);
	MaterialSettings->RestoreProperties(this);
	AddToolPropertySource(MaterialSettings);

	PreviewMesh->EnableWireframe(Settings->bWireframe);

	UMeshStatisticsProperties* Statistics = NewObject<UMeshStatisticsProperties>(this);
	Statistics->Update(*PreviewMesh->GetPreviewDynamicMesh());
	AddToolPropertySource(Statistics);

	UMeshAnalysisProperties* MeshAnalysis = NewObject<UMeshAnalysisProperties>(this);
	MeshAnalysis->Update(*PreviewMesh->GetPreviewDynamicMesh(), (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	AddToolPropertySource(MeshAnalysis);

	UpdateVisualization();

	SetToolDisplayName(LOCTEXT("ToolName", "Mesh Inspector"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Visualize various geometric properties and attributes of the Mesh"),
		EToolMessageLevel::UserNotification);
}


UE::Geometry::FDynamicMeshAABBTree3* UMeshInspectorTool::GetSpatial()
{
	if (MeshAABBTree.IsValid() == false)
	{
		MeshAABBTree = MakeUnique<FDynamicMeshAABBTree3>(PreviewMesh->GetMesh(), true);
	}
	return MeshAABBTree.Get();
}


void UMeshInspectorTool::OnShutdown(EToolShutdownType ShutdownType)
{
	UE::ToolTarget::ShowSourceObject(Target);

	if (PreviewMesh != nullptr)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	Settings->SaveProperties(this);
	MaterialSettings->SaveProperties(this);
}


void UMeshInspectorTool::Precompute()
{
	BoundaryEdges.Reset();
	BoundaryBowties.Reset();
	UVSeamEdges.Reset();
	UVBowties.Reset();
	NormalSeamEdges.Reset();
	TangentSeamEdges.Reset();
	GroupBoundaryEdges.Reset();
	MissingUVTriangleEdges.Reset();

	const FDynamicMesh3* TargetMesh = PreviewMesh->GetPreviewDynamicMesh();
	const FDynamicMeshUVOverlay* UVOverlay =
		TargetMesh->HasAttributes() ? TargetMesh->Attributes()->PrimaryUV() : nullptr;
	const FDynamicMeshNormalOverlay* NormalOverlay =
		TargetMesh->HasAttributes() ? TargetMesh->Attributes()->PrimaryNormals() : nullptr;
	const FDynamicMeshNormalOverlay* TangentXOverlay =
		(TargetMesh->HasAttributes() && TargetMesh->Attributes()->HasTangentSpace()) ? TargetMesh->Attributes()->PrimaryTangents() : nullptr;
	const FDynamicMeshNormalOverlay* TangentYOverlay =
		(TargetMesh->HasAttributes() && TargetMesh->Attributes()->HasTangentSpace()) ? TargetMesh->Attributes()->PrimaryBiTangents() : nullptr;

	for (int eid : TargetMesh->EdgeIndicesItr())
	{
		bool bIsBoundaryEdge = TargetMesh->IsBoundaryEdge(eid);
		if (bIsBoundaryEdge)
		{
			BoundaryEdges.Add(eid);
		}
		if (UVOverlay != nullptr && UVOverlay->IsSeamEdge(eid))
		{
			UVSeamEdges.Add(eid);
		}
		if (bIsBoundaryEdge == false && NormalOverlay != nullptr && NormalOverlay->IsSeamEdge(eid))
		{
			NormalSeamEdges.Add(eid);
		}
		if (bIsBoundaryEdge == false && TangentXOverlay != nullptr && (TangentXOverlay->IsSeamEdge(eid) || TangentYOverlay->IsSeamEdge(eid)) )
		{
			TangentSeamEdges.Add(eid);
		}
		if (TargetMesh->IsGroupBoundaryEdge(eid))
		{
			GroupBoundaryEdges.Add(eid);
		}
	}

	for (int vid : TargetMesh->VertexIndicesItr())
	{
		if (TargetMesh->IsBowtieVertex(vid))
		{
			BoundaryBowties.Add(vid);
		}

		if (UVOverlay != nullptr && UVOverlay->IsBowtieInOverlay(vid))
		{
			UVBowties.Add(vid);
		}
	}

	for (int tid : TargetMesh->TriangleIndicesItr())
	{
		if (UVOverlay != nullptr && !UVOverlay->IsSetTriangle(tid))
		{
			const FIndex3i Edges = TargetMesh->GetTriEdges(tid);
			MissingUVTriangleEdges.Add(Edges[0]);
			MissingUVTriangleEdges.Add(Edges[1]);
			MissingUVTriangleEdges.Add(Edges[2]);
		}

	}
}


void UMeshInspectorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	const FDynamicMesh3* TargetMesh = PreviewMesh->GetPreviewDynamicMesh();
	FTransform Transform = PreviewMesh->GetTransform();

	if (BoundaryBowties.Num() > 0 && Settings->bBowtieVertices)
	{
		FToolDataVisualizer BowtieRenderer;
		BowtieRenderer.PointColor = FColor(240, 15, 15);
		BowtieRenderer.PointSize = 25.0;
		BowtieRenderer.BeginFrame(RenderAPI);
		BowtieRenderer.SetTransform(Transform);
		for (int32 vid : BoundaryBowties)
		{
			BowtieRenderer.DrawPoint(TargetMesh->GetVertex(vid));
		}
		BowtieRenderer.EndFrame();
	}

	if (UVBowties.Num() > 0 && Settings->bUVBowties)
	{
		FToolDataVisualizer BowtieRenderer;
		BowtieRenderer.PointColor = FColor(15, 240, 15);
		BowtieRenderer.PointSize = 12.0;
		BowtieRenderer.BeginFrame(RenderAPI);
		BowtieRenderer.SetTransform(Transform);
		for (int32 vid : UVBowties)
		{
			BowtieRenderer.DrawPoint(TargetMesh->GetVertex(vid));
		}
		BowtieRenderer.EndFrame();
	}
}


void UMeshInspectorTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	float DPIScale = Canvas->GetDPIScale();
	UFont* UseFont = GEngine->GetSmallFont();
	FViewCameraState CamState = RenderAPI->GetCameraState();
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	FVector3d LocalEyePosition(LocalToWorldTransform.InverseTransformPosition((FVector3d)CamState.Position));
	FVector3d LocalEyeDirection(LocalToWorldTransform.InverseTransformVectorNoScale((FVector3d)CamState.Forward()));

	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
	FDynamicMeshAABBTree3* Spatial = GetSpatial();

	// Code below draws the different type of mesh indices. The maximum number of labels to draw is
	// capped. Points are clipped against the view frustum, so that only visible labels are drawn.
	// In addition raycasts are used to check that the label target point is visible.
	// This is all a bit expensive, it would be nice to do this in parallel and/or cache 
	// the result for the current view

	// config settings for drawing numbers
	int32 DrawMaxNumbers = 500;
	bool bCanShowAllValues = false;
	switch (Settings->ShowIndices)
	{
	case EMeshInspectorToolDrawIndexMode::TriangleID: bCanShowAllValues = (Mesh->TriangleCount() < DrawMaxNumbers); break;
	case EMeshInspectorToolDrawIndexMode::VertexID: bCanShowAllValues = (Mesh->VertexCount() < DrawMaxNumbers); break;
	case EMeshInspectorToolDrawIndexMode::EdgeID: bCanShowAllValues = (Mesh->EdgeCount() < DrawMaxNumbers); break;
	case EMeshInspectorToolDrawIndexMode::GroupID: bCanShowAllValues = true; break;
	}

	// figure out 2D pixel-space rectangle to use for clipping
	FIntRect ViewRect = SceneView->UnconstrainedViewRect;
	ViewRect.InflateRect( (bCanShowAllValues) ? 0 : -100 );
	FAxisAlignedBox2d ViewRectBox(
		FVector2d((double)ViewRect.Min.X, (double)ViewRect.Min.Y),
		FVector2d((double)ViewRect.Max.X, (double)ViewRect.Max.Y));

	// IsVisibleFunc checks if the camera projection of a 3D world-space point is considered "visible"
	auto IsVisibleFunc = [SceneView, ViewRectBox](const FVector3d WorldPosition)
	{
		FVector2D PixelLocation;
		if (SceneView->WorldToPixel(WorldPosition, PixelLocation) )
		{
			return ViewRectBox.Contains(PixelLocation);
		}
		return false;
	};

	int32 NumbersDrawnCount = 0;

	// DrawNumberIfVisible draws the given number at the given 3D position, projected into the 2D canvas, if it is visible
	// PositionHitTestFunc argument is called after raycast checks w/ the hit triangle, ray-distance, and local-space hit position, 
	// to determine if visible point along the eye ray is suitable to draw the number at (ie if it is visible, etc)
	auto DrawNumberIfVisible = [Mesh, Spatial, DPIScale, UseFont, this, LocalEyePosition, LocalEyeDirection, SceneView, Canvas, &NumbersDrawnCount, DrawMaxNumbers, &IsVisibleFunc]
	(int32 Number, FVector3d LocalPosition, TFunctionRef<bool(int32, double, FVector3d)> PositionHitTestFunc)
	{
		FVector3d WorldPosition = LocalToWorldTransform.TransformPosition(LocalPosition);
		if (NumbersDrawnCount >= DrawMaxNumbers || IsVisibleFunc(WorldPosition) == false )
		{
			return;
		}
		FRay3d LocalEyeRay;
		LocalEyeRay.Origin = LocalEyePosition;
		LocalEyeRay.Direction = Normalized(LocalPosition - LocalEyePosition);
		if (LocalEyeRay.Direction.Dot(LocalEyeDirection) < 0)
		{
			return;		// ray to Position is pointing away from the camera
		}
		double LocalRayHitT;
		int HitTID;
		FVector3d HitBaryCoords;
		if (Spatial->FindNearestHitTriangle(LocalEyeRay, LocalRayHitT, HitTID, HitBaryCoords))
		{
			FVector3d LocalHitPos = LocalEyeRay.PointAt(LocalRayHitT);
			if (PositionHitTestFunc(HitTID, LocalRayHitT, LocalHitPos))
			{
				FVector2D PixelPos;
				SceneView->WorldToPixel(WorldPosition, PixelPos);
				FString String = FString::Printf(TEXT("%d"), Number);
				Canvas->DrawShadowedString(PixelPos.X / (double)DPIScale, PixelPos.Y / (double)DPIScale, *String, UseFont, FLinearColor::White);
				NumbersDrawnCount++;
			}
		}
		else
		{
			// if we did not hit the mesh then this item must be at a grazing angle in which case it should be considered visible
			FVector2D PixelPos;
			SceneView->WorldToPixel(WorldPosition, PixelPos);
			FString String = FString::Printf(TEXT("%d"), Number);
			Canvas->DrawShadowedString(PixelPos.X / (double)DPIScale, PixelPos.Y / (double)DPIScale, *String, UseFont, FLinearColor::White);
			NumbersDrawnCount++;
		}
	};


	if (Settings->ShowIndices == EMeshInspectorToolDrawIndexMode::TriangleID)
	{
		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			DrawNumberIfVisible(tid, Mesh->GetTriCentroid(tid),
				[&](int32 HitTID, double, FVector3d) { return HitTID == tid; });
		}
	}
	else if (Settings->ShowIndices == EMeshInspectorToolDrawIndexMode::VertexID)
	{
		FRandomStream TargetJitter(31337);
		for (int32 vid : Mesh->VertexIndicesItr())
		{
			// slightly jitter the position here to avoid issues w/ rays that directly pass through vertices 'missing' due to precision limitations
			FVector3d Position = Mesh->GetVertex(vid) + 0.0001 * TargetJitter.GetUnitVector();
			DrawNumberIfVisible(vid, Position, [&](int32 HitTID, double LocalRayHitT, FVector3d LocalHitPos) { return Distance(LocalHitPos, Position) < 0.001; });
		}
	}
	else if (Settings->ShowIndices == EMeshInspectorToolDrawIndexMode::EdgeID)
	{
		FRandomStream TargetJitter(31337);
		for (int32 eid : Mesh->EdgeIndicesItr())
		{
			FVector3d Position = Mesh->GetEdgePoint(eid, 0.5) + 0.0001 * TargetJitter.GetUnitVector();
			FIndex2i EdgeT = Mesh->GetEdgeT(eid);
			DrawNumberIfVisible(eid, Position,
				[&](int32 HitTID, double RayHitT, FVector3d) { return EdgeT.Contains(HitTID); });
		}
	}
	else if (Settings->ShowIndices == EMeshInspectorToolDrawIndexMode::GroupID)
	{
		if (bDrawGroupsDataValid == false)
		{
			GroupVisualizationCache.UpdateGroupInfo_ConnectedComponents(*Mesh, *ActiveGroupSet);
			bDrawGroupsDataValid = true;
		}
		for (const FGroupVisualizationCache::FGroupInfo& GroupInfo : GroupVisualizationCache)
		{
			DrawNumberIfVisible(GroupInfo.GroupID, GroupInfo.Center,
				[&](int32 HitTID, double, FVector3d) { return HitTID == GroupInfo.CenterTris.A || HitTID == GroupInfo.CenterTris.B; });
		}
	}
}


void UMeshInspectorTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	GetToolManager()->PostInvalidation();
	UpdateVisualization();
}

void UMeshInspectorTool::UpdateVisualization()
{
	// return if tool is not in a valid state (e.g., has already shut down)
	if (!MaterialSettings || !PreviewMesh)
	{
		return;
	}

	PreviewMesh->EnableWireframe(Settings->bWireframe);

	// determine custom material
	MaterialSettings->UpdateMaterials();

	// if override color function was set, clear it
	if (MaterialSettings->MaterialMode != EMeshInspectorMaterialMode::GroupColor)
	{
		PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	else
	{
		PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
		{
			return LinearColors::SelectFColor(ActiveGroupSet->GetTriangleGroup(TriangleID));
		}, UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}

	UMaterialInterface* OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	if (OverrideMaterial == nullptr)
	{
		PreviewMesh->ClearOverrideRenderMaterial();
	}
	else
	{
		PreviewMesh->SetOverrideRenderMaterial(OverrideMaterial);
	}

	FColor BoundaryEdgeColor(240, 15, 15);
	float BoundaryEdgeThickness = LineWidthMultiplier * 4.0;
	FColor UVSeamColor(15, 240, 15);
	float UVSeamThickness = LineWidthMultiplier * 2.0;
	const FColor MissingUVColor(240, 15, 15);
	const float MissingUVThickness = LineWidthMultiplier * 2.0;
	FColor NormalSeamColor(15, 240, 240);
	float NormalSeamThickness = LineWidthMultiplier * 2.0;
	FColor TangentSeamColor(64, 240, 240);
	float TangentSeamThickness = LineWidthMultiplier * 2.0;
	FColor PolygonBorderColor(240, 15, 240);
	float PolygonBorderThickness = LineWidthMultiplier * 2.0;
	FColor NormalColor(15, 15, 240);
	float NormalThickness = LineWidthMultiplier * 2.0f;
	FColor TangentColor(240, 15, 15);
	FColor BinormalColor(15, 240, 15);
	float TangentThickness = LineWidthMultiplier * 2.0f;

	float BoundaryEdgeDepthBias = 0.2f;
	float UVSeamDepthBias = 0.3f;
	const float MissingUVDepthBias = 0.3f;
	float NormalSeamDepthBias = 0.3f;
	float TangentSeamDepthBias = 0.3f;
	float PolygonBorderDepthBias = 0.2f;
	float NormalDepthBias = 0.0f;
	float TangentDepthBias = 0.35f;

	// Used to scale normals and tangents appropriately
	FVector ComponentScale = DrawnLineSet->GetComponentTransform().GetScale3D();
	FVector3f InverseScale(1 / ComponentScale.X, 1 / ComponentScale.Y, 1 / ComponentScale.Z);

	const FDynamicMesh3* TargetMesh = PreviewMesh->GetPreviewDynamicMesh();
	FVector3d A, B;

	DrawnLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager(), !Settings->bDrawHiddenEdgesAndSeams));
	DrawnLineSet->Clear();
	if (Settings->bBoundaryEdges)
	{
		for (int eid : BoundaryEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}

	if (Settings->bUVSeams)
	{
		for (int eid : UVSeamEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, UVSeamColor, UVSeamThickness, UVSeamDepthBias);
		}
	}

	if (Settings->bMissingUVs)
	{
		for (int eid : MissingUVTriangleEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, MissingUVColor, MissingUVThickness, UVSeamDepthBias);
		}
	}

	if (Settings->bNormalSeams)
	{
		for (int eid : NormalSeamEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, NormalSeamColor, NormalSeamThickness, NormalSeamDepthBias);
		}
	}

	if (Settings->bTangentSeams)
	{
		for (int eid : TangentSeamEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, TangentSeamColor, TangentSeamThickness, TangentSeamDepthBias);
		}
	}

	if (Settings->bPolygonBorders)
	{
		for (int eid : GroupBoundaryEdges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, PolygonBorderColor, PolygonBorderThickness, PolygonBorderDepthBias);
		}
	}

	if (Settings->bNormalVectors && TargetMesh->HasAttributes() && TargetMesh->Attributes()->PrimaryNormals() != nullptr)
	{
		// Note that for normals and tangent vectors, we want to allow the origin of the vector to undergo the full
		// transform of the component, but the direction we travel along the vector has to be inversely scaled
		// to end up with the original length in the end (and to reflect the fact that Unreal does not consider
		// the scale transform when adjusting normals for an object).
		FVector3f NormalScaling = Settings->NormalLength * InverseScale;

		const FDynamicMeshNormalOverlay* NormalOverlay = TargetMesh->Attributes()->PrimaryNormals();
		FVector3d TriV[3];
		FVector3f TriN[3];
		for (int tid : TargetMesh->TriangleIndicesItr())
		{
			if ( NormalOverlay->IsSetTriangle(tid) )
			{
				TargetMesh->GetTriVertices(tid, TriV[0], TriV[1], TriV[2]);
				NormalOverlay->GetTriElements(tid, TriN[0], TriN[1], TriN[2]);
				for (int j = 0; j < 3; ++j)
				{
					DrawnLineSet->AddLine((FVector)TriV[j], (FVector)((FVector3f)TriV[j] + NormalScaling * TriN[j]),
						NormalColor, NormalThickness, NormalDepthBias);
				}
			}
		}
	}

	if (Settings->bTangentVectors && PreviewMesh->GetMesh()->Attributes()->HasTangentSpace() )
	{
		// See note in normals about scaling
		FVector3f TangentScaling = Settings->TangentLength * InverseScale;

		FDynamicMeshTangents Tangents(PreviewMesh->GetMesh());
		for (int TID : TargetMesh->TriangleIndicesItr())
		{
			FVector3d TriV[3];
			TargetMesh->GetTriVertices(TID, TriV[0], TriV[1], TriV[2]);
			for (int SubIdx = 0; SubIdx < 3; SubIdx++)
			{
				FVector3f Vert(TriV[SubIdx]);
				FVector3f Tangent, Bitangent;
				Tangents.GetTangentVectors(TID, SubIdx, Tangent, Bitangent);
				DrawnLineSet->AddLine((FVector)Vert, (FVector)(Vert + TangentScaling * Tangent),
					TangentColor, TangentThickness, TangentDepthBias);
				DrawnLineSet->AddLine((FVector)Vert, (FVector)(Vert + TangentScaling * Bitangent),
					BinormalColor, TangentThickness, TangentDepthBias);
			}
		}
	}
}


bool UMeshInspectorTool::HasAccept() const
{
	return false;
}

bool UMeshInspectorTool::CanAccept() const
{
	return false;
}



void UMeshInspectorTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 1,
		TEXT("IncreaseLineWidth"), 
		LOCTEXT("IncreaseLineWidth", "Increase Line Width"),
		LOCTEXT("IncreaseLineWidthTooltip", "Increase line width of rendering"),
		EModifierKey::Shift, EKeys::Equals,
		[this]() { IncreaseLineWidthAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
		TEXT("DecreaseLineWidth"), 
		LOCTEXT("DecreaseLineWidth", "Decrease Line Width"),
		LOCTEXT("DecreaseLineWidthTooltip", "Decrease line width of rendering"),
		EModifierKey::None, EKeys::Equals,
		[this]() { DecreaseLineWidthAction(); });
}



void UMeshInspectorTool::IncreaseLineWidthAction()
{
	LineWidthMultiplier = LineWidthMultiplier * 1.25f;
	GetToolManager()->PostInvalidation();
}

void UMeshInspectorTool::DecreaseLineWidthAction()
{
	LineWidthMultiplier = LineWidthMultiplier * (1.0f / 1.25f);
	GetToolManager()->PostInvalidation();
}



void UMeshInspectorTool::OnSelectedGroupLayerChanged()
{
	UpdateActiveGroupLayer();
}

void UMeshInspectorTool::UpdateActiveGroupLayer()
{
	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeUnique<UE::Geometry::FPolygroupSet>(PreviewMesh->GetMesh());
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		const FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*PreviewMesh->GetMesh(), SelectedName);
		ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
		ActiveGroupSet = MakeUnique<UE::Geometry::FPolygroupSet>(PreviewMesh->GetMesh(), FoundAttrib);
	}
	bDrawGroupsDataValid = false;

	// force visualization update
	UpdateVisualization();
}



#undef LOCTEXT_NAMESPACE

