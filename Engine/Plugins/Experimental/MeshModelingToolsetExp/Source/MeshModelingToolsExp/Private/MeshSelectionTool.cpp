// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSelectionTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Changes/ToolCommandChangeSequence.h"
#include "Changes/MeshChange.h"
#include "Util/ColorConstants.h"
#include "Selections/MeshConnectedComponents.h"
#include "MeshRegionBoundaryLoops.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolSetupUtil.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "ModelingToolTargetUtil.h"
#include "Properties/MeshStatisticsProperties.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Properties/MeshUVChannelProperties.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selections/GeometrySelectionUtil.h"

#include "Algo/MaxElement.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSelectionTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshSelectionTool"

/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UMeshSelectionToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshSelectionTool* SelectionTool = NewObject<UMeshSelectionTool>(SceneState.ToolManager);
	SelectionTool->SetWorld(SceneState.World);

	return SelectionTool;
}

void UMeshSelectionToolBuilder::InitializeNewTool(UMeshSurfacePointTool* Tool, const FToolBuilderState& SceneState) const
{
	UMeshSurfacePointMeshEditingToolBuilder::InitializeNewTool(Tool, SceneState);
	if (UMeshSelectionTool* SelectionTool = Cast<UMeshSelectionTool>(Tool))
	{
		UE::Geometry::FGeometrySelection Selection;
		if (UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, SelectionTool->GetTarget(), Selection))
		{
			SelectionTool->SetGeometrySelection(MoveTemp(Selection));
		}
	}
}

/*
 * Properties
 */
void UMeshSelectionToolActionPropertySet::PostAction(EMeshSelectionToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}

/*
 * Tool
 */
UMeshSelectionTool::UMeshSelectionTool()
{
}

void UMeshSelectionTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UMeshSelectionTool::Setup()
{
	UDynamicMeshBrushTool::Setup();

	// hide strength and falloff
	BrushProperties->bShowStrength = BrushProperties->bShowFalloff = false;
	BrushProperties->RestoreProperties(this);

	SelectionProps = NewObject<UMeshSelectionToolProperties>(this);
	SelectionProps->RestoreProperties(this);
	AddToolPropertySource(SelectionProps);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this);
	PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { PolygroupLayerProperties->InitializeGroupLayers(&ReadMesh); });
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { UpdateActiveGroupLayer(); bColorsUpdatePending = true; bFullMeshInvalidationPending = true; });
	AddToolPropertySource(PolygroupLayerProperties);
	UpdateActiveGroupLayer();

	UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
	UVChannelProperties->RestoreProperties(this);
	PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { UVChannelProperties->Initialize(&ReadMesh, false); });
	UVChannelProperties->ValidateSelection(true);
	UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue) { CacheUVIslandIDs(); bColorsUpdatePending = true; bFullMeshInvalidationPending = true; });
	AddToolPropertySource(UVChannelProperties);

	// we could probably calculate this on-demand but we need to do it before making any mesh changes? or update?
	CacheUVIslandIDs();

	AddSubclassPropertySets();

	SelectionActions = NewObject<UMeshSelectionEditActions>(this);
	SelectionActions->Initialize(this);
	AddToolPropertySource(SelectionActions);

	EditActions = CreateEditActions();
	AddToolPropertySource(EditActions);

	// set autocalculated tangents
	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

	// disable shadows
	PreviewMesh->SetShadowsEnabled(false);

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.9f, 0.1f, 0.1f), GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	// enable secondary triangle buffers
	PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return SelectedTriangles[TriangleID] ? true : false;
	});

	// enable auto-chunking of mesh into separate render buffers, so that partial updates can be done during selection paint
	PreviewMesh->SetEnableRenderMeshDecomposition(true);

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	SelectedVertices = TBitArray<>(false, Mesh->MaxVertexID());
	SelectedTriangles = TBitArray<>(false, Mesh->MaxTriangleID());

	this->Selection = NewObject<UMeshSelectionSet>(this);
	Selection->GetOnModified().AddLambda([this](USelectionSet* SelectionObj)
	{
		OnExternalSelectionChange();
	});

	// rebuild octree if mesh changes
	PreviewMesh->GetOnMeshChanged().AddLambda([this]() { 
		bOctreeValid = false; 
		bFullMeshInvalidationPending = true; 
		bColorsUpdatePending = true; 
		CacheUVIslandIDs(); 
		UpdateActiveGroupLayer();
	});

	SelectionProps->WatchProperty(SelectionProps->FaceColorMode,
								  [this](EMeshFacesColorMode NewValue)
								  {
									  bColorsUpdatePending = true; UpdateVisualization(false);
								  });
	bColorsUpdatePending = (SelectionProps->FaceColorMode != EMeshFacesColorMode::None);

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(PreviewMesh->GetWorld(), PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowWireframe = true;
		MeshElementsDisplay->Settings->bShowUVSeams = false;
		MeshElementsDisplay->Settings->bShowNormalSeams = false;
		MeshElementsDisplay->Settings->bShowColorSeams = false;
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("MeshSelection"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		PreviewMesh->ProcessMesh(ProcessFunc);
	});

	MeshStatisticsProperties = NewObject<UMeshStatisticsProperties>(this);
	AddToolPropertySource(MeshStatisticsProperties);

	// initialize our selection from input selection, if available 
	if (InputGeometrySelection.IsEmpty() == false)
	{
		TArray<int32> AddFaces;
		UE::Geometry::EnumerateSelectionTriangles(InputGeometrySelection, *Mesh, 
			[&](int32 TriangleID) { AddFaces.Add(TriangleID); });
		Selection->AddIndices(EMeshSelectionElementType::Face, AddFaces);
		OnExternalSelectionChange();
	}

	RecalculateBrushRadius();
	UpdateVisualization(true);

	SetToolDisplayName(LOCTEXT("ToolName", "Select Triangles"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartMeshSelectionTool", "This Tool allows you to modify the mesh based on a triangle selection. [Q] cyles through Selection Mode. [A] cycles through Face Color modes. [ and ] change brush size, < and > grow/shrink selection."),
		EToolMessageLevel::UserNotification);


}



UMeshSelectionToolActionPropertySet* UMeshSelectionTool::CreateEditActions()
{
	UMeshSelectionMeshEditActions* Actions = NewObject<UMeshSelectionMeshEditActions>(this);
	Actions->Initialize(this);
	return Actions;
}



void UMeshSelectionTool::ApplyShutdownAction(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshSelectionToolTransactionName", "Edit Mesh"));

		TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);

		if (bHaveModifiedMesh )
		{
			FDynamicMesh3 ResultMesh(*PreviewMesh->GetMesh());
			if (ResultMesh.IsCompact() == false)
			{
				FCompactMaps CompactMaps;
				ResultMesh.CompactInPlace(&CompactMaps);
				for (int32& tid : SelectedFaces)
				{
					tid = CompactMaps.GetTriangleMapping(tid);
				}
			}

			UE::ToolTarget::CommitDynamicMeshUpdate(Target, ResultMesh, true);
		}

		if (SelectionType == EMeshSelectionElementType::Face)
		{
			if ( SelectedFaces.Num() > 0 )
			{
				FGeometrySelection OutputSelection;
				for ( int32 tid : SelectedFaces )
				{
					OutputSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
				}
				UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, GetTarget(), OutputSelection);
			}
		}

		GetToolManager()->EndUndoTransaction();
	}
	else if (ShutdownType == EToolShutdownType::Cancel)
	{
		for (AActor* Spawned : SpawnedActors)
		{
			Spawned->Destroy();
		}
	}
}


void UMeshSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SelectionProps->SaveProperties(this);
	BrushProperties->SaveProperties(this);
	UVChannelProperties->SaveProperties(this);
	PolygroupLayerProperties->SaveProperties(this);

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("MeshSelection"));
	}
	MeshElementsDisplay->Disconnect();

	ApplyShutdownAction(ShutdownType);
}



void UMeshSelectionTool::SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn)
{
	InputGeometrySelection = SelectionIn;
}


void UMeshSelectionTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UDynamicMeshBrushTool::RegisterActions(ActionSet);


	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 50,
		TEXT("TriSelectIncreaseSize"),
		LOCTEXT("TriSelectIncreaseSize", "Increase Size"),
		LOCTEXT("TriSelectIncreaseSizeTooltip", "Increase Brush Size"),
		EModifierKey::None, EKeys::D,
		[this]() { IncreaseBrushSizeAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 51,
		TEXT("TriSelectDecreaseSize"),
		LOCTEXT("TriSelectDecreaseSize", "Decrease Size"),
		LOCTEXT("TriSelectDecreaseSizeTooltip", "Decrease Brush Size"),
		EModifierKey::None, EKeys::S,
		[this]() { DecreaseBrushSizeAction(); });


	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 1,
		TEXT("MeshSelectionToolDelete"),
		LOCTEXT("MeshSelectionToolDelete", "Delete"),
		LOCTEXT("MeshSelectionToolDeleteTooltip", "Delete Selected Elements"),
		EModifierKey::None, EKeys::Delete,
		[this]() { DeleteSelectedTriangles(); });


#if WITH_EDITOR  	// enum HasMetaData()  is not available at runtime
	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::CycleSelectionMode,
		TEXT("CycleSelectionMode"),
		LOCTEXT("CycleSelectionMode", "Cycle Selection Mode"),
		LOCTEXT("CycleSelectionModeTooltip", "Cycle through selection modes"),
		EModifierKey::None, EKeys::Q,
		[this]() {
			const UEnum* SelectionModeEnum = StaticEnum<EMeshSelectionToolPrimaryMode>();
			check(SelectionModeEnum);
			int32 NumEnum = SelectionModeEnum->NumEnums() - 1;
			do {
				SelectionProps->SelectionMode = (EMeshSelectionToolPrimaryMode)(((int32)SelectionProps->SelectionMode + 1) % NumEnum);
			} while (SelectionModeEnum->HasMetaData(TEXT("Hidden"), (int32)SelectionProps->SelectionMode));
		}
	);

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::CycleViewMode,
		TEXT("CycleViewMode"),
		LOCTEXT("CycleViewMode", "Cycle View Mode"),
		LOCTEXT("CycleViewModeTooltip", "Cycle through face coloring modes"),
		EModifierKey::None, EKeys::A,
		[this]() {
			const UEnum* ViewModeEnum = StaticEnum<EMeshFacesColorMode>();
			check(ViewModeEnum);
			int32 NumEnum = ViewModeEnum->NumEnums() - 1;
			do {
				SelectionProps->FaceColorMode = (EMeshFacesColorMode)(((int32)SelectionProps->FaceColorMode + 1) % NumEnum);
			} while (ViewModeEnum->HasMetaData(TEXT("Hidden"), (int32)SelectionProps->FaceColorMode));
		}
	);
#endif

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::ShrinkSelection,
		TEXT("ShrinkSelection"),
		LOCTEXT("ShrinkSelection", "Shrink Selection"),
		LOCTEXT("ShrinkSelectionTooltip", "Shrink selection"),
		EModifierKey::Shift, EKeys::Comma,
		[this]() { GrowShrinkSelection(false); });

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::GrowSelection,
		TEXT("GrowSelection"),
		LOCTEXT("GrowSelection", "Grow Selection"),
		LOCTEXT("GrowSelectionTooltip", "Grow selection"),
		EModifierKey::Shift, EKeys::Period,
		[this]() { GrowShrinkSelection(true); });

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::OptimizeSelection,
		TEXT("OptimizeSelection"),
		LOCTEXT("OptimizeSelection", "Optimize Selection"),
		LOCTEXT("OptimizeSelectionTooltip", "Optimize selection"),
		EModifierKey::None, EKeys::O,
		[this]() { OptimizeSelection(); });

	ActionSet.RegisterAction(this, (int32)EMeshSelectionToolActions::SmoothBoundary,
		TEXT("SmoothBoundary"),
		LOCTEXT("SmoothBoundary", "Smooth Boundary"),
		LOCTEXT("SmoothBoundaryTooltip", "Smooth Boundary"),
		EModifierKey::None, EKeys::B,
		[this]() { SmoothSelectionBoundary(); });

}



void UMeshSelectionTool::OnExternalSelectionChange()
{
	int32 NumTriangles = SelectedTriangles.Num();
	TSet<int32> AllModifiedTriangles;
	for (int32 k = 0; k < NumTriangles; ++k)
	{
		if (SelectedTriangles[k])
		{
			AllModifiedTriangles.Add(k);
		}
	}

	SelectedVertices.SetRange(0, SelectedVertices.Num(), false);
	SelectedTriangles.SetRange(0, SelectedTriangles.Num(), false);

	if (SelectionType == EMeshSelectionElementType::Vertex)
	{
		ensure(false);		// not supported
		for (int VertIdx : Selection->Vertices)
		{
			SelectedVertices[VertIdx] = true;
		}
	}
	else if (SelectionType == EMeshSelectionElementType::Face)
	{
		for (int FaceIdx : Selection->Faces)
		{
			SelectedTriangles[FaceIdx] = true;
			AllModifiedTriangles.Add(FaceIdx);
		}
		// todo: more efficient for rendering update to compute the delta set here...
		OnRegionHighlightUpdated(AllModifiedTriangles);
	}
}




bool UMeshSelectionTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	bool bHit = UDynamicMeshBrushTool::HitTest(Ray, OutHit);
	if (bHit && SelectionProps->bHitBackFaces == false)
	{
		const FDynamicMesh3* SourceMesh = PreviewMesh->GetPreviewDynamicMesh();
		FVector3d Normal, Centroid; 
		double Area;
		SourceMesh->GetTriInfo(OutHit.FaceIndex, Normal, Area, Centroid);
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
		FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

		if (Normal.Dot((Centroid - LocalEyePosition)) > 0)
		{
			bHit = false;
		}
	}
	return bHit;
}


void UMeshSelectionTool::OnBeginDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnBeginDrag(WorldRay);

	PreviewBrushROI.Reset();
	bInRemoveStroke = GetShiftToggle();
	BeginChange(bInRemoveStroke == false);
	StartStamp = UBaseBrushTool::LastBrushStamp;
	LastStamp = StartStamp;
	bStampPending = true;
	LongTransactions.Open(LOCTEXT("MeshSelectionChange", "Mesh Selection"), GetToolManager());
}



void UMeshSelectionTool::OnUpdateDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnUpdateDrag(WorldRay);
	LastStamp = UBaseBrushTool::LastBrushStamp;
	bStampPending = true;
}



TUniquePtr<FDynamicMeshOctree3>& UMeshSelectionTool::GetOctree()
{
	if (bOctreeValid == false)
	{
		Octree = MakeUnique<FDynamicMeshOctree3>();
		Octree->Initialize(PreviewMesh->GetPreviewDynamicMesh());
		bOctreeValid = true;
	}
	return Octree;
}



void UMeshSelectionTool::CalculateVertexROI(const FBrushStampData& Stamp, TArray<int>& VertexROI)
{
	FTransform3d Transform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d StampPosLocal = Transform.InverseTransformPosition((FVector3d)Stamp.WorldPosition);

	// TODO: need dynamic vertex hash table!
	float Radius = GetCurrentBrushRadiusLocal();
	float RadiusSqr = Radius * Radius;
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	for (int VertIdx : Mesh->VertexIndicesItr())
	{
		FVector3d Position = Mesh->GetVertex(VertIdx);
		if ((Position - StampPosLocal).SquaredLength() < RadiusSqr)
		{
			VertexROI.Add(VertIdx);
		}
	}
}




void UMeshSelectionTool::CalculateTriangleROI(const FBrushStampData& Stamp, TArray<int>& TriangleROI)
{
	FTransform3d Transform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d StampPosLocal = Transform.InverseTransformPosition((FVector3d)Stamp.WorldPosition);

	// always select first triangle
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	float Radius = GetCurrentBrushRadiusLocal();
	float RadiusSqr = Radius * Radius;
	if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::VolumetricBrush)
	{
		if (Mesh->IsTriangle(Stamp.HitResult.FaceIndex))
		{
			TriangleROI.Add(Stamp.HitResult.FaceIndex);
		}

		FAxisAlignedBox3d Bounds(StampPosLocal- Radius*FVector3d::One(), StampPosLocal+ Radius*FVector3d::One());
		TemporaryBuffer.Reset();
		GetOctree()->RangeQuery(Bounds, TemporaryBuffer);

		for (int32 TriIdx : TemporaryBuffer)
		{
			FVector3d Position = Mesh->GetTriCentroid(TriIdx);
			if ((Position - StampPosLocal).SquaredLength() < RadiusSqr)
			{
				TriangleROI.Add(TriIdx);
			}
		}
	}
	else
	{
		TArray<int32> StartROI;
		StartROI.Add(Stamp.HitResult.FaceIndex);
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI,  &TemporaryBuffer, &TemporarySet,
			[Mesh, RadiusSqr, StampPosLocal](int t1, int t2) { return (Mesh->GetTriCentroid(t2) - StampPosLocal).SquaredLength() < RadiusSqr; });

	}


}




static void UpdateList(TArray<int>& List, int Value, bool bAdd)
{
	if (bAdd)
	{
		List.Add(Value);
	}
	else
	{
		List.RemoveSwap(Value);
	}
}


void UMeshSelectionTool::ApplyStamp(const FBrushStampData& Stamp)
{
	IndexBuf.Reset();

	bool bDesiredValue = bInRemoveStroke ? false : true;

	if (SelectionType == EMeshSelectionElementType::Face)
	{
		CalculateTriangleROI(Stamp, IndexBuf);
		UpdateFaceSelection(Stamp, IndexBuf);
	}
	else
	{
		CalculateVertexROI(Stamp, IndexBuf);
		for (int VertIdx : IndexBuf)
		{
			if (SelectedVertices[VertIdx] != bDesiredValue)
			{
				SelectedVertices[VertIdx] = bDesiredValue;
				UpdateList(Selection->Vertices, VertIdx, bDesiredValue);
				if (ActiveSelectionChange != nullptr)
				{
					ActiveSelectionChange->Add(VertIdx);
				}
			}
		}

		ensure(false);		// not supported yet
	}
}






void UMeshSelectionTool::UpdateFaceSelection(const FBrushStampData& Stamp, const TArray<int>& TriangleROI)
{
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	const TArray<int>* UseROI = &TriangleROI;

	TArray<int> LocalROI;
	if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::AllConnected)
	{
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, TriangleROI, LocalROI, &TemporaryBuffer, &TemporarySet);
		UseROI = &LocalROI;
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::AllInGroup)
	{
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, TriangleROI, LocalROI, &TemporaryBuffer, &TemporarySet,
			[&](int t1, int t2) { return ActiveGroupSet->GetTriangleGroup(t1) == ActiveGroupSet->GetTriangleGroup(t2); } );
		UseROI = &LocalROI;
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::ByMaterial)
	{
		const FDynamicMeshMaterialAttribute* MaterialIDs = Mesh->Attributes()->GetMaterialID();
		// If material IDs are missing, this logic will be skipped and selection will fall back to brush selection
		if (MaterialIDs)
		{
			TArray<int32> StartROI;
			StartROI.Add(Stamp.HitResult.FaceIndex);
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, LocalROI, &TemporaryBuffer, &TemporarySet,
				[Mesh, MaterialIDs](int t1, int t2) { return MaterialIDs->GetValue(t1) == MaterialIDs->GetValue(t2); });
			UseROI = &LocalROI;
		}
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::ByMaterialAll)
	{
		const FDynamicMeshMaterialAttribute* MaterialIDs = Mesh->Attributes()->GetMaterialID();
		// If material IDs are missing, this logic will be skipped and selection will fall back to brush selection
		if (MaterialIDs)
		{
			int32 HitMaterialID = MaterialIDs->GetValue(Stamp.HitResult.FaceIndex);
			for (int32 TID : Mesh->TriangleIndicesItr())
			{
				if (MaterialIDs->GetValue(TID) == HitMaterialID)
				{
					LocalROI.Add(TID);
				}
			}
			UseROI = &LocalROI;
		}
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::ByUVIsland)
	{
		TArray<int32> StartROI;
		StartROI.Add(Stamp.HitResult.FaceIndex);
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, LocalROI, &TemporaryBuffer, &TemporarySet,
			[&](int t1, int t2) { return TriangleToUVIsland[t1] == TriangleToUVIsland[t2]; });
		UseROI = &LocalROI;
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::AllWithinAngle)
	{
		TArray<int32> StartROI; 
		StartROI.Add(Stamp.HitResult.FaceIndex);
		FVector3d StartNormal = Mesh->GetTriNormal(StartROI[0]);
		int AngleTol = SelectionProps->AngleTolerance;
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, LocalROI, &TemporaryBuffer, &TemporarySet,
			[Mesh, AngleTol, StartNormal](int t1, int t2) { return UE::Geometry::AngleD(Mesh->GetTriNormal(t2), StartNormal) < AngleTol; });

		UseROI = &LocalROI;
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::AngleFiltered)
	{
		TSet<int32> BrushROI(TriangleROI);
		TArray<int32> StartROI;
		StartROI.Add(Stamp.HitResult.FaceIndex);
		FVector3d StartNormal = Mesh->GetTriNormal(StartROI[0]);
		int AngleTol = SelectionProps->AngleTolerance;
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, LocalROI, &TemporaryBuffer, &TemporarySet,
			[Mesh, AngleTol, StartNormal, &BrushROI](int t1, int t2) { return BrushROI.Contains(t2) && UE::Geometry::AngleD(Mesh->GetTriNormal(t2), StartNormal) < AngleTol; });
		UseROI = &LocalROI;
	}
	else if (SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::Visible)
	{
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
		FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

		for (int tid : TriangleROI)
		{
			FVector3d Centroid = Mesh->GetTriCentroid(tid);
			int HitTID = GetOctree()->FindNearestHitObject(FRay3d(LocalEyePosition, UE::Geometry::Normalized(Centroid - LocalEyePosition)));
			if (HitTID == tid)
			{
				LocalROI.Add(HitTID);
			}
		}
		UseROI = &LocalROI;
	}

	// separate paths for add and remove because we can't efficiently remove from Selection->Faces TArray
	if (bInRemoveStroke == false)
	{
		for (int TriIdx : *UseROI)
		{
			if (SelectedTriangles[TriIdx] == false)
			{
				SelectedTriangles[TriIdx] = true;
				Selection->Faces.Add(TriIdx);
				if (ActiveSelectionChange != nullptr)
				{
					ActiveSelectionChange->Add(TriIdx);
				}
			}
		}
	}
	else
	{
		bool bModified = false;
		TArray<int32> CurSelection = Selection->Faces;
		for (int TriIdx : *UseROI)
		{
			if (SelectedTriangles[TriIdx] == true)
			{
				SelectedTriangles[TriIdx] = false;
				bModified = true;
				if (ActiveSelectionChange != nullptr)
				{
					ActiveSelectionChange->Add(TriIdx);
				}
			}
		}
		// rebuild selection
		if (bModified)
		{
			Selection->Faces.Reset();
			for (int32 tid : CurSelection)
			{
				if (SelectedTriangles[tid])
				{
					Selection->Faces.Add(tid);
				}
			}
		}
	}

	OnRegionHighlightUpdated(*UseROI);
}




void UMeshSelectionTool::OnEndDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnEndDrag(Ray);

	bInRemoveStroke = false;
	bStampPending = false;

	// close change record
	TUniquePtr<FToolCommandChange> Change = EndChange();
	if (Change)
	{
		GetToolManager()->EmitObjectChange(Selection, MoveTemp(Change), LOCTEXT("MeshSelectionChange", "Mesh Selection"));
		LongTransactions.Close(GetToolManager());
	}
}


bool UMeshSelectionTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UDynamicMeshBrushTool::OnUpdateHover(DevicePos);

	// todo get rid of this redundant hit test!
	FHitResult OutHit;
	if ( UDynamicMeshBrushTool::HitTest(DevicePos.WorldRay, OutHit) )
	{
		PreviewBrushROI.Reset();
		if (SelectionType == EMeshSelectionElementType::Face)
		{
			CalculateTriangleROI(LastBrushStamp, PreviewBrushROI);
		}
		else
		{
			CalculateVertexROI(LastBrushStamp, PreviewBrushROI);
		}
	}

	return true;
}


FBox UMeshSelectionTool::GetWorldSpaceFocusBox()
{
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() > 0)
	{
		FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();

		const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
		FTransform3d Transform(PreviewMesh->GetTransform());
		for (int32 tid : SelectedFaces)
		{
			FIndex3i Tri = Mesh->GetTriangle(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				Bounds.Contain(Transform.TransformPosition(Mesh->GetVertex(Tri[j])));
			}
		}

		if (Bounds.MaxDim() > FMathf::ZeroTolerance)
		{
			return (FBox)Bounds;
		}
	}

	return PreviewMesh->GetActor()->GetComponentsBoundingBox();
}



bool UMeshSelectionTool::SupportsNestedCancelCommand()
{
	return true;
}

bool UMeshSelectionTool::CanCurrentlyNestedCancel()
{
	return Selection && (Selection->Faces.Num() > 0);
}

bool UMeshSelectionTool::ExecuteNestedCancelCommand()
{
	if (CanCurrentlyNestedCancel())
	{
		// Only actually clear if there's no in-progress change; this just means escape does nothing in the middle of a mouse drag
		// (We could make esc undo the active change and cancel the drag in this case; not clear if that is worthwhile. In most similar situations, escape just exits the tool.)
		if (!ActiveSelectionChange)
		{
			ClearSelection();
		}
		return true;
	}
	return false;
}



void UMeshSelectionTool::OnRegionHighlightUpdated()
{
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::SecondaryIndexBuffers, false);

}
void UMeshSelectionTool::OnRegionHighlightUpdated(const TArray<int32>& Triangles)
{
	PreviewMesh->NotifyRegionDeferredEditCompleted(Triangles, EMeshRenderAttributeFlags::SecondaryIndexBuffers);
}
void UMeshSelectionTool::OnRegionHighlightUpdated(const TSet<int32>& Triangles)
{
	PreviewMesh->NotifyRegionDeferredEditCompleted(Triangles, EMeshRenderAttributeFlags::SecondaryIndexBuffers);
}



void UMeshSelectionTool::UpdateVisualization(bool bSelectionModified)
{
	check(SelectionType == EMeshSelectionElementType::Face);  // only face selection supported so far

	SetToolPropertySourceEnabled(UVChannelProperties,
								 SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::ByUVIsland ||
								 SelectionProps->FaceColorMode == EMeshFacesColorMode::ByUVIsland);

	if (bFullMeshInvalidationPending)
	{
		PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			MeshStatisticsProperties->Update(ReadMesh);
		});
		MeshElementsDisplay->NotifyMeshChanged();
		bFullMeshInvalidationPending = false;
	}

	// force an update of renderbuffers
	if (bSelectionModified)
	{
		PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FullUpdate, EMeshRenderAttributeFlags::AllVertexAttribs, true);
	}

	if (bColorsUpdatePending)
	{
		if (SelectionProps->FaceColorMode != EMeshFacesColorMode::None)
		{
			PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetVertexColorMaterial(GetToolManager()));
			PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
			{
				return GetCurrentFaceColor(Mesh, TriangleID);
			}, 
			UPreviewMesh::ERenderUpdateMode::FastUpdate);
		}
		else
		{
			PreviewMesh->ClearOverrideRenderMaterial();
			PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
		}

		bColorsUpdatePending = false;
	}
}



FColor UMeshSelectionTool::GetCurrentFaceColor(const FDynamicMesh3* Mesh, int TriangleID)
{
	if (SelectionProps->FaceColorMode == EMeshFacesColorMode::ByGroup)
	{
		return LinearColors::SelectFColor(ActiveGroupSet->GetTriangleGroup(TriangleID));
	}
	else if (SelectionProps->FaceColorMode == EMeshFacesColorMode::ByMaterialID)
	{
		return LinearColors::SelectFColor( Mesh->Attributes()->GetMaterialID()->GetValue(TriangleID) );
	}
	else if (SelectionProps->FaceColorMode == EMeshFacesColorMode::ByUVIsland)
	{
		return LinearColors::SelectFColor(TriangleToUVIsland[TriangleID]);
	}
	return FColor::Red;

}


void UMeshSelectionTool::CacheUVIslandIDs()
{
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
	FMeshConnectedComponents Components(Mesh);

	TriangleToUVIsland.SetNum(Mesh->MaxTriangleID());

	int32 ActiveUVLayer = UVChannelProperties->GetSelectedChannelIndex(true);
	const FDynamicMeshUVOverlay* UV = Mesh->Attributes()->GetUVLayer(ActiveUVLayer);

	Components.FindConnectedTriangles([&](int32 TriIdx0, int32 TriIdx1)
	{
		return UV->AreTrianglesConnected(TriIdx0, TriIdx1);
	});

	int32 NumComponents = Components.Num();
	for (int32 ci = 0; ci < NumComponents; ++ci)
	{
		for (int32 TriIdx : Components.GetComponent(ci).Indices)
		{
			TriangleToUVIsland[TriIdx] = ci;
		}
	}
}


void UMeshSelectionTool::UpdateActiveGroupLayer()
{
	// todo: need this to be const
	const FDynamicMesh3* SourceMesh = PreviewMesh->GetMesh();

	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(SourceMesh);
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		const FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*SourceMesh, SelectedName);
		ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(SourceMesh, FoundAttrib);
	}

}


void UMeshSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UDynamicMeshBrushTool::Render(RenderAPI);

	FTransform WorldTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();

	if (SelectionProps->bShowPoints)
	{
		float PDIScale = RenderAPI->GetCameraState().GetPDIScalingFactor();
		if (SelectionType == EMeshSelectionElementType::Vertex)
		{
			MeshDebugDraw::DrawVertices(Mesh, Selection->Vertices,
				12.0f*PDIScale, FColor::Orange, RenderAPI->GetPrimitiveDrawInterface(), WorldTransform);
			MeshDebugDraw::DrawVertices(Mesh, PreviewBrushROI,
				8.0f*PDIScale, FColor(40, 200, 40), RenderAPI->GetPrimitiveDrawInterface(), WorldTransform);
		}
		else
		{
			MeshDebugDraw::DrawTriCentroids(Mesh, PreviewBrushROI,
				4.0f*PDIScale, FColor(40, 200, 40), RenderAPI->GetPrimitiveDrawInterface(), WorldTransform);
		}
	}
}


void UMeshSelectionTool::OnTick(float DeltaTime)
{
	if (bStampPending)
	{
		ApplyStamp(LastStamp);
		bStampPending = false;
	}

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EMeshSelectionToolActions::NoAction;
	}

	if (bFullMeshInvalidationPending)
	{
		UpdateVisualization(false);
	}

	MeshElementsDisplay->OnTick(DeltaTime);
}



void UMeshSelectionTool::BeginChange(bool bAdding)
{
	check(ActiveSelectionChange == nullptr);
	ActiveSelectionChange = new FMeshSelectionChangeBuilder(SelectionType, bAdding);
}

void UMeshSelectionTool::CancelChange()
{
	if (ActiveSelectionChange != nullptr)
	{
		delete ActiveSelectionChange;
		ActiveSelectionChange = nullptr;
	}
}

TUniquePtr<FToolCommandChange> UMeshSelectionTool::EndChange()
{
	if (ActiveSelectionChange != nullptr)
	{
		TUniquePtr<FMeshSelectionChange> Result = MoveTemp(ActiveSelectionChange->Change);
		delete ActiveSelectionChange;
		ActiveSelectionChange = nullptr;

		return Result;
	}
	return TUniquePtr<FMeshSelectionChange>();
}






void UMeshSelectionTool::RequestAction(EMeshSelectionToolActions ActionType)
{
	if (bHavePendingAction)
	{
		return;
	}

	PendingAction = ActionType;
	bHavePendingAction = true;
}


void UMeshSelectionTool::ApplyAction(EMeshSelectionToolActions ActionType)
{
	switch (ActionType)
	{
		case EMeshSelectionToolActions::SelectAll:
			SelectAll();
			break;
	
		case EMeshSelectionToolActions::SelectAllByMaterial:
			SelectAllByMaterial();
			break;

		case EMeshSelectionToolActions::ClearSelection:
			ClearSelection();
			break;

		case EMeshSelectionToolActions::InvertSelection:
			InvertSelection();
			break;


		case EMeshSelectionToolActions::GrowSelection:
			GrowShrinkSelection(true);
			break;

		case EMeshSelectionToolActions::ShrinkSelection:
			GrowShrinkSelection(false);
			break;

		case EMeshSelectionToolActions::SelectLargestComponentByArea:
			SelectLargestComponent(true);
			break;

		case EMeshSelectionToolActions::SelectLargestComponentByTriCount:
			SelectLargestComponent(false);
			break;

		case EMeshSelectionToolActions::OptimizeSelection:
			OptimizeSelection();
			break;

		case EMeshSelectionToolActions::ExpandToConnected:
			ExpandToConnected();
			break;

		case EMeshSelectionToolActions::DeleteSelected:
			DeleteSelectedTriangles();
			break;

		case EMeshSelectionToolActions::DisconnectSelected:
			DisconnectSelectedTriangles();
			break;

		case EMeshSelectionToolActions::SmoothBoundary:
			SmoothSelectionBoundary();
			break;

		case EMeshSelectionToolActions::SeparateSelected:
			SeparateSelectedTriangles(true);
			break;

		case EMeshSelectionToolActions::DuplicateSelected:
			SeparateSelectedTriangles(false);
			break;

		case EMeshSelectionToolActions::FlipSelected:
			FlipSelectedTriangles();
			break;

		case EMeshSelectionToolActions::CreateGroup:
			AssignNewGroupToSelectedTriangles();
			break;
	}
}



void UMeshSelectionTool::SelectAll()
{
	BeginChange(true);
	
	TArray<int32> AddFaces;
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	for (int tid : Mesh->TriangleIndicesItr())
	{
		if (SelectedTriangles[tid] == false)
		{
			AddFaces.Add(tid);
		}
	}
	
	ActiveSelectionChange->Add(AddFaces);
	Selection->AddIndices(EMeshSelectionElementType::Face, AddFaces);
	
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("SelectAll", "Select All"));

	OnExternalSelectionChange();
}


void UMeshSelectionTool::SelectAllByMaterial()
{
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	if (Selection->Faces.Num() == 0|| Mesh->HasAttributes() == false || Mesh->Attributes()->HasMaterialID() == false)
	{
		SelectAll();
		return;
	}

	BeginChange(true);
	
	TSet<int32> SelectedMaterialIDs;
	const FDynamicMeshMaterialAttribute* MaterialIDs = Mesh->Attributes()->GetMaterialID();
	for (int tid : Mesh->TriangleIndicesItr())
	{
		if ( SelectedTriangles[tid] )
		{
			SelectedMaterialIDs.Add( MaterialIDs->GetValue(tid) );
		}
	}

	TArray<int32> AddFaces;
	for (int tid : Mesh->TriangleIndicesItr())
	{
		if ( SelectedTriangles[tid] == false && SelectedMaterialIDs.Contains(MaterialIDs->GetValue(tid)) )
		{
			AddFaces.Add(tid);
		}
	}
	
	ActiveSelectionChange->Add(AddFaces);
	Selection->AddIndices(EMeshSelectionElementType::Face, AddFaces);
	
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("ExpandToMaterial", "Select Materials"));

	OnExternalSelectionChange();
}



void UMeshSelectionTool::ClearSelection()
{
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	BeginChange(false);
	ActiveSelectionChange->Add(SelectedFaces);
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);

	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("ClearSelection", "Clear Selection"));

	OnExternalSelectionChange();
}




void UMeshSelectionTool::InvertSelection()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	TArray<int32> InvertedFaces;
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	for (int tid : Mesh->TriangleIndicesItr())
	{
		if (SelectedTriangles[tid] == false)
		{
			InvertedFaces.Add(tid);
		}
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("InvertSelection", "Invert Selection"));

	// clear current selection
	BeginChange(false);
	ActiveSelectionChange->Add(SelectedFaces);
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);
	TUniquePtr<FToolCommandChange> ClearChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(ClearChange), LOCTEXT("InvertSelection", "Invert Selection"));

	// add inverted selection
	BeginChange(true);
	ActiveSelectionChange->Add(InvertedFaces);
	Selection->AddIndices(EMeshSelectionElementType::Face, InvertedFaces);
	TUniquePtr<FToolCommandChange> AddChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(AddChange), LOCTEXT("InvertSelection", "Invert Selection"));

	GetToolManager()->EndUndoTransaction();

	OnExternalSelectionChange();
}





void UMeshSelectionTool::GrowShrinkSelection(bool bGrow)
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	TArray<int32> Vertices;
	UE::Geometry::TriangleToVertexIDs(Mesh, SelectedFaces, Vertices);

	TSet<int32> ChangeFaces;
	for (int vid : Vertices)
	{
		int OutCount = 0;
		for (int tid : Mesh->VtxTrianglesItr(vid))
		{
			if (SelectedTriangles[tid] == false)
			{
				OutCount++;
			}
		}
		if (OutCount == 0)
		{
			continue;
		}

		for (int tid : Mesh->VtxTrianglesItr(vid))
		{
			if ( (bGrow && SelectedTriangles[tid] == false) || (bGrow == false && SelectedTriangles[tid]) )
			{
				ChangeFaces.Add(tid);
			}
		}
	}
	if( SelectionProps->SelectionMode == EMeshSelectionToolPrimaryMode::AllInGroup )
	{
		TSet<int32> AdjacentFaces{ChangeFaces};
		TSet<int32> AdjacentGroups{};
		ChangeFaces.Empty();
		for ( int32 TID : AdjacentFaces )
		{
			AdjacentGroups.Add(ActiveGroupSet->GetTriangleGroup(TID));
		}
		for ( int32 TID : Mesh->TriangleIndicesItr() )
		{
			if ( AdjacentGroups.Contains(ActiveGroupSet->GetTriangleGroup(TID)) )
			{
				ChangeFaces.Add(TID);
			}
		}
	}
	if (ChangeFaces.Num() == 0)
	{
		return;
	}
	BeginChange(bGrow);
	ActiveSelectionChange->Add(ChangeFaces);
	if (bGrow)
	{
		Selection->AddIndices(EMeshSelectionElementType::Face, ChangeFaces);
		TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
		GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("GrowSelection", "Grow Selection"));
	}
	else
	{
		Selection->RemoveIndices(EMeshSelectionElementType::Face, ChangeFaces);
		TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
		GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("ShrinkSelection", "Shrink Selection"));
	}
	OnExternalSelectionChange();
}





void UMeshSelectionTool::ExpandToConnected()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	TArray<int32> Queue(SelectedFaces);
	TSet<int32> AddFaces;

	while (Queue.Num() > 0)
	{
		int32 CurTri = Queue.Pop(EAllowShrinking::No);
		FIndex3i NbrTris = Mesh->GetTriNeighbourTris(CurTri);

		for (int j = 0; j < 3; ++j)
		{
			int32 tid = NbrTris[j];
			if (tid != FDynamicMesh3::InvalidID && SelectedTriangles[tid] == false && AddFaces.Contains(tid) == false)
			{
				AddFaces.Add(tid);
				Queue.Add(tid);
			}
		}
	}
	if (AddFaces.Num() == 0)
	{
		return;
	}

	BeginChange(true);
	ActiveSelectionChange->Add(AddFaces);
	Selection->AddIndices(EMeshSelectionElementType::Face, AddFaces);
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
	GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("ExpandToConnected", "Expand Selection"));
	OnExternalSelectionChange();
}


void UMeshSelectionTool::SelectLargestComponent(bool bWeightByArea)
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	// each component gets its own group id
	FMeshConnectedComponents Components(Mesh);
	Components.FindConnectedTriangles(SelectedFaces);

	if (Components.Num() == 0) // no triangles?
	{
		ClearSelection();
		return;
	}


	int BestComponent = 0;
	FMeshConnectedComponents::FComponent* MaxComponent = Algo::MaxElementBy(Components, [bWeightByArea, &Mesh](const FMeshConnectedComponents::FComponent& Component)
	{
		if (bWeightByArea)
		{
			double AreaSum = 0;
			for (int TID : Component.Indices)
			{
				AreaSum += Mesh->GetTriArea(TID);
			}
			return AreaSum;
		}
		else
		{
			return (double)Component.Indices.Num();
		}
	});

	BeginChange(false);
	for (FMeshConnectedComponents::FComponent& Component : Components)
	{
		if (&Component != MaxComponent)
		{
			ActiveSelectionChange->Add(Component.Indices);
			Selection->RemoveIndices(EMeshSelectionElementType::Face, Component.Indices);
		}
	}
	
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(SelectionChange), LOCTEXT("SelectLargestComponentByArea", "Select Largest Component By Area"));

	OnExternalSelectionChange();
}


void UMeshSelectionTool::OptimizeSelection()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	if (Selection->Faces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	FMeshFaceSelection FaceSelection(Mesh);
	TSet<int> OriginalSelection(Selection->Faces);
	FaceSelection.Select(Selection->Faces);
	FaceSelection.LocalOptimize(true);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("OptimizeSelection", "Optimize Selection"));

	// remove faces from the current selection that are not in the optimized one
	BeginChange(false);

	for (int32 FaceSelIdx = Selection->Faces.Num() - 1; FaceSelIdx >= 0; FaceSelIdx--)
	{
		int32 TID = Selection->Faces[FaceSelIdx];
		if (!FaceSelection.IsSelected(TID))
		{
			Selection->Faces.RemoveAtSwap(FaceSelIdx, EAllowShrinking::No);
			ActiveSelectionChange->Add(TID);
		}
	}
	Selection->NotifySelectionSetModified();

	TUniquePtr<FToolCommandChange> DeselectChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(DeselectChange), LOCTEXT("OptimizeSelection", "Optimize Selection"));

	// add faces from the optimized selection to the current selection, if they were not in the original
	BeginChange(true);

	Selection->Faces.Reserve(FaceSelection.Num());
	for (int32 TID : FaceSelection.AsSet())
	{
		if (!OriginalSelection.Contains(TID))
		{
			ActiveSelectionChange->Add(TID);
			Selection->Faces.Add(TID);
		}
	}
	Selection->NotifySelectionSetModified();

	check(Selection->Faces.Num() == FaceSelection.Num());
	
	TUniquePtr<FToolCommandChange> AddChange = EndChange();

	GetToolManager()->EmitObjectChange(Selection, MoveTemp(AddChange), LOCTEXT("OptimizeSelection", "Optimize Selection"));

	GetToolManager()->EndUndoTransaction();

	OnExternalSelectionChange();
}


void UMeshSelectionTool::DeleteSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}
	if (SelectedFaces.Num() >= PreviewMesh->GetMesh()->TriangleCount())
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	// clear current selection
	BeginChange(false);
	for (int tid : SelectedFaces)
	{
		ActiveSelectionChange->Add(tid);
	}
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
	ChangeSeq->AppendChange(Selection, MoveTemp(SelectionChange));

	// delete triangles and emit delete triangles change
	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&SelectedFaces](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		FDynamicMeshEditor Editor(&Mesh);
		Editor.RemoveTriangles(SelectedFaces, true, [&ChangeTracker](int TriangleID) { ChangeTracker.SaveTriangle(TriangleID, true); });
	});
	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolDeleteFaces", "Delete Faces"));

	bFullMeshInvalidationPending = true;
	OnExternalSelectionChange();
	bHaveModifiedMesh = true;
	bOctreeValid = false;
}


void UMeshSelectionTool::DisconnectSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	// split out selected triangles and emit triangle change
	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&SelectedFaces](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		// save vertices and triangles that are on the boundary of the selection
		FMeshRegionBoundaryLoops BoundaryLoops(&Mesh, SelectedFaces);
		for (const FEdgeLoop& Loop : BoundaryLoops.Loops)
		{
			// include the whole one-ring in case the disconnect creates bowties that need to be split
			ChangeTracker.SaveVertexOneRingTriangles(Loop.Vertices, true);
		}

		FDynamicMeshEditor Editor(&Mesh);
		Editor.DisconnectTriangles(SelectedFaces);
	});
	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolDisconnectFaces", "Disconnect Faces"));

	bFullMeshInvalidationPending = true;
	bHaveModifiedMesh = true;
}



void UMeshSelectionTool::SeparateSelectedTriangles(bool bDeleteSelected)
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* SourceMesh = PreviewMesh->GetPreviewDynamicMesh();
	if (SelectedFaces.Num() == SourceMesh->TriangleCount())
	{
		return;		// don't separate entire mesh
	}

	// extract copy of triangles
	FDynamicMesh3 SeparatedMesh;
	SeparatedMesh.EnableTriangleGroups();
	SeparatedMesh.EnableAttributes();
	SeparatedMesh.Attributes()->EnableMatchingAttributes(*SourceMesh->Attributes());
	FDynamicMeshEditor Editor(&SeparatedMesh);
	FMeshIndexMappings Mappings; FDynamicMeshEditResult EditResult;
	Editor.AppendTriangles(SourceMesh, SelectedFaces, Mappings, EditResult);

	// emit new asset
	FTransform3d Transform(PreviewMesh->GetTransform());
	GetToolManager()->BeginUndoTransaction(
		(bDeleteSelected) ? LOCTEXT("MeshSelectionToolSeparate", "Separate") : LOCTEXT("MeshSelectionToolDuplicate", "Duplicate") );

	// build array of materials from the original
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	TArray<UMaterialInterface*> Materials = MaterialSet.Materials;

	AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
	FString AssetName = TargetActor->GetActorNameOrLabel();

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = (FTransform)Transform;
	NewMeshObjectParams.BaseName = (AssetName.IsEmpty()) ? TEXT("Submesh") : (AssetName + TEXT("_Submesh"));
	NewMeshObjectParams.Materials = Materials;
	NewMeshObjectParams.SetMesh(&SeparatedMesh);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		SpawnedActors.Add(Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();

	// delete selected triangles from this mesh
	if (bDeleteSelected)
	{
		DeleteSelectedTriangles();
	}

	// Currently have to mark the mesh as 'modified' so we can Accept, because if we Cancel,
	// Actor created by duplicate operation will be rolled back
	bHaveModifiedMesh = true;		
}



void UMeshSelectionTool::FlipSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	// clear current selection
	BeginChange(false);
	for (int tid : SelectedFaces)
	{
		ActiveSelectionChange->Add(tid);
	}
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
	ChangeSeq->AppendChange(Selection, MoveTemp(SelectionChange));

	// flip normals
	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&SelectedFaces](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		for (int TID : SelectedFaces)
		{
			ChangeTracker.SaveTriangle(TID, true);
		}
		FDynamicMeshEditor Editor(&Mesh);
		Editor.ReverseTriangleOrientations(SelectedFaces, true);
	});
	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolFlipFaces", "Flip Face Orientations"));

	bHaveModifiedMesh = true;
}


void UMeshSelectionTool::AssignNewGroupToSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	// clear current selection
	BeginChange(false);
	for (int tid : SelectedFaces)
	{
		ActiveSelectionChange->Add(tid);
	}
	Selection->RemoveIndices(EMeshSelectionElementType::Face, SelectedFaces);
	TUniquePtr<FToolCommandChange> SelectionChange = EndChange();
	ChangeSeq->AppendChange(Selection, MoveTemp(SelectionChange));

	// assign new groups to triangles
	// note: using an FMeshChange is kind of overkill here
	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&SelectedFaces, this](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		// each component gets its own group id
		FMeshConnectedComponents Components(&Mesh);
		Components.FindConnectedTriangles(SelectedFaces);

		for (FMeshConnectedComponents::FComponent& Component : Components)
		{
			int NewGroupID = Mesh.AllocateTriangleGroup();
			for (int tid : Component.Indices)
			{
				ChangeTracker.SaveTriangle(tid, true);
				ActiveGroupSet->SetGroup(tid, NewGroupID, Mesh);
			}
		}
	});
	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolCreateGroup", "Create Polygroup"));

	OnExternalSelectionChange();
	bHaveModifiedMesh = true;
}

void UMeshSelectionTool::SmoothSelectionBoundary()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	FMeshRegionBoundaryLoops BoundaryLoops(PreviewMesh->GetMesh(), SelectedFaces, true);
	if (BoundaryLoops.Loops.Num() == 0)
	{
		return;
	}

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	TUniquePtr<FMeshChange> MeshChange = PreviewMesh->TrackedEditMesh(
		[&BoundaryLoops, this](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		constexpr double Alpha = 0.75;

		TMap<int, FVector3d> NewLoopPositions;

		for (const FEdgeLoop& Loop : BoundaryLoops.Loops)
		{
			int NumLoopVertices = Loop.GetVertexCount();

			for (int LoopVertexIndex = 0; LoopVertexIndex < NumLoopVertices; ++LoopVertexIndex)
			{
				const FVector3d PrevPoint = Loop.GetPrevVertex(LoopVertexIndex);
				const FVector3d NextPoint = Loop.GetNextVertex(LoopVertexIndex);
				const FVector3d Avg = 0.5 * (PrevPoint + NextPoint);

				const FVector3d CurrPoint = Loop.GetVertex(LoopVertexIndex);

				const FVector3d NewCurrPoint = (1.0 - Alpha) * CurrPoint + Alpha * Avg;
				const int VertexIndex = Loop.Vertices[LoopVertexIndex];

				// TODO: Reproject to original surface?
				// TODO: Fix the UVs

				NewLoopPositions.Add(VertexIndex, NewCurrPoint);
			}
		}

		for (const TPair<int, FVector3d>& NewVert : NewLoopPositions) 
		{
			ChangeTracker.SaveVertexOneRingTriangles(NewVert.Key, true);
		}

		PreviewMesh->EditMesh([NewLoopPositions](FDynamicMesh3& Mesh)
		{
			for (const TPair<int, FVector3d>& NewVert : NewLoopPositions)
			{
				Mesh.SetVertex(NewVert.Key, NewVert.Value);
			}
		});
	});

	ChangeSeq->AppendChange(PreviewMesh, MoveTemp(MeshChange));

	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshSelectionToolSmoothBoundary", "Smooth Selection Boundary"));

	OnExternalSelectionChange();
	bHaveModifiedMesh = true;
}



#undef LOCTEXT_NAMESPACE

