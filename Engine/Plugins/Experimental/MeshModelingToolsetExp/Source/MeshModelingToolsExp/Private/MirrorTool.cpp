// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorTool.h"

#include "ModelingObjectsCreationAPI.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "CompositionOps/MirrorOp.h"
#include "Drawing/MeshDebugDrawing.h"
#include "DynamicMeshToMeshDescription.h"
#include "InteractiveToolManager.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ModelingToolTargetUtil.h"
#include "Misc/MessageDialog.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "PropertySets/OnAcceptProperties.h" //UOnAcceptHandleSourcesProperties
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/MeshTransforms.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MirrorTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMirrorTool"

namespace MirrorTool_Local
{
	FTransform WithoutScale(FTransform T)
	{
		T.SetScale3D(FVector::One());
		return T;
	}
	FTransform OnlyScale(const FTransform& TransformIn)
	{
		FTransform T = FTransform::Identity;
		T.SetScale3D(TransformIn.GetScale3D());
		return T;
	}
	const FString MirrorToolIdentifier = TEXT("MirrorTool");
}

// Tool builder functions

UMultiSelectionMeshEditingTool* UMirrorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMirrorTool>(SceneState.ToolManager);
}


// Operator factory

TUniquePtr<FDynamicMeshOperator> UMirrorOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FMirrorOp> MirrorOp = MakeUnique<FMirrorOp>();

	// Set up inputs and settings
	MirrorOp->OriginalMesh = MirrorTool->MeshesToMirror[ComponentIndex]->GetMesh();
	MirrorOp->bAppendToOriginal = MirrorTool->Settings->OperationMode == EMirrorOperationMode::MirrorAndAppend;
	MirrorOp->bCropFirst = MirrorTool->Settings->bCropAlongMirrorPlaneFirst;
	MirrorOp->bSimplifyAlongNewEdges = MirrorTool->Settings->bSimplifyAlongCrop;
	MirrorOp->bWeldAlongPlane = MirrorTool->Settings->bWeldVerticesOnMirrorPlane;
	switch (MirrorTool->Settings->WeldVerticesNormalMode)
	{
	case EMeshMirrorWeldNormalMode::MirrorNormals:
		MirrorOp->WeldNormalMode = UE::Geometry::EMeshMirrorNormalMode::MirrorNormals;
		break;
	case EMeshMirrorWeldNormalMode::AverageMirrorNormals:
		MirrorOp->WeldNormalMode = UE::Geometry::EMeshMirrorNormalMode::AverageMirrorNormals;
		break;
	}
	MirrorOp->PlaneTolerance = MirrorTool->Settings->PlaneTolerance;
	MirrorOp->bAllowBowtieVertexCreation = MirrorTool->Settings->bAllowBowtieVertexCreation;

	FTransform LocalToWorld = MirrorTool_Local::WithoutScale((FTransform) UE::ToolTarget::GetLocalToWorldTransform(MirrorTool->Targets[ComponentIndex]));
	MirrorOp->SetTransform(LocalToWorld);

	// Now we can get the plane parameters in local space.
	MirrorOp->LocalPlaneOrigin = LocalToWorld.InverseTransformPosition(MirrorTool->MirrorPlaneOrigin);;

	FVector3d WorldNormal = MirrorTool->MirrorPlaneNormal;
	FTransformSRT3d LocalToWorldSRT(LocalToWorld); // Convert to Geometry::FTransformSRT3d for InverseTransformNormal function
	MirrorOp->LocalPlaneNormal = LocalToWorldSRT.InverseTransformNormal(MirrorTool->MirrorPlaneNormal);

	return MirrorOp;
}


// Tool property functions

void UMirrorToolActionPropertySet::PostAction(EMirrorToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


// Tool itself
UMirrorTool::UMirrorTool()
{
}

bool UMirrorTool::CanAccept() const
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}


void UMirrorTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// Editing the "show preview" option changes whether we need to be displaying the preview or the original mesh.
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMirrorToolProperties, bShowPreview)))
	{
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			UE::ToolTarget::SetSourceObjectVisible(Targets[ComponentIdx], !Settings->bShowPreview);
		}
		for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
		{
			Preview->SetVisibility(Settings->bShowPreview);
		}
	}

	// Regardless of what changed, update the previews.
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}

void UMirrorTool::OnTick(float DeltaTime)
{
	// Deal with any buttons that may have been clicked
	if (PendingAction != EMirrorToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = EMirrorToolAction::NoAction;
	}

	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Tick(DeltaTime);
	}
	
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}
}

void UMirrorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	// Have the plane draw itself.
	PlaneMechanic->Render(RenderAPI);
}

void UMirrorTool::Setup()
{
	using namespace MirrorTool_Local;

	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Mirror"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartMirrorTool", "Mirror one or more meshes across a plane. Ctrl+[shift+]clicking on a spot on the original mesh will place the plane there. Buttons will reset it."),
		EToolMessageLevel::UserNotification);

	// Set up the properties
	Settings = NewObject<UMirrorToolProperties>(this, TEXT("Mirror Tool Settings"));
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->InitializeDefaultWithAuto();
	OutputTypeProperties->OutputType = UCreateMeshObjectTypeProperties::AutoIdentifier;
	OutputTypeProperties->RestoreProperties(this, MirrorToolIdentifier);
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	HandleSourcesProperties = NewObject<UOnAcceptHandleSourcesProperties>(this);
	HandleSourcesProperties->RestoreProperties(this, MirrorToolIdentifier);
	AddToolPropertySource(HandleSourcesProperties);

	ToolActions = NewObject<UMirrorToolActionPropertySet>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	Settings->WatchProperty(Settings->WriteTo, [this](EMirrorSaveMode SaveMode) {
		bool bShowOutputOptions = SaveMode == EMirrorSaveMode::NewObjects;
		SetToolPropertySourceEnabled(OutputTypeProperties, bShowOutputOptions);
		SetToolPropertySourceEnabled(HandleSourcesProperties, bShowOutputOptions);
	});

	CheckAndDisplayWarnings();

	// Fill in the MeshesToMirror array with suitably converted meshes.
	for (int i = 0; i < Targets.Num(); i++)
	{
		// Convert into dynamic mesh
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> DynamicMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		*DynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[i]);
		// Bake the scale part of the transform
		FTransform Transform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[i]);
		MeshTransforms::ApplyTransform(*DynamicMesh, MirrorTool_Local::OnlyScale(Transform), true);

		// Wrap the dynamic mesh in a replacement change target
		UDynamicMeshReplacementChangeTarget* WrappedTarget = MeshesToMirror.Add_GetRef(NewObject<UDynamicMeshReplacementChangeTarget>());

		// Set callbacks so previews are invalidated on undo/redo changing the meshes
		WrappedTarget->SetMesh(DynamicMesh);
		WrappedTarget->OnMeshChanged.AddLambda([this, i]() { Previews[i]->InvalidateResult(); });
	}

	// Set the visibility of the StaticMeshComponents depending on whether we are showing them or the preview.
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::SetSourceObjectVisible(Targets[ComponentIdx], !Settings->bShowPreview);
	}

	// Initialize the PreviewMesh and BackgroundCompute objects
	SetupPreviews();

	// Update the bounding box of the meshes.
	CombinedBounds.Init();
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FVector ComponentOrigin, ComponentExtents;
		UE::ToolTarget::GetTargetActor(Targets[ComponentIdx])->GetActorBounds(false, ComponentOrigin, ComponentExtents);
		CombinedBounds += FBox::BuildAABB(ComponentOrigin, ComponentExtents);
	}

	// Set the initial mirror plane. We want the plane to start in the middle if we're doing a simple
	// mirror (i.e., not appending, and not cropping). Otherwise, we want the plane to start to one side.
	MirrorPlaneOrigin = (FVector3d)CombinedBounds.GetCenter();
	MirrorPlaneNormal = FVector3d(0, -1, 0);
	if (Settings->OperationMode == EMirrorOperationMode::MirrorAndAppend || Settings->bCropAlongMirrorPlaneFirst)
	{
		MirrorPlaneOrigin.Y = CombinedBounds.Min.Y;
	}

	// Set up the mirror plane mechanic, which manages the gizmo
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), FFrame3d(MirrorPlaneOrigin, MirrorPlaneNormal));

	// Have the plane mechanic update things properly
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		MirrorPlaneNormal = PlaneMechanic->Plane.Rotation.AxisZ();
		MirrorPlaneOrigin = PlaneMechanic->Plane.Origin;
		for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
		{
			Preview->InvalidateResult();
		}
		});

	// Include the original components in the ctrl+click hit testing even though we made them 
	// invisible, since we want to be able to reposition the plane onto the original mesh.
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		PlaneMechanic->SetPlaneCtrlClickBehaviorTarget->InvisibleComponentsToHitTest.Add(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]));
	}

	// Start the preview calculations
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}

void UMirrorTool::SetupPreviews()
{
	// Create a preview (with an op) for each selected component.
	int32 NumMeshes = MeshesToMirror.Num();
	for (int32 PreviewIndex = 0; PreviewIndex < NumMeshes; ++PreviewIndex)
	{
		UMirrorOperatorFactory* MirrorOpCreator = NewObject<UMirrorOperatorFactory>();
		MirrorOpCreator->MirrorTool = this;
		MirrorOpCreator->ComponentIndex = PreviewIndex;

		UMeshOpPreviewWithBackgroundCompute* Preview = Previews.Add_GetRef(
			NewObject<UMeshOpPreviewWithBackgroundCompute>(MirrorOpCreator, "Preview"));
		Preview->Setup(GetTargetWorld(), MirrorOpCreator);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
		Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

		const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[PreviewIndex]);
		Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

		// Set initial preview to unprocessed mesh, so that things don't disappear initially
		Preview->PreviewMesh->UpdatePreview(MeshesToMirror[PreviewIndex]->GetMesh().Get());
		FTransform Transform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[PreviewIndex]);
		Preview->PreviewMesh->SetTransform(MirrorTool_Local::WithoutScale(Transform));
		Preview->SetVisibility(Settings->bShowPreview);
	}
}

void UMirrorTool::CheckAndDisplayWarnings()
{
	// We can have more than one warning, which makes this a bit more work.
	FText SameSourceWarning;
	FText ScaleWarning;

	// See if any of the selected components have the same source.
	TArray<int32> MapToFirstOccurrences;
	bool bAnyHaveSameSource = GetMapToSharedSourceData(MapToFirstOccurrences);
	if (bAnyHaveSameSource)
	{
		SameSourceWarning = LOCTEXT("MirrorMultipleAssetsWithSameSource", "WARNING: Multiple meshes in your selection use the same source asset! Only the \"Create New Assets\" save mode is supported.");
		
		// We could forcefully set the save mode to CreateNewAssets, but the setting will persist on new invocations
		// of the tool, which may surprise the user. So, it's up to them to set it.
	}

	// See if any of the selected components have a nonuniform scaling transform.
	IPrimitiveComponentBackedTarget* NonUniformScalingTarget = nullptr;
	IPrimitiveComponentBackedTarget* ZeroScalingTarget = nullptr;
	for (int32 i = 0; i < Targets.Num(); ++i)
	{
		IPrimitiveComponentBackedTarget* Component = Cast<IPrimitiveComponentBackedTarget>(Targets[i]);
		const FVector Scaling = Component->GetWorldTransform().GetScale3D();
		if (Scaling.X == 0 || Scaling.Y == 0 || Scaling.Z == 0)
		{
			ZeroScalingTarget = Component;
			break;
		}
		if (Scaling.X != Scaling.Y || Scaling.Y != Scaling.Z)
		{
			NonUniformScalingTarget = Component;
			// don't break; continue in case we see a ZeroScalingTarget (which is worse)
		}
	}

	if (ZeroScalingTarget)
	{
		ScaleWarning = FText::Format(
			LOCTEXT("MirrorZeroScaledAsset", "WARNING: The item \"{0}\" has a zero-scale on at least one axis. Mirroring cannot be correctly applied in this case. Consider instead baking the scale before mirroring."),
			FText::FromString(ZeroScalingTarget->GetOwnerActor()->GetName()));
	}
	else if (NonUniformScalingTarget) // Only show the non-uniform-scale warning if the more-severe zero-scale warning does not apply
	{
		ScaleWarning = FText::Format(
			LOCTEXT("MirrorNonUniformScaledAsset", "WARNING: The item \"{0}\" has a non-uniform scaling transform. The mirror will be applied in world-space, so the underlying asset will not have mirror symmetry. Consider instead baking the scale before mirroring."),
			FText::FromString(NonUniformScalingTarget->GetOwnerActor()->GetName()));
	}

	if (bAnyHaveSameSource && (NonUniformScalingTarget || ZeroScalingTarget))
	{
		// Concatenates the two warnings with an extra line in between.
		GetToolManager()->DisplayMessage(FText::Format(LOCTEXT("CombinedWarnings", "{0}\n\n{1}"),
			SameSourceWarning, ScaleWarning), EToolMessageLevel::UserWarning);
	}
	else if (bAnyHaveSameSource)
	{
		GetToolManager()->DisplayMessage(SameSourceWarning, EToolMessageLevel::UserWarning);
	}
	else if (NonUniformScalingTarget || ZeroScalingTarget)
	{
		GetToolManager()->DisplayMessage(ScaleWarning, EToolMessageLevel::UserWarning);
	}
}

void UMirrorTool::OnShutdown(EToolShutdownType ShutdownType)
{
	using namespace MirrorTool_Local;

	Settings->SaveProperties(this);
	OutputTypeProperties->SaveProperties(this, MirrorToolIdentifier);
	HandleSourcesProperties->SaveProperties(this, MirrorToolIdentifier);

	PlaneMechanic->Shutdown();

	// Restore (unhide) the source meshes
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	// Swap in results, if appropriate
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MirrorToolTransactionName", "Mirror Tool"));

		// Gather results
		TArray<FDynamicMeshOpResult> Results;
		for (int32 PreviewIndex = 0; PreviewIndex < Previews.Num(); ++PreviewIndex)
		{
			UMeshOpPreviewWithBackgroundCompute* Preview = Previews[PreviewIndex];
			FTransform Transform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[PreviewIndex]);
			Results.Emplace(Preview->Shutdown());
			MeshTransforms::ApplyTransformInverse(*(Results.Last().Mesh), MirrorTool_Local::OnlyScale(Transform), true);
			Results.Last().Transform.SetScale(Transform.GetScale3D() * Results.Last().Transform.GetScale3D());
		}

		// Convert to output. This will also edit the selection.
		GenerateAsset(Results);

		// If we created new objects, deal with the input objects according to the user's settings.
		if (Settings->WriteTo != EMirrorSaveMode::InputObjects)
		{
			TArray<AActor*> Actors;
			for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
			{
				Actors.Add(UE::ToolTarget::GetTargetActor(Targets[ComponentIdx]));
			}
			HandleSourcesProperties->ApplyMethod(Actors, GetToolManager());
		}

		GetToolManager()->EndUndoTransaction();
	}
	else
	{
		for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
		{
			Preview->Cancel();
		}
	}
}

void UMirrorTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	if (Results.Num() == 0)
	{
		return;
	}

	int32 NumSourceMeshes = MeshesToMirror.Num();

	// check if we entirely cut away any meshes
	bool bWantToDestroy = false;
	for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
	{
		if (Results[OrigMeshIdx].Mesh->TriangleCount() == 0)
		{
			bWantToDestroy = true;
			break;
		}
	}
	// if so ask user what to do
	if (bWantToDestroy)
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
			LOCTEXT("MirrorDestroyQuestion", "The mirror plane cropping has entirely cut away at least one mesh. Do you actually want to delete these mesh components? Note that either way all actors will remain, and meshes that are not fully cut away will still be mirrored as normal."), 
			LOCTEXT("MirrorDestroyTitle", "Delete mesh components?"));
		if (Ret == EAppReturnType::No || Ret == EAppReturnType::Cancel)
		{
			bWantToDestroy = false;
		}
	}

	// Properly deal with each result, setting up the selection at the same time.
	FSelectedObjectsChangeList NewSelection;
	NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
	for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
	{
		FDynamicMesh3* Mesh = Results[OrigMeshIdx].Mesh.Get();
		check(Mesh != nullptr);

		if (Mesh->TriangleCount() == 0)
		{
			if (bWantToDestroy)
			{
				UE::ToolTarget::GetTargetComponent(Targets[OrigMeshIdx])->DestroyComponent();
			}
			continue;
		}
		else if (Settings->WriteTo == EMirrorSaveMode::InputObjects)
		{
			NewSelection.Actors.Add(UE::ToolTarget::GetTargetActor(Targets[OrigMeshIdx]));

			UE::ToolTarget::CommitDynamicMeshUpdate(Targets[OrigMeshIdx], *Mesh, true);
		}
		else
		{
			// Build array of materials from the original.
			TArray<UMaterialInterface*> Materials;
			IMaterialProvider* TargetMaterial = Cast<IMaterialProvider>(Targets[OrigMeshIdx]);
			for (int MaterialIdx = 0, NumMaterials = TargetMaterial->GetNumMaterials(); MaterialIdx < NumMaterials; MaterialIdx++)
			{
				Materials.Add(TargetMaterial->GetMaterial(MaterialIdx));
			}

			FCreateMeshObjectParams NewMeshObjectParams;
			NewMeshObjectParams.TargetWorld = GetTargetWorld();
			NewMeshObjectParams.Transform = (FTransform)Results[OrigMeshIdx].Transform;
			NewMeshObjectParams.BaseName = TEXT("Mirror");
			NewMeshObjectParams.Materials = Materials;
			NewMeshObjectParams.SetMesh(Mesh);
			if (OutputTypeProperties->OutputType == UCreateMeshObjectTypeProperties::AutoIdentifier)
			{
				UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[OrigMeshIdx], NewMeshObjectParams);
			}
			else
			{
				OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
			}
			FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
			if (Result.IsOK() && Result.NewActor != nullptr)
			{
				NewSelection.Actors.Add(Result.NewActor);
			}
		}
	}

	// Update the selection
	if (NewSelection.Actors.Num() > 0)
	{
		GetToolManager()->RequestSelectionChange(NewSelection);
	}
}


// Action support

void UMirrorTool::RequestAction(EMirrorToolAction ActionType)
{
	if (PendingAction == EMirrorToolAction::NoAction)
	{
		PendingAction = ActionType;
	}
}

void UMirrorTool::ApplyAction(EMirrorToolAction ActionType)
{
	FVector3d ShiftedPlaneOrigin = (FVector3d)CombinedBounds.GetCenter();

	if (ActionType == EMirrorToolAction::ShiftToCenter)
	{
		// We keep the same orientation here
		PlaneMechanic->SetDrawPlaneFromWorldPos(ShiftedPlaneOrigin, FVector3d(), true);
	}
	else
	{
		// We still start from the center, but adjust one of the coordinates and set direction.
		FVector3d DirectionVector;
		switch (ActionType)
		{
		case EMirrorToolAction::Left:
			ShiftedPlaneOrigin.Y = CombinedBounds.Min.Y;
			DirectionVector = FVector3d(0, -1.0, 0);
			break;
		case EMirrorToolAction::Right:
			ShiftedPlaneOrigin.Y = CombinedBounds.Max.Y;
			DirectionVector = FVector3d(0, 1.0, 0);
			break;
		case EMirrorToolAction::Up:
			ShiftedPlaneOrigin.Z = CombinedBounds.Max.Z;
			DirectionVector = FVector3d(0, 0, 1.0);
			break;
		case EMirrorToolAction::Down:
			ShiftedPlaneOrigin.Z = CombinedBounds.Min.Z;
			DirectionVector = FVector3d(0, 0, -1.0);
			break;
		case EMirrorToolAction::Forward:
			ShiftedPlaneOrigin.X = CombinedBounds.Max.X;
			DirectionVector = FVector3d(1.0, 0, 0);
			break;
		case EMirrorToolAction::Backward:
			ShiftedPlaneOrigin.X = CombinedBounds.Min.X;
			DirectionVector = FVector3d(-1.0, 0, 0);
			break;
		}

		// The user can optionally have the button change the direction only
		if (ToolActions->bButtonsOnlyChangeOrientation)
		{
			ShiftedPlaneOrigin = MirrorPlaneOrigin;	// Keeps the same
		}
		PlaneMechanic->SetDrawPlaneFromWorldPos(ShiftedPlaneOrigin, DirectionVector, false);
	}
}

void UMirrorTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
}

#undef LOCTEXT_NAMESPACE

