// Copyright Epic Games, Inc. All Rights Reserved.

#include "PatternTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "ToolSceneQueriesUtil.h"
#include "ModelingToolTargetUtil.h"
#include "TransformSequence.h"
#include "Selection/ToolSelectionUtil.h"

#include "ModelingObjectsCreationAPI.h"
#include "ModelingComponentsSettings.h"

#include "Drawing/PreviewGeometryActor.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include "ContextObjectStore.h"
#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/ComponentBoundTransformProxy.h"
#include "BaseGizmos/GizmoActor.h"
#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/GizmoUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/GizmoBoxComponent.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformSubGizmoUtil.h"

#include "Engine/World.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "Engine/StaticMeshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PatternTool)


using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPatternTool"


/*
 * ToolBuilder
 */

bool UPatternToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int32 ValidTargets = 0;

	// This tool currently only has 5 output modes, each of them either supporting either static or dynamic meshes.
	// Other targets which are UPrimitiveComponent backed are not currently supported.
	// todo: Add support for skeletal meshes and volumes
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [&ValidTargets](UActorComponent* Component)
	{
		if (Cast<UStaticMeshComponent>(Component) || Cast<UDynamicMeshComponent>(Component))
		{
			ValidTargets++;
		}
	});
	
	return ValidTargets > 0;
}

UMultiSelectionMeshEditingTool* UPatternToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UPatternTool* NewTool = NewObject<UPatternTool>(SceneState.ToolManager);
	NewTool->SetEnableCreateISMCs(bEnableCreateISMCs);
	return NewTool;
	
}

void UPatternToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	TArray<TObjectPtr<UToolTarget>> Targets;
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [this, &Targets, &SceneState](UActorComponent* Component)
	{
		if (Cast<UStaticMeshComponent>(Component) || Cast<UDynamicMeshComponent>(Component))
		{
			Targets.Add(SceneState.TargetManager->BuildTarget(Component, GetTargetRequirements()));
		}
	});

	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);
}

const FToolTargetTypeRequirements& UPatternToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(
		UPrimitiveComponentBackedTarget::StaticClass()
		);
	return TypeRequirements;
}

/*
 * PatternGenerators
 */

// these pattern generators should be promoted to GeometryProcessing, however they need some
// work to clean up the API and make them more correct (ie handling of step size seems a bit
// flaky, particularly for circle patterns)

class FPatternGenerator
{
public:
	enum class ESpacingMode
	{
		ByCount = 0,
		ByStepSize = 1,
		Packed = 2
	};

public:
	// input settings
	FRandomStream RotationRandomStream;
	FRandomStream ScaleRandomStream;
	FRandomStream TranslationRandomStream;
	
	FQuaterniond StartRotation = FQuaterniond::Identity();
	FQuaterniond EndRotation = FQuaterniond::Identity();
	FRotator RotationJitterRange = FRotator::ZeroRotator;	// using an FRotator so that the sampling can be done about 3 axes individually and converted to a quat when needed
	bool bInterpolateRotation = false;						// if false, only StartRotation is used
	bool bJitterRotation = false;
	
	FVector3d StartTranslation = FVector3d::Zero();
	FVector3d EndTranslation = FVector3d::Zero();
	FVector3d TranslationJitterRange = FVector3d::Zero();
	bool bInterpolateTranslation = false;					// if false, only StartTranslation is used
	bool bJitterTranslation = false;

	FVector3d StartScale = FVector3d::One();
	FVector3d EndScale = FVector3d::One();
	FVector3d ScaleJitterRange = FVector3d::Zero();
	bool bInterpolateScale = false;							// if false, only StartScale is used
	bool bJitterScale = false;
	
	FAxisAlignedBox3d Dimensions = FAxisAlignedBox3d(FVector3d::Zero(), 10.0);

public:
	// current result pattern
	TArray<FTransformSRT3d> Pattern;


public:
	void ResetPattern()
	{
		Pattern.Reset();
	}

	void AddPatternElement(FTransformSRT3d NewTransform, double Alpha)
	{
		ApplyElementTransforms(NewTransform, Alpha);
		Pattern.Add(NewTransform);
	}


	void ApplyElementTransforms(FTransformSRT3d& Transform, double Alpha)
	{
		if (bInterpolateRotation)
		{
			FQuaterniond InterpRotation(StartRotation, EndRotation, Alpha);
			Transform.SetRotation(Transform.GetRotation() * InterpRotation);
		}
		else
		{
			Transform.SetRotation(Transform.GetRotation() * StartRotation);
		}

		if (bInterpolateScale)
		{
			Transform.SetScale( Transform.GetScale() * Lerp(StartScale, EndScale, Alpha) );
		}
		else
		{
			Transform.SetScale( Transform.GetScale() * StartScale );
		}

		if (bInterpolateTranslation)
		{
			Transform.SetTranslation( Transform.GetTranslation() + Lerp(StartTranslation, EndTranslation, Alpha) );
		}
		else
		{
			Transform.SetTranslation( Transform.GetTranslation() + StartTranslation );
		}
		
		// Lerp operations are performed on a vector-component basis because each component should be able to independently
		// vary without respect for the other two components. This is true for rotation, scale, and translation.
		if (bJitterRotation)
		{
			FRotator RotationJitter;

			// TODO: maybe RotationJitterRange should have user-definable lower and upper bound like scale.
			RotationJitter.Pitch = FMath::Lerp(-RotationJitterRange.Pitch, RotationJitterRange.Pitch, RotationRandomStream.GetFraction());
			RotationJitter.Roll  = FMath::Lerp(-RotationJitterRange.Roll,  RotationJitterRange.Roll,  RotationRandomStream.GetFraction());
			RotationJitter.Yaw   = FMath::Lerp(-RotationJitterRange.Yaw,   RotationJitterRange.Yaw,   RotationRandomStream.GetFraction());

			Transform.SetRotation(Transform.GetRotation() * FQuaterniond(RotationJitter));
		}

		if (bJitterScale)
		{
			FVector3d ScaleJitter;
			FVector3d TransformScale = Transform.GetScale();

			ScaleJitter.X = FMath::Max(UPatternTool_ScaleSettings::MinScale, TransformScale.X + FMath::Lerp(-ScaleJitterRange.X, ScaleJitterRange.X, ScaleRandomStream.GetFraction()));
			ScaleJitter.Y = FMath::Max(UPatternTool_ScaleSettings::MinScale, TransformScale.Y + FMath::Lerp(-ScaleJitterRange.Y, ScaleJitterRange.Y, ScaleRandomStream.GetFraction()));
			ScaleJitter.Z = FMath::Max(UPatternTool_ScaleSettings::MinScale, TransformScale.Z + FMath::Lerp(-ScaleJitterRange.Z, ScaleJitterRange.Z, ScaleRandomStream.GetFraction()));
				
			Transform.SetScale( ScaleJitter );
		}


		if (bJitterTranslation)
		{
			FVector3d TranslationJitter;

			// TODO: maybe TranslationJitterRange should have user-definable lower and upper bound like scale.
			TranslationJitter.X = FMath::Lerp(-TranslationJitterRange.X, TranslationJitterRange.X, TranslationRandomStream.GetFraction());
			TranslationJitter.Y = FMath::Lerp(-TranslationJitterRange.Y, TranslationJitterRange.Y, TranslationRandomStream.GetFraction());
			TranslationJitter.Z = FMath::Lerp(-TranslationJitterRange.Z, TranslationJitterRange.Z, TranslationRandomStream.GetFraction());
			
			Transform.SetTranslation( Transform.GetTranslation() + TranslationJitter);
		}
	}



	double GetAlpha(double AlphaStep, int32 k, int32 Iterations, bool bForceLastStepToOne)
	{
		if (k == 0)
		{
			return 0;
		}
		else if (bForceLastStepToOne && k == Iterations-1)
		{
			return 1.0;
		}
		else
		{
			return FMathd::Clamp((double)k * AlphaStep, 0.0, 1.0);
		}
	}

	void ComputeSteps(FPatternGenerator::ESpacingMode UseSpacingMode, int32 CountIn, double StepSizeIn, double LengthIn, int32& Iterations, double& AlphaStep, bool& bForceLastStepToOne)
	{
		Iterations = 1;
		AlphaStep = 0.5;
		bForceLastStepToOne = true;

		if (UseSpacingMode == ESpacingMode::ByCount)
		{
			Iterations = CountIn;
			AlphaStep = 1.0 / FMath::Max(Iterations-1, 1);
		}
		else if (UseSpacingMode == ESpacingMode::ByStepSize)
		{
			bForceLastStepToOne = false;
			AlphaStep = StepSizeIn / LengthIn;
			Iterations = FMath::Clamp( (int)(1.0 / (AlphaStep + FMathd::ZeroTolerance) ) + 1, 1, 1000);
		}
		else if (UseSpacingMode == ESpacingMode::Packed)
		{
			bForceLastStepToOne = false;
			AlphaStep = StepSizeIn / LengthIn;
			Iterations = FMath::Clamp( (int)(1.0 / (AlphaStep + FMathd::ZeroTolerance) ) + 1, 1, 1000);
		}
		else
		{
			check(false);
		}
	}
};



class FLinearPatternGenerator : public FPatternGenerator
{
public:

	FFrame3d StartFrame;
	FFrame3d EndFrame;

	ESpacingMode SpacingMode = ESpacingMode::ByCount;

	int Axis = 0;
	int32 Count = 5;
	double StepSize = 1.0;

	// Not used for LineFill
	ESpacingMode SpacingModeY = ESpacingMode::ByCount;
	int AxisY = 1;
	int32 CountY = 5;
	double StepSizeY = 1.0;


	enum class EFillMode
	{
		LineFill,
		RectangleFill
	};
	EFillMode FillMode = EFillMode::LineFill;


	void UpdatePattern()
	{
		ResetPattern();

		if (FillMode == EFillMode::LineFill)
		{
			UpdatePattern_LineFill();
		}
		else if (FillMode == EFillMode::RectangleFill)
		{
			UpdatePattern_RectangleFill();
		}
		else
		{
			ensure(false);
			AddPatternElement(StartFrame.ToTransform(), 1.0);
		}
	}


	void UpdatePattern_LineFill();
	void UpdatePattern_RectangleFill();
};


void FLinearPatternGenerator::UpdatePattern_LineFill()
{
	double LineLength = Distance(StartFrame.Origin, EndFrame.Origin);

	int32 Iterations = 1;
	double AlphaStep = 0.5;
	bool bForceLastStepToOne = true;
	double UseStepSize = (SpacingMode == ESpacingMode::Packed) ? Dimensions.Dimension(Axis) : StepSize;
	ComputeSteps(SpacingMode, Count, UseStepSize, LineLength, Iterations, AlphaStep, bForceLastStepToOne);

	for (int32 k = 0; k < Iterations; ++k)
	{
		FFrame3d PatternFrame = StartFrame;
		double Alpha = 0;
		if (bForceLastStepToOne && k == Iterations-1)
		{
			Alpha = 1;
			PatternFrame = EndFrame;
		}
		else if (k != 0)
		{
			Alpha = GetAlpha(AlphaStep, k, Iterations, bForceLastStepToOne);
			PatternFrame = Lerp(StartFrame, EndFrame, Alpha);
		}

		AddPatternElement(PatternFrame.ToTransform(), Alpha);
	}
}



void FLinearPatternGenerator::UpdatePattern_RectangleFill()
{
	FVector3d LocalPt = StartFrame.ToFramePoint(EndFrame.Origin);
	double ExtentX = LocalPt[Axis];
	double ExtentY = LocalPt[AxisY];

	int32 IterationsX = 1;
	double AlphaStepX = 0.5;
	bool bForceLastStepToOneX = true;
	double UseStepSizeX = (SpacingMode == ESpacingMode::Packed) ? Dimensions.Dimension(Axis) : StepSize;
	ComputeSteps(SpacingMode, Count, UseStepSizeX, ExtentX, IterationsX, AlphaStepX, bForceLastStepToOneX);

	int32 IterationsY = 1;
	double AlphaStepY = 0.5;
	bool bForceLastStepToOneY = true;
	double UseStepSizeY = (SpacingModeY == ESpacingMode::Packed) ? Dimensions.Dimension(AxisY): StepSizeY;
	ComputeSteps(SpacingModeY, CountY, UseStepSizeY, ExtentY, IterationsY, AlphaStepY, bForceLastStepToOneY);

	FFrame3d MidFrameOrientation = Lerp(StartFrame, EndFrame, 0.5);
	FFrame3d Frame00 = StartFrame;
	FFrame3d Frame11 = EndFrame;
	FFrame3d Frame10 = StartFrame;
	Frame10.Origin += ExtentX * StartFrame.GetAxis(Axis);
	Frame10.Rotation = MidFrameOrientation.Rotation;
	FFrame3d Frame01 = StartFrame;
	Frame01.Origin += ExtentY * StartFrame.GetAxis(AxisY);
	Frame01.Rotation = MidFrameOrientation.Rotation;

	for (int32 yi = 0; yi < IterationsY; ++yi)
	{
		double AlphaY = GetAlpha(AlphaStepY, yi, IterationsY, bForceLastStepToOneY);
		FFrame3d Frame0 = Lerp(Frame00, Frame01, AlphaY);
		FFrame3d Frame1 = Lerp(Frame10, Frame11, AlphaY);

		for (int32 xi = 0; xi < IterationsX; ++xi)
		{
			double AlphaX = GetAlpha(AlphaStepX, xi, IterationsX, bForceLastStepToOneX);
			FFrame3d PatternFrame = Lerp(Frame0, Frame1, AlphaX);

			AddPatternElement(PatternFrame.ToTransform(), AlphaX*AlphaY);
		}
	}
}




class FRadialPatternGenerator : public FPatternGenerator
{
public:
	FFrame3d CenterFrame;

	double StartAngleDeg = 0.0;
	double EndAngleDeg = 360.0;
	double AngleShift = 0.0;

	double Radius = 100.0;

	bool bOriented = true;
	int AxisIndexToAlign = 0;	// depends on SinglePlane of tool

	ESpacingMode SpacingMode = ESpacingMode::ByCount;

	int32 Count = 5;
	double StepSizeDeg = 1.0;

	enum class EFillMode
	{
		CircleFill,
	};
	EFillMode FillMode = EFillMode::CircleFill;


	void UpdatePattern()
	{
		ResetPattern();

		if (FillMode == EFillMode::CircleFill)
		{
			UpdatePattern_CircleFill();
		}
	}

	void UpdatePattern_CircleFill();

};


void FRadialPatternGenerator::UpdatePattern_CircleFill()
{
	double ArcLengthDeg = FMathd::Abs(EndAngleDeg - StartAngleDeg);

	int32 Iterations = 1;
	double AlphaStep = 0.5;

	if (SpacingMode == ESpacingMode::ByCount)
	{
		Iterations = Count;
		AlphaStep = 1.0 / FMath::Max(Iterations, 1);
	}
	else if (SpacingMode == ESpacingMode::ByStepSize)
	{
		AlphaStep = StepSizeDeg / ArcLengthDeg;
		Iterations = FMath::Clamp( (int)(1.0 / AlphaStep) + 1, 1, 1000);
	}
	else if (SpacingMode == ESpacingMode::Packed)
	{
		double ArcLenStepSize = Dimensions.Dimension(0);		// currently only support X axis in circle pattern?
		double CalcStepSizeRad = ArcLenStepSize / Radius;
		AlphaStep = (CalcStepSizeRad * FMathd::RadToDeg) / ArcLengthDeg;
		Iterations = FMath::Clamp( (int)(1.0 / AlphaStep) + 1, 1, 1000);
	}
	else
	{
		check(false);
	}

	for (int32 k = 0; k < Iterations; ++k)
	{
		double Alpha = (double)k * AlphaStep;
		if (k == 0)
		{
			Alpha = 0;
		}
		Alpha = FMathd::Clamp(Alpha, 0.0, 1.0);

		double Angle = (1.0-Alpha)*StartAngleDeg + (Alpha)*EndAngleDeg;
		Angle += AngleShift;
		double FrameX = Radius * FMathd::Cos(Angle * FMathd::DegToRad);
		double FrameY = Radius * FMathd::Sin(Angle * FMathd::DegToRad);

		FFrame3d PatternFrame(CenterFrame.FromPlaneUV(FVector2d(FrameX, FrameY), 2));

		if (bOriented)
		{
			FVector3d Axis = Normalized(PatternFrame.Origin - CenterFrame.Origin);
			if (IsNormalized(Axis))
			{
				PatternFrame.ConstrainedAlignAxis(AxisIndexToAlign, Axis, CenterFrame.Z());
			}
		}

		AddPatternElement(PatternFrame.ToTransform(), Alpha);
	}

}




/*
 * Tool
 * 
 * TODO: convert dynamic mesh to temporary static mesh to allow instance rendering (does it work?)
 * TODO: when outputting a merged dynamic mesh, scale/rotation will be baked in, and so there are no limits
 *       on non-uniform scaling, but it would need to be baked into the preview dynamicmesh components 
 *       (and for static meshes being converted, static preview would have to become dynamic preview)
 * TODO: investigate using temporary ISMComponents instead of multiple SMComponents
 * 
 */

UPatternTool::UPatternTool()
{
}


void UPatternTool::Setup()
{
	UInteractiveTool::Setup();

	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(GetTargetWorld(), FTransform::Identity);

	// Must be done before creating gizmos, so that we can bind the mechanic to them.
	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);

	Settings = NewObject<UPatternToolSettings>();
	AddToolPropertySource(Settings);
	Settings->RestoreProperties(this);
	Settings->WatchProperty(Settings->SingleAxis, [this](EPatternToolSingleAxis SingleAxis) { OnSingleAxisUpdated(); });
	Settings->WatchProperty(Settings->SinglePlane, [this](EPatternToolSinglePlane SinglePlane) { OnSinglePlaneUpdated(); });
	Settings->WatchProperty(Settings->Shape, [this](EPatternToolShape) { OnShapeUpdated(); } );
	Settings->WatchProperty(Settings->bHideSources, [this](bool bNewValue) { OnSourceVisibilityToggled(!bNewValue); } );
	Settings->WatchProperty(Settings->Seed, [this](int32 NewSeed) { MarkPatternDirty(); } );
	Settings->WatchProperty(Settings->bProjectElementsDown, [this](bool bNewValue) { MarkPatternDirty(); } );
	Settings->WatchProperty(Settings->ProjectionOffset, [this](float NewValue) { MarkPatternDirty(); } );
	Settings->WatchProperty(Settings->bHideSources, [this](bool bNewValue) { OnSourceVisibilityToggled(!bNewValue); } );
	Settings->WatchProperty(Settings->bUseRelativeTransforms, [this](bool bNewValue) { MarkPatternDirty(); } );
	Settings->WatchProperty(Settings->bRandomlyPickElements, [this](bool bNewValue) { MarkPatternDirty(); });

	BoundingBoxSettings = NewObject<UPatternTool_BoundingBoxSettings>();
	AddToolPropertySource(BoundingBoxSettings);
	BoundingBoxSettings->RestoreProperties(this);
	BoundingBoxSettings->WatchProperty(BoundingBoxSettings->bIgnoreTransforms, [this](bool bNewValue) { MarkPatternDirty(); } );
	BoundingBoxSettings->WatchProperty(BoundingBoxSettings->Adjustment, [this](float NewScale) { MarkPatternDirty(); });
	SetToolPropertySourceEnabled(BoundingBoxSettings, false);
	
	LinearSettings = NewObject<UPatternTool_LinearSettings>();
	AddToolPropertySource(LinearSettings);
	LinearSettings->RestoreProperties(this);
	LinearSettings->WatchProperty(LinearSettings->SpacingMode, [this](EPatternToolAxisSpacingMode NewSpacingMode) { OnSpacingModeUpdated(); });
	LinearSettings->WatchProperty(LinearSettings->bCentered, [this](bool bNewValue) { ResetTransformGizmoPosition(); });
	LinearExtentWatcherIdx = LinearSettings->WatchProperty(LinearSettings->Extent, [this](double NewValue){ ResetTransformGizmoPosition(); });
	SetToolPropertySourceEnabled(LinearSettings, false);

	GridSettings = NewObject<UPatternTool_GridSettings>();
	AddToolPropertySource(GridSettings);
	GridSettings->RestoreProperties(this);
	GridSettings->WatchProperty(GridSettings->SpacingX, [this](EPatternToolAxisSpacingMode NewSpacingMode) { OnSpacingModeUpdated(); });
	GridSettings->WatchProperty(GridSettings->SpacingY, [this](EPatternToolAxisSpacingMode NewSpacingMode) { OnSpacingModeUpdated(); });
	GridSettings->WatchProperty(GridSettings->bCenteredX, [this](bool bNewValue) { ResetTransformGizmoPosition(); });
	GridSettings->WatchProperty(GridSettings->bCenteredY, [this](bool bNewValue) { ResetTransformGizmoPosition(); });
	GridExtentXWatcherIdx = GridSettings->WatchProperty(GridSettings->ExtentX, [this](double NewValue){ ResetTransformGizmoPosition(); });
	GridExtentYWatcherIdx = GridSettings->WatchProperty(GridSettings->ExtentY, [this](double NewValue){ ResetTransformGizmoPosition(); });
	SetToolPropertySourceEnabled(GridSettings, false);

	RadialSettings = NewObject<UPatternTool_RadialSettings>();
	AddToolPropertySource(RadialSettings);
	RadialSettings->RestoreProperties(this);
	RadialSettings->WatchProperty(RadialSettings->SpacingMode, [this](EPatternToolAxisSpacingMode NewSpacingMode) { OnSpacingModeUpdated(); });
	RadiusWatcherIdx = RadialSettings->WatchProperty(RadialSettings->Radius, [this](double NewValue){ ResetTransformGizmoPosition(); });
	SetToolPropertySourceEnabled(RadialSettings, false);

	RotationSettings = NewObject<UPatternTool_RotationSettings>();
	AddToolPropertySource(RotationSettings);
	RotationSettings->RestoreProperties(this);
	
	TranslationSettings = NewObject<UPatternTool_TranslationSettings>();
	AddToolPropertySource(TranslationSettings);
	TranslationSettings->RestoreProperties(this);

	ScaleSettings = NewObject<UPatternTool_ScaleSettings>();
	AddToolPropertySource(ScaleSettings);
	ScaleSettings->RestoreProperties(this);

	auto OnProportionalChanged = [this](bool bNewValue)
	{
		if (bNewValue)
		{
			CachedStartScale = ScaleSettings->StartScale;
			CachedEndScale = ScaleSettings->EndScale;
			CachedJitterScale = ScaleSettings->Jitter;
		}
	};
	ScaleSettings->WatchProperty(ScaleSettings->bProportional, OnProportionalChanged);
	OnProportionalChanged(true);	// Initialize StartScaleDirection and EndScaleDirection

	auto ApplyProportionalScale = [this](FVector& NewVector, FVector& CachedVector)
	{
		// Determines which component of the vector is being changed by looking at which component is
		// most different from the previous values
		FVector Difference = NewVector - CachedVector;
		int32 DifferenceMaxElementIndex = MaxAbsElementIndex(Difference);
		
		// This approach to proportional scaling is desirable because when a user manually enters data
		// numerically, we scale the other two components such that the entered value is unchanged unless
		// doing so would result in component values less than MinScale, in which case the the resulting
		// vector will be in the correct direction but lengthened to a degree to ensure all components are
		// greater than or equal to MinScale.
		double ScaleFactor = FMath::Max(NewVector[DifferenceMaxElementIndex] / CachedVector[DifferenceMaxElementIndex], UPatternTool_ScaleSettings::MinScale / CachedVector[MinElementIndex(CachedVector)]);
		
		NewVector = CachedVector * ScaleFactor;
		CachedVector = NewVector;
	};
	
	StartScaleWatcherIdx = ScaleSettings->WatchProperty(ScaleSettings->StartScale, [this, &ApplyProportionalScale](const FVector& NewStartScale)
	{
		if (ScaleSettings->bProportional)
		{
			ApplyProportionalScale(ScaleSettings->StartScale, CachedStartScale);
			ScaleSettings->SilentUpdateWatcherAtIndex(StartScaleWatcherIdx);

			// This is needed in addition to the call in OnPropertyModified due to the fact that these watchers are
			// called after OnPropertyModified and in some cases the scale values used were inconsistent with the scale
			// values being displayed in the details panel.
			OnParametersUpdated();
		}
	});
	
	EndScaleWatcherIdx = ScaleSettings->WatchProperty(ScaleSettings->EndScale, [this, &ApplyProportionalScale](const FVector& NewEndScale)
	{
		if (ScaleSettings->bProportional)
		{
			ApplyProportionalScale(ScaleSettings->EndScale, CachedEndScale);
			ScaleSettings->SilentUpdateWatcherAtIndex(EndScaleWatcherIdx);
			
			OnParametersUpdated();
		}
	});

	JitterScaleWatcherIdx = ScaleSettings->WatchProperty(ScaleSettings->Jitter, [this, &ApplyProportionalScale](const FVector& NewJitterScale)
	{
		if (ScaleSettings->bProportional)
		{
			ApplyProportionalScale(ScaleSettings->Jitter, CachedJitterScale);
			ScaleSettings->SilentUpdateWatcherAtIndex(JitterScaleWatcherIdx);
			
			OnParametersUpdated();
		}
	});
	
	OutputSettings = NewObject<UPatternTool_OutputSettings>();
	AddToolPropertySource(OutputSettings);
	OutputSettings->RestoreProperties(this);
	OutputSettings->bCreateISMCs &= bEnableCreateISMCs;
	OutputSettings->bEnableCreateISMCs = bEnableCreateISMCs;

	BoundingBoxVisualizer.LineThickness = 2.0f;
	
	InitializeElements();
	for (const FPatternElement& Element : Elements)
	{
		if (Element.SourceStaticMesh != nullptr)
		{
			OutputSettings->bHaveStaticMeshes = true;
		}
	}

	CurrentStartFrameWorld = FFrame3d(Elements[0].SourceTransform);
	
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize( GetTargetWorld(), CurrentStartFrameWorld );
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() { OnMainFrameUpdated(); });

	UGizmoViewContext* GizmoViewContext = nullptr;
	UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore();
	if (ContextObjectStore)
	{
		GizmoViewContext = ContextObjectStore->FindContext<UGizmoViewContext>();
	}
	
	// The advantage of using a fully visually symmetric component here is that we can use the same component 
	//  regardless of whether we're translating in a line or a plane, so we don't need to swap it out when that
	//  changes.
	// Note that this helper method already attaches the component to the parent gizmo actor.
	UGizmoBoxComponent* BoxComponent = AGizmoActor::AddDefaultBoxComponent(GetWorld(), PlaneMechanic->PlaneTransformGizmo->GetGizmoActor(),
		GizmoViewContext, FLinearColor::Red, FVector::ZeroVector);
	BoxComponent->LineThickness = 5.0f;
	BoxComponent->Dimensions = FVector(BoxComponent->LineThickness * 1.5);
	BoxComponent->NotifyExternalPropertyUpdates();
	PatternGizmoComponent = BoxComponent;

	PatternGizmoProxy = NewObject<UComponentBoundTransformProxy>(this);
	PatternGizmoProxy->OnTransformChanged.AddUObject(this, &UPatternTool::OnTransformGizmoUpdated);
	PatternGizmoProxy->BindToComponent(BoxComponent, 
		// bStoreScaleSeparately, doesn't matter because scale isn't used
		false);

	// Needs to be called before any of the watchers call ResetTransformGizmoPosition in order to set the
	// proxy as the target of the gizmo, otherwise the gizmo will attempt to dereference an invalid StateTarget
	// due to SetActiveTarget having never been called
	ReconstructTransformGizmos();
	
	SetToolDisplayName(LOCTEXT("ToolName", "Pattern"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartPatternTool", "Create Patterns for the selected Objects"),
		EToolMessageLevel::UserNotification);

	if (bHaveNonUniformScaleElements)
	{
		// todo: this message only applies in certain contexts like if bSeparateActors=true or if 
		// emitting an ISMC. Should make it dynamic and/or smarter.
		GetToolManager()->DisplayMessage(LOCTEXT("NonUniformScaleWarning", 
			"Source Objects have Non-Uniform Scaling, which may prevent Pattern Transforms from working correctly."), EToolMessageLevel::UserWarning);
	}
}







void UPatternTool::OnShutdown(EToolShutdownType ShutdownType)
{
	PlaneMechanic->Shutdown();
	PlaneMechanic = nullptr;

	// destroy all the preview components we created
	for (UPrimitiveComponent* Component : AllComponents)
	{
		Component->UnregisterComponent();
		Component->DestroyComponent();
	}
	AllComponents.Reset();
	AllPreviewComponents.Reset();
	PreviewComponents.Reset();
	StaticMeshPools.Reset();
	DynamicMeshPools.Reset();

	PreviewGeometry->Disconnect();

	Settings->SaveProperties(this);
	BoundingBoxSettings->SaveProperties(this);
	LinearSettings->SaveProperties(this);
	GridSettings->SaveProperties(this);
	RadialSettings->SaveProperties(this);
	RotationSettings->SaveProperties(this);
	TranslationSettings->SaveProperties(this);
	ScaleSettings->SaveProperties(this);
	OutputSettings->SaveProperties(this);

	DragAlignmentMechanic->Shutdown();

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	
	OnSourceVisibilityToggled(true);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		EmitResults();
	}
}




void UPatternTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
	PlaneMechanic->Render(RenderAPI);
	// We should only render here if bVisualize is true and also visible in the details panel.
	// We don't want to leave the user with no obvious way to turn it off which might happen if they
	// enable bVisualize and then change the spacing mode to something other than packed
	if (BoundingBoxSettings->IsPropertySetEnabled() && BoundingBoxSettings->bVisualize)
	{
		RenderBoundingBoxes(RenderAPI);
	}
	
	if (bPatternNeedsUpdating)
	{
		// throttle update rate as it is somewhat expensive
		double DeltaTime = (FDateTime::Now() - LastPatternUpdateTime).GetTotalSeconds();
		if (DeltaTime > 0.05)
		{
			UpdatePattern();
			bPatternNeedsUpdating = false;
			LastPatternUpdateTime = FDateTime::Now();
		}
	}
}

void UPatternTool::RenderBoundingBoxes(IToolsContextRenderAPI* RenderAPI)
{
	BoundingBoxVisualizer.BeginFrame(RenderAPI);

	// Render the individual elements' PatternBounds along the current pattern with green lines
	BoundingBoxVisualizer.LineColor = FLinearColor::Green;
	for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		FPatternElement& Element = Elements[ElemIdx];
		
		if (Settings->bUseRelativeTransforms)
		{
			const FTransformSRT3d RelativePositionTransform(FQuaterniond(RotationSettings->StartRotation), FVector3d::ZeroVector, ScaleSettings->StartScale);
			BoundingBoxVisualizer.PushTransform(FTransform(RelativePositionTransform.TransformVector(Element.RelativePosition)));
		}
	
		for (int32 k = 0; k < CurrentPattern.Num(); ++k)
		{
			BoundingBoxVisualizer.PushTransform(FTransform(CurrentPattern[k].GetTranslation()) * CurrentStartFrameWorld.ToFTransform());
			BoundingBoxVisualizer.DrawWireBox(FBox(Element.PatternBounds));
			BoundingBoxVisualizer.PopTransform();
		}

		if (Settings->bUseRelativeTransforms)
		{
			BoundingBoxVisualizer.PopTransform();
		}
	}

	// Render the CombinedPatternBounds along the current pattern with red lines
	BoundingBoxVisualizer.LineColor = FLinearColor::Red;
	for (int32 k = 0; k < CurrentPattern.Num(); ++k)
	{
		BoundingBoxVisualizer.PushTransform(FTransform(CurrentPattern[k].GetTranslation()) * CurrentStartFrameWorld.ToFTransform());
		BoundingBoxVisualizer.DrawWireBox(FBox(CombinedPatternBounds));
		BoundingBoxVisualizer.PopTransform();
	}
	
	BoundingBoxVisualizer.EndFrame();
}

void UPatternTool::OnSourceVisibilityToggled(bool bVisible)
{
	for (int32 k = 0; k < Targets.Num(); ++k)
	{
		UE::ToolTarget::SetSourceObjectVisible(Targets[k], bVisible);
	}
}


void UPatternTool::MarkPatternDirty()
{
	if (bPatternNeedsUpdating == false)
	{
		bPatternNeedsUpdating = true;
		LastPatternUpdateTime = FDateTime::Now();
	}
}




void UPatternTool::InitializeElements()
{
	int32 NumElements = Targets.Num();
	Elements.SetNum(NumElements);

	for (int32 TargetIdx = 0; TargetIdx < NumElements; TargetIdx++)
	{
		FPatternElement& Element = Elements[TargetIdx];
		Element.TargetIndex = TargetIdx;

		Element.SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[TargetIdx]);
		Element.SourceMaterials = UE::ToolTarget::GetMaterialSet(Targets[TargetIdx], false).Materials;
		Element.SourceTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]);
		Element.RelativePosition = Element.SourceTransform.GetTranslation() - Elements[0].SourceTransform.GetTranslation();
		Element.BaseRotateScale = Element.SourceTransform;
		Element.BaseRotateScale.SetTranslation(FVector3d::Zero());		// clear translation from base transform
		Element.SourceTransform.SetRotation(FQuaterniond::Identity());	// clear rotate/scale from source transform, so only location is used
		Element.SourceTransform.SetScale(FVector::One());

		bHaveNonUniformScaleElements = bHaveNonUniformScaleElements || Element.BaseRotateScale.HasNonUniformScale();

		if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(Element.SourceComponent))
		{
			Element.SourceStaticMesh = StaticMeshComp->GetStaticMesh();
			if (Element.SourceStaticMesh != nullptr)
			{
				Element.LocalBounds = Element.SourceStaticMesh->GetBounds().GetBox();
			}
			else
			{
				Element.bValid = false;
			}
		}	
		else if (UDynamicMeshComponent* DynamicMeshComp = Cast<UDynamicMeshComponent>(Element.SourceComponent))
		{
			Element.SourceDynamicMesh = DynamicMeshComp->GetDynamicMesh();
			if (Element.SourceDynamicMesh != nullptr)
			{
				Element.SourceDynamicMesh->ProcessMesh([&](const FDynamicMesh3& Mesh) {
					Element.LocalBounds = Mesh.GetBounds(true);
					});
			}
			else
			{
				Element.bValid = false;
			}

		}
		else
		{
			Element.bValid = false;
		}
		
		Element.PatternBounds = Element.LocalBounds;
	}

	PreviewComponents.SetNum(NumElements);

	ComputeCombinedPatternBounds();
}




void UPatternTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == Settings || PropertySet == OutputSettings)
	{
		return;
	}

	if (PropertySet == ScaleSettings && ScaleSettings->bProportional)
	{
		// We are silencing all watchers if Property corresponds to an FVector UProperty because this indicates that
		// the "Reset to Default" button was pressed. If individual components are modified instead, then Property will
		// not be castable to an FStructProperty
		
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (StructProperty != nullptr && StructProperty->Struct->GetName() == FString("Vector"))
		{
			CachedStartScale = ScaleSettings->StartScale;
			CachedEndScale = ScaleSettings->EndScale;
			CachedJitterScale = ScaleSettings->Jitter;
			
			ScaleSettings->SilentUpdateWatcherAtIndex(StartScaleWatcherIdx);
			ScaleSettings->SilentUpdateWatcherAtIndex(EndScaleWatcherIdx);
			ScaleSettings->SilentUpdateWatcherAtIndex(JitterScaleWatcherIdx);
		}
	}

	OnParametersUpdated();
}


void UPatternTool::OnMainFrameUpdated()
{
	CurrentStartFrameWorld = PlaneMechanic->Plane;

	MarkPatternDirty();
}


void UPatternTool::OnShapeUpdated()
{
	bool bLinearSettings = (Settings->Shape == EPatternToolShape::Line);
	bool bGridSettings = (Settings->Shape == EPatternToolShape::Grid);
	bool bRadialSettings = (Settings->Shape == EPatternToolShape::Circle);

	SetToolPropertySourceEnabled(LinearSettings, bLinearSettings);
	SetToolPropertySourceEnabled(GridSettings, bGridSettings);
	SetToolPropertySourceEnabled(RadialSettings, bRadialSettings);

	// This keeps bUsingSingleAxis correct, fixes up gizmos, and calls OnParametersUpdated
	if (bLinearSettings)
	{
		OnSingleAxisUpdated();
	}
	else if (bGridSettings || bRadialSettings)
	{
		OnSinglePlaneUpdated();
	}
}

void UPatternTool::OnSingleAxisUpdated()
{
	bUsingSingleAxis = true;
	OnParametersUpdated();
	ReconstructTransformGizmos();
}

void UPatternTool::OnSinglePlaneUpdated()
{
	bUsingSingleAxis = false;
	OnParametersUpdated();
	ReconstructTransformGizmos();
}

void UPatternTool::OnSpacingModeUpdated()
{
	bool bBoundingBoxSettings = false;
	
	if (Settings->Shape == EPatternToolShape::Line)
	{
		if (LinearSettings->SpacingMode == EPatternToolAxisSpacingMode::Packed)
		{
			bBoundingBoxSettings = true;
		}
	}
	else if (Settings->Shape == EPatternToolShape::Grid)
	{
		if (GridSettings->SpacingX == EPatternToolAxisSpacingMode::Packed || GridSettings->SpacingY == EPatternToolAxisSpacingMode::Packed)
		{
			bBoundingBoxSettings = true;
		}
	}
	else if (Settings->Shape == EPatternToolShape::Circle)
	{
		if (RadialSettings->SpacingMode == EPatternToolAxisSpacingMode::Packed)
		{
			bBoundingBoxSettings = true;
		}
	}

	SetToolPropertySourceEnabled(BoundingBoxSettings, bBoundingBoxSettings);

	OnParametersUpdated();
}

void UPatternTool::OnParametersUpdated()
{
	MarkPatternDirty();
}

void UPatternTool::SetEnableCreateISMCs(bool bEnable)
{
	bEnableCreateISMCs = bEnable;
	if (OutputSettings)
	{
		OutputSettings->bCreateISMCs &= bEnableCreateISMCs;
		OutputSettings->bEnableCreateISMCs = bEnableCreateISMCs;
	}
}

void UPatternTool::OnTransformGizmoUpdated(UTransformProxy* Proxy, FTransform Transform)
{
	const FVector3d GizmoToToolOrigin = PatternGizmoProxy->GetTransform().GetLocation() - CurrentStartFrameWorld.Origin;
	
	// Recompute extents using current shape, SingleAxis/SinglePlane, and PatternGizmo position
	switch (Settings->Shape)
	{
	case EPatternToolShape::Line:
		LinearSettings->Extent = GizmoToToolOrigin.ProjectOnTo(CurrentStartFrameWorld.GetAxis((int) Settings->SingleAxis)).Length();
		if (LinearSettings->bCentered) { LinearSettings->Extent *= 2.0f; }

		LinearSettings->SilentUpdateWatcherAtIndex(LinearExtentWatcherIdx);
		break;
	case EPatternToolShape::Grid:
		{
		const int32 XIndex = Settings->SinglePlane == EPatternToolSinglePlane::YZPlane ? 1 : 0;
		const int32 YIndex = Settings->SinglePlane == EPatternToolSinglePlane::XYPlane ? 1 : 2;

		GridSettings->ExtentX = GizmoToToolOrigin.ProjectOnTo(CurrentStartFrameWorld.GetAxis(XIndex)).Length();
		if (GridSettings->bCenteredX) { GridSettings->ExtentX *= 2.0f; }

		GridSettings->ExtentY = GizmoToToolOrigin.ProjectOnTo(CurrentStartFrameWorld.GetAxis(YIndex)).Length();
		if (GridSettings->bCenteredY) { GridSettings->ExtentY *= 2.0f; }

		GridSettings->SilentUpdateWatcherAtIndex(GridExtentXWatcherIdx);
		GridSettings->SilentUpdateWatcherAtIndex(GridExtentYWatcherIdx);
		break;
		}
	case EPatternToolShape::Circle:
		RadialSettings->Radius = GizmoToToolOrigin.Length();	// This is actually the simplest case because of how conveniently a circle is defined
		
		RadialSettings->SilentUpdateWatcherAtIndex(RadiusWatcherIdx);
		break;
	}
	
	MarkPatternDirty();
}

void UPatternTool::ResetTransformGizmoPosition()
{
	if (!PatternGizmo || !ensure(PatternGizmoProxy))
	{
		return;
	}

	// There are two ways we can deal with our transform, particularly the rotation. Either we can
	//  keep the sub gizmo unrotated in parent gizmo space, and change which axes we use on the gizmo,
	//  or we can have the gizmo operate on the same axis/axes and change the rotation here to align
	//  that axis in the proper direction.
	// For now we do the latter. 

	FVector3d OffsetFromOrigin = FVector3d::ZeroVector;
	double DistanceFromOriginX = 0.0f;
	double DistanceFromOriginY = 0.0f;
	bool bDistancesSet = false;

	// The gizmo is always restricted to moving in its local x-axis or xy-plane even when other axes or planes
	// are used in the tool. This rotation correctly adjusts the gizmo's local frame such that the local x-axis
	// or xy-plane correspond to the proper directions in the frame of the tool.
	FRotator RotationInLocalSpace = FRotator::ZeroRotator;
	
	switch (Settings->Shape)
	{
	case EPatternToolShape::Line:
		
		DistanceFromOriginX = LinearSettings->Extent;
		if (LinearSettings->bCentered) { DistanceFromOriginX /= 2.0f; }

		if (Settings->SingleAxis == EPatternToolSingleAxis::XAxis)
		{
			OffsetFromOrigin = FVector3d(DistanceFromOriginX, 0, 0);
		}
		else if (Settings->SingleAxis == EPatternToolSingleAxis::YAxis)
		{
			OffsetFromOrigin = FVector3d(0, DistanceFromOriginX, 0);
			RotationInLocalSpace.Add(0, 90, 0);
		}
		else if (Settings->SingleAxis == EPatternToolSingleAxis::ZAxis)
		{
			OffsetFromOrigin = FVector3d(0, 0, DistanceFromOriginX);
			RotationInLocalSpace.Add(90, 0, 0);
		}
		
		break;

	case EPatternToolShape::Grid:
		
		DistanceFromOriginX = GridSettings->ExtentX;
		if (GridSettings->bCenteredX) { DistanceFromOriginX /= 2.0f; }

		DistanceFromOriginY = GridSettings->ExtentY;
		if (GridSettings->bCenteredY) { DistanceFromOriginY /= 2.0f; }
		
		bDistancesSet = true;
		
		// FALLTHROUGH TO NEXT CASE
		
	case EPatternToolShape::Circle:
		
		if (!bDistancesSet)
		{
			DistanceFromOriginX = RadialSettings->Radius;
			DistanceFromOriginY = 0.0f;
		}

		if (Settings->SinglePlane == EPatternToolSinglePlane::XYPlane)
		{
			OffsetFromOrigin = FVector3d(DistanceFromOriginX, DistanceFromOriginY, 0);
		}
		else if (Settings->SinglePlane == EPatternToolSinglePlane::XZPlane)
		{
			OffsetFromOrigin = FVector3d(DistanceFromOriginX, 0, DistanceFromOriginY);
			RotationInLocalSpace.Add(0, 0, 90);
		}
		else if (Settings->SinglePlane == EPatternToolSinglePlane::YZPlane)
		{
			OffsetFromOrigin = FVector3d(0, DistanceFromOriginX, DistanceFromOriginY);
			RotationInLocalSpace.Add(90, 0, 0);
		}
		
		break;
	}

	const FVector3d ProxyTranslation = CurrentStartFrameWorld.Origin + FRotator(CurrentStartFrameWorld.Rotation).RotateVector(OffsetFromOrigin);

	TGuardValue<bool> SetPivotGuard(PatternGizmoProxy->bSetPivotMode, true);
	PatternGizmoProxy->SetTransform(FTransform(RotationInLocalSpace, OffsetFromOrigin) * CurrentStartFrameWorld.ToFTransform());
}

void UPatternTool::ReconstructTransformGizmos()
{
	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	if (PatternGizmo)
	{
		GizmoManager->DestroyGizmo(PatternGizmo);
	}

	// There are a couple ways to create a sub gizmo here. The proper way is to add the extra element individually,
	//  as we do here, but it could have been done with a TRS subgizmo.

	UE::GizmoUtil::FTransformSubGizmoCommonParams Params;
	Params.Component = PatternGizmoComponent;
	Params.TransformProxy = PatternGizmoProxy;
	Params.bManipulatesRootComponent = false;
	Params.bAxisIsBasedOnRootComponent = false;

	if (bUsingSingleAxis)
	{
		UAxisPositionGizmo* AxisGizmo = UE::GizmoUtil::CreateGizmoViaSimpleBuilder<UAxisPositionGizmo>(
			GetToolManager()->GetPairedGizmoManager(), FString(), this);
		Params.OuterForSubobjects = AxisGizmo;
		Params.Axis = EAxis::X;
		AxisGizmo->InitializeAsTranslateGizmo(Params,
			// No shared state
			nullptr);
		PatternGizmo = AxisGizmo;
	}
	else
	{
		UPlanePositionGizmo* PlaneGizmo = UE::GizmoUtil::CreateGizmoViaSimpleBuilder<UPlanePositionGizmo>(
			GetToolManager()->GetPairedGizmoManager(), FString(), this);
		Params.OuterForSubobjects = PlaneGizmo;
		Params.Axis = EAxis::Z;
		PlaneGizmo->InitializeAsTranslateGizmo(Params,
			// No shared state
			nullptr);
		PatternGizmo = PlaneGizmo;
	}

	ResetTransformGizmoPosition();
}


void UPatternTool::ResetPreviews()
{
	int32 NumElements = Elements.Num();
	if (PreviewComponents.Num() != NumElements)
	{
		return;
	}

	for (int32 ElemIdx = 0; ElemIdx < NumElements; ++ElemIdx)
	{
		FPatternElement& Element = Elements[ElemIdx];
		if (Element.bValid)
		{
			FComponentSet& ElemComponents = PreviewComponents[ElemIdx];
			if (Element.SourceStaticMesh)
			{
				ReturnStaticMeshes(Element, ElemComponents);
			}
			else if (Element.SourceDynamicMesh != nullptr)
			{
				ReturnDynamicMeshes(Element, ElemComponents);
			}
		}
	}
}



static void InitializeGenerator(FPatternGenerator& Generator, UPatternTool* Tool)
{
	// The way this is seeded is kind of arbitrary but is reliable for the purpose of deterministically
	// providing each type of jitter a random stream independent of the others given a single input seed.
	Generator.RotationRandomStream.Initialize(Tool->Settings->Seed);
	Generator.ScaleRandomStream.Initialize(Generator.RotationRandomStream.GetUnsignedInt());
	Generator.TranslationRandomStream.Initialize(Generator.RotationRandomStream.GetUnsignedInt());
	
	Generator.bInterpolateRotation = Tool->RotationSettings->bInterpolate;
	Generator.bJitterRotation = Tool->RotationSettings->bJitter;
	Generator.RotationJitterRange = Tool->RotationSettings->Jitter;
	if (Generator.bInterpolateRotation)
	{
		Generator.StartRotation = FQuaterniond(Tool->RotationSettings->StartRotation);
		Generator.EndRotation = FQuaterniond(Tool->RotationSettings->EndRotation);
	}
	else
	{
		Generator.StartRotation = Generator.EndRotation = FQuaterniond(Tool->RotationSettings->StartRotation);
	}

	Generator.bInterpolateTranslation = Tool->TranslationSettings->bInterpolate;
	Generator.bJitterTranslation = Tool->TranslationSettings->bJitter;
	Generator.TranslationJitterRange = Tool->TranslationSettings->Jitter;
	if (Generator.bInterpolateTranslation)
	{
		Generator.StartTranslation = Tool->TranslationSettings->StartTranslation;
		Generator.EndTranslation = Tool->TranslationSettings->EndTranslation;
	}
	else
	{
		Generator.StartTranslation = Generator.EndTranslation = Tool->TranslationSettings->StartTranslation;
	}

	Generator.bInterpolateScale = Tool->ScaleSettings->bInterpolate;
	Generator.bJitterScale = Tool->ScaleSettings->bJitter;
	Generator.ScaleJitterRange = Tool->ScaleSettings->Jitter;
	if (Generator.bInterpolateScale)
	{
		Generator.StartScale = Tool->ScaleSettings->StartScale;
		Generator.EndScale = Tool->ScaleSettings->EndScale;
	}
	else
	{
		Generator.StartScale = Generator.EndScale = Tool->ScaleSettings->StartScale;
	}
}



void UPatternTool::GetPatternTransforms_Linear(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut)
{
	FLinearPatternGenerator Generator;
	InitializeGenerator(Generator, this);
	
	ComputeCombinedPatternBounds();
	Generator.Dimensions = CombinedPatternBounds;
	
	double ExtentX = LinearSettings->Extent;

	Generator.StartFrame = FFrame3d();
	Generator.Axis = (int32)Settings->SingleAxis;
	FVector3d AxisX = Generator.StartFrame.GetAxis(Generator.Axis);

	if (LinearSettings->bCentered)
	{
		Generator.StartFrame.Origin -= 0.5 * ExtentX * AxisX;
	}

	Generator.EndFrame = Generator.StartFrame;
	Generator.EndFrame.Origin += ExtentX * AxisX;

	Generator.FillMode = FLinearPatternGenerator::EFillMode::LineFill;

	Generator.SpacingMode = (FLinearPatternGenerator::ESpacingMode)(int)LinearSettings->SpacingMode;
	Generator.Count = LinearSettings->Count;
	Generator.StepSize = LinearSettings->StepSize;

	Generator.UpdatePattern();
	TransformsOut = MoveTemp(Generator.Pattern);
}



void UPatternTool::GetPatternTransforms_Grid(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut)
{
	FLinearPatternGenerator Generator;
	InitializeGenerator(Generator, this);
	
	ComputeCombinedPatternBounds();
	Generator.Dimensions = CombinedPatternBounds;
	
	double ExtentX = GridSettings->ExtentX;
	double ExtentY = GridSettings->ExtentY;

	Generator.StartFrame = FFrame3d();
	switch (Settings->SinglePlane)
	{
	case EPatternToolSinglePlane::XYPlane:
		Generator.Axis = 0; Generator.AxisY = 1; break;
	case EPatternToolSinglePlane::XZPlane:
		Generator.Axis = 0; Generator.AxisY = 2; break;
	case EPatternToolSinglePlane::YZPlane:
		Generator.Axis = 1; Generator.AxisY = 2; break;
	}
	FVector3d AxisX = Generator.StartFrame.GetAxis(Generator.Axis);
	FVector3d AxisY = Generator.StartFrame.GetAxis(Generator.AxisY);

	if (GridSettings->bCenteredX)
	{
		Generator.StartFrame.Origin -= 0.5 * ExtentX * AxisX;
	}
	if (GridSettings->bCenteredY)
	{
		Generator.StartFrame.Origin -= 0.5 * ExtentY * AxisY;
	}

	Generator.EndFrame = Generator.StartFrame;
	Generator.EndFrame.Origin += ExtentX * AxisX + ExtentY * AxisY;

	Generator.FillMode = FLinearPatternGenerator::EFillMode::RectangleFill;

	Generator.SpacingMode = (FLinearPatternGenerator::ESpacingMode)(int)GridSettings->SpacingX;
	Generator.Count = GridSettings->CountX;
	Generator.StepSize = GridSettings->StepSizeX;

	Generator.SpacingModeY = (FLinearPatternGenerator::ESpacingMode)(int)GridSettings->SpacingY;
	Generator.CountY = GridSettings->CountY;
	Generator.StepSizeY = GridSettings->StepSizeY;

	Generator.UpdatePattern();
	TransformsOut = MoveTemp(Generator.Pattern);
}




void UPatternTool::GetPatternTransforms_Radial(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut)
{
	FRadialPatternGenerator Generator;
	InitializeGenerator(Generator, this);

	ComputeCombinedPatternBounds();
	Generator.Dimensions = CombinedPatternBounds;
	
	Generator.Radius = RadialSettings->Radius;
	
	// Orient the CenterFrame based on the plane setting of the tool & determine which axis is used if bOriented is true
	const FVector3d Origin(0, 0, 0);
	switch (Settings->SinglePlane)
	{
	case EPatternToolSinglePlane::XYPlane:
		Generator.CenterFrame = FFrame3d(Origin, FVector3d(0, 0, 1));
		Generator.AxisIndexToAlign = 0;
		break;
	case EPatternToolSinglePlane::XZPlane:
		Generator.CenterFrame = FFrame3d(Origin, FVector3d(0, 1, 0));
		Generator.AxisIndexToAlign = 2;
		break;
	case EPatternToolSinglePlane::YZPlane:
		Generator.CenterFrame = FFrame3d(Origin, FVector3d(1, 0, 0));
		Generator.AxisIndexToAlign = 2;
		break;
	default:
		Generator.CenterFrame = FFrame3d();
	}
	
	Generator.StartAngleDeg = RadialSettings->StartAngle;
	Generator.EndAngleDeg = RadialSettings->EndAngle;
	Generator.AngleShift = RadialSettings->AngleShift;
	
	Generator.bOriented = RadialSettings->bOriented;

	Generator.SpacingMode = (FRadialPatternGenerator::ESpacingMode)(int)RadialSettings->SpacingMode;

	Generator.FillMode = FRadialPatternGenerator::EFillMode::CircleFill;
	
	Generator.Count = RadialSettings->Count;
	Generator.StepSizeDeg = RadialSettings->StepSize;

	Generator.UpdatePattern();
	TransformsOut = MoveTemp(Generator.Pattern);
}

void UPatternTool::UpdatePattern()
{
	TArray<FTransformSRT3d> Pattern;
	
	if (Settings->Shape == EPatternToolShape::Line)
	{
		GetPatternTransforms_Linear(Pattern);
	}
	if (Settings->Shape == EPatternToolShape::Grid)
	{
		GetPatternTransforms_Grid(Pattern);
	}
	else if (Settings->Shape == EPatternToolShape::Circle)
	{
		GetPatternTransforms_Radial(Pattern);
	}
	
	// Return all current preview components in use to the preview component pool
	ResetPreviews();

	int32 NumPatternItems = Pattern.Num();
	int32 NumElements = Elements.Num();
	check(PreviewComponents.Num() == NumElements);

	auto AddComponent = [this, &Pattern](int32 PatternItemIdx, const FPatternElement& Element, const FTransformSRT3d& ElementTransform, FComponentSet& ElemComponents)
	{
		UPrimitiveComponent* Component = nullptr;
		if (Element.SourceDynamicMesh != nullptr)
		{
			Component = GetPreviewDynamicMesh(Element);
		}
		else if (Element.SourceStaticMesh != nullptr)
		{
			Component = GetPreviewStaticMesh(Element);
		}
		if (Component != nullptr)
		{
			FTransform WorldTransform;
			ComputeWorldTransform(WorldTransform, ElementTransform, Pattern[PatternItemIdx]);

			Component->SetWorldTransform(WorldTransform);
			Component->SetVisibility(true);

			ElemComponents.Components.Add(Component);
		}
	};

	if (Settings->bRandomlyPickElements)
	{
		FRandomStream Stream(Settings->Seed);
		for (int32 k = 0; k < NumPatternItems; ++k)
		{
			int32 ElemIdx = Stream.RandHelper(NumElements);
			FPatternElement& Element = Elements[ElemIdx];
			FTransformSRT3d ElementTransform = Element.BaseRotateScale;
			FComponentSet& ElemComponents = PreviewComponents[ElemIdx];

			if (Settings->bUseRelativeTransforms)
			{
				ElementTransform.SetTranslation(ElementTransform.GetTranslation() + Element.RelativePosition);
			}

			AddComponent(k, Element, ElementTransform, ElemComponents);
		}
	}
	else // always pick all elements
	{
		for (int32 ElemIdx = 0; ElemIdx < NumElements; ++ElemIdx)
		{
			FPatternElement& Element = Elements[ElemIdx];
			FTransformSRT3d ElementTransform = Element.BaseRotateScale;
			FComponentSet& ElemComponents = PreviewComponents[ElemIdx];

			if (Settings->bUseRelativeTransforms)
			{
				ElementTransform.SetTranslation(ElementTransform.GetTranslation() + Element.RelativePosition);
			}

			for (int32 k = 0; k < NumPatternItems; ++k)
			{
				AddComponent(k, Element, ElementTransform, ElemComponents);
			}
		}
	}


	// ResetPreviews() does not hide the preview components because changing their visibility
	// will destroy their SceneProxies, which is expensive, and in most cases the same set of
	// objects will immediately be shown, re-creating the proxy. So instead they are just all
	// left visible, and HideReturnedPreviewMeshes() fixes up the visibility on any Components
	// returned to the pools that were not immediately re-used
	HideReturnedPreviewMeshes();

	CurrentPattern = MoveTemp(Pattern);
}

void UPatternTool::ComputeWorldTransform(FTransform& OutWorldTransform, const FTransform& InElementTransform, const FTransform& InPatternTransform) const
{
	OutWorldTransform = InElementTransform * InPatternTransform * CurrentStartFrameWorld.ToFTransform();
	
	if (Settings->bProjectElementsDown)
	{
		FVector3d ProjectionAxis = -CurrentStartFrameWorld.Z();
		const FRay WorldRay(OutWorldTransform.GetTranslation(), ProjectionAxis);
		FHitResult HitResult;
	
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, HitResult, WorldRay, &AllPreviewComponents, nullptr))
		{
			FVector3d ProjectionTranslation = ProjectionAxis * (HitResult.Distance + Settings->ProjectionOffset);
			OutWorldTransform.SetTranslation(OutWorldTransform.GetTranslation() + ProjectionTranslation);
		}
	}
}

void UPatternTool::ComputePatternBounds(int32 ElemIdx)
{
	FPatternElement& Element = Elements[ElemIdx];
	FTransformSRT3d Transform = Element.BaseRotateScale;
	
	if (!BoundingBoxSettings->bIgnoreTransforms)
	{
		Transform = (FTransform) FTransformSRT3d(FQuaterniond(RotationSettings->StartRotation), FVector3d::ZeroVector, ScaleSettings->StartScale) * Transform;
	}

	Element.PatternBounds = FAxisAlignedBox3d(Element.LocalBounds, Transform);
}

void UPatternTool::ComputeCombinedPatternBounds()
{
	// CombinedPatternBounds is the bounding box that contains every pattern element and is computed by setting it
	// to the first element's pattern bounds and then expanding the box to contain each following element.

	ComputePatternBounds(0);
	CombinedPatternBounds = Elements[0].PatternBounds;

	// If StartScale or StartRotation are not zero vectors, then Element.RelativePosition must be transformed to be accurate if bUseRelativeTransforms is true
	const FTransformSRT3d RelativePositionTransform(FQuaterniond(RotationSettings->StartRotation), FVector3d::ZeroVector, ScaleSettings->StartScale);
	
	for (int32 ElemIdx = 1; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		const FPatternElement& Element = Elements[ElemIdx];
		ComputePatternBounds(ElemIdx);
		
		if (Settings->bUseRelativeTransforms)
		{
			CombinedPatternBounds.Contain(Element.PatternBounds.Min + RelativePositionTransform.TransformVector(Element.RelativePosition));
			CombinedPatternBounds.Contain(Element.PatternBounds.Max + RelativePositionTransform.TransformVector(Element.RelativePosition));
		}
		else
		{
			CombinedPatternBounds.Contain(Element.PatternBounds);
		}
	}

	// The user can manually adjust the box if desired to fine tune packed behavior
	CombinedPatternBounds.Expand(BoundingBoxSettings->Adjustment);
}

UStaticMeshComponent* UPatternTool::GetPreviewStaticMesh(const FPatternElement& Element)
{
	FComponentSet* FoundPool = StaticMeshPools.Find(Element.TargetIndex);
	if (FoundPool == nullptr)
	{
		StaticMeshPools.Add(Element.TargetIndex, FComponentSet{});
		FoundPool = StaticMeshPools.Find(Element.TargetIndex);
		check(FoundPool != nullptr);
	}

	if (FoundPool->Components.Num() > 0)
	{
		UStaticMeshComponent* FoundComponent = Cast<UStaticMeshComponent>(FoundPool->Components.Pop(EAllowShrinking::No));
		check(FoundComponent != nullptr);
		return FoundComponent;
	}

	UStaticMeshComponent* StaticMeshComp = NewObject<UStaticMeshComponent>(PreviewGeometry->GetActor());

	StaticMeshComp->SetStaticMesh(Element.SourceStaticMesh);
	for (int32 j = 0; j < Element.SourceMaterials.Num(); ++j)
	{
		StaticMeshComp->SetMaterial(j, Element.SourceMaterials[j]);
	}

	StaticMeshComp->SetupAttachment(PreviewGeometry->GetActor()->GetRootComponent());
	StaticMeshComp->RegisterComponent();

	AllComponents.Add(StaticMeshComp);
	AllPreviewComponents.Add(StaticMeshComp);

	return StaticMeshComp;
}

void UPatternTool::ReturnStaticMeshes(FPatternElement& Element, FComponentSet& ComponentSet)
{
	if (ComponentSet.Components.Num() == 0) return;

	FComponentSet* FoundPool = StaticMeshPools.Find(Element.TargetIndex);
	check(FoundPool != nullptr);		// should never happen as we would have created in GetPreviewStaticMesh()

	for (UPrimitiveComponent* Component : ComponentSet.Components)
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			FoundPool->Components.Add(StaticMeshComponent);
		}
	}

	ComponentSet.Components.Reset();
}





UDynamicMeshComponent* UPatternTool::GetPreviewDynamicMesh(const FPatternElement& Element)
{
	FComponentSet* FoundPool = DynamicMeshPools.Find(Element.TargetIndex);
	if (FoundPool == nullptr)
	{
		DynamicMeshPools.Add(Element.TargetIndex, FComponentSet{});
		FoundPool = DynamicMeshPools.Find(Element.TargetIndex);
		check(FoundPool != nullptr);
	}

	if (FoundPool->Components.Num() > 0)
	{
		UDynamicMeshComponent* FoundComponent = Cast<UDynamicMeshComponent>(FoundPool->Components.Pop(EAllowShrinking::No));
		check(FoundComponent != nullptr);
		return FoundComponent;
	}

	UDynamicMeshComponent* DynamicMeshComp = NewObject<UDynamicMeshComponent>(PreviewGeometry->GetActor());

	DynamicMeshComp->SetGenerateOverlapEvents(false);
	DynamicMeshComp->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	DynamicMeshComp->CollisionType = ECollisionTraceFlag::CTF_UseSimpleAsComplex;

	DynamicMeshComp->ConfigureMaterialSet(Element.SourceMaterials);
	DynamicMeshComp->SetupAttachment(PreviewGeometry->GetActor()->GetRootComponent());
	DynamicMeshComp->RegisterComponent();

	FDynamicMesh3 ElementMeshCopy(Element.SourceDynamicMesh->GetMeshRef());
	DynamicMeshComp->SetMesh(MoveTemp(ElementMeshCopy));
	
	AllComponents.Add(DynamicMeshComp);
	AllPreviewComponents.Add(DynamicMeshComp);

	return DynamicMeshComp;
}

void UPatternTool::ReturnDynamicMeshes(FPatternElement& Element, FComponentSet& ComponentSet)
{
	if (ComponentSet.Components.Num() == 0) return;

	FComponentSet* FoundPool = DynamicMeshPools.Find(Element.TargetIndex);
	check(FoundPool != nullptr);		// should never happen as we would have created in GetPreviewStaticMesh()

	for (UPrimitiveComponent* Component : ComponentSet.Components)
	{
		if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
		{
			FoundPool->Components.Add(DynamicMeshComponent);
		}
	}

	ComponentSet.Components.Reset();
}



void UPatternTool::HideReturnedPreviewMeshes()
{
	for (TPair<int32, FComponentSet> Pair : StaticMeshPools)
	{
		for (UPrimitiveComponent* Component : Pair.Value.Components)
		{
			if (Component->IsVisible())
			{
				Component->SetVisibility(false);
			}
		}
	}
	for (TPair<int32, FComponentSet> Pair : DynamicMeshPools)
	{
		for (UPrimitiveComponent* Component : Pair.Value.Components)
		{
			if (Component->IsVisible())
			{
				Component->SetVisibility(false);
			}
		}
	}
}




void UPatternTool::EmitResults()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("CreatePattern", "Create Pattern"));

	bool bSeparateActorPerItem = OutputSettings->bSeparateActors;
	bool bCreateISMForStaticMeshes = OutputSettings->bCreateISMCs;
	bool bConvertToDynamic = OutputSettings->bConvertToDynamic;
	const UModelingComponentsSettings* ModelingSettings = GetDefault<UModelingComponentsSettings>();
	
	int32 NumElements = Elements.Num();
	int32 NumPatternItems = CurrentPattern.Num();

	TArray<AActor*> NewActors;		// set of new actors created by operation

	// Used when constructing actors, not appending components to an actor. Otherwise UPrimitiveComponent::SetVisibility()
	// is used instead. Keeping everything invisible until the end allows us to avoid hitting the created components
	// when projecting downward, just like we do in the previews via ignored components.
	constexpr bool bPropagateToChildren = true;
	auto SetActorComponentsVisibility = [bPropagateToChildren](AActor* InActor, const bool bNewVisibility)
	{
		for (UActorComponent* Component : InActor->GetComponents())
		{
			if (UPrimitiveComponent* ActorComponentAsPrimitiveComponent = Cast<UPrimitiveComponent, UActorComponent>(Component))
			{
				ActorComponentAsPrimitiveComponent->SetVisibility(bNewVisibility, bPropagateToChildren);
			}
		}
	};

	// TODO: investigate use of CopyPropertiesForUnrelatedObjects to transfer settings from source to target Components/Actors

	for (int32 ElemIdx = 0; ElemIdx < NumElements; ++ElemIdx)
	{
		FPatternElement& Element = Elements[ElemIdx];
		if (Element.bValid == false)
		{
			continue;
		}
		FTransformSRT3d ElementTransform = Element.BaseRotateScale;

		if (Settings->bUseRelativeTransforms)
		{
			ElementTransform.SetTranslation(ElementTransform.GetTranslation() + Element.RelativePosition);
		}
		
		if (Element.SourceDynamicMesh != nullptr || bConvertToDynamic )
		{
			static FGetMeshParameters GetMeshParams;
			GetMeshParams.bWantMeshTangents = true;
			FDynamicMesh3 ElementMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[ElemIdx], GetMeshParams);

			// this lambda creates a new dynamic mesh actor w/ the materials from the current Element
			auto EmitDynamicMeshActor = [this, &Element, ModelingSettings](FDynamicMesh3&& MoveMesh, FString BaseName, FTransformSRT3d Transform)
			{
				FCreateMeshObjectParams NewMeshObjectParams;
				NewMeshObjectParams.TargetWorld = GetTargetWorld();
				NewMeshObjectParams.Transform = (FTransform)Transform;
				NewMeshObjectParams.BaseName = BaseName;
				NewMeshObjectParams.Materials = Element.SourceMaterials;
				NewMeshObjectParams.SetMesh(MoveTemp(MoveMesh));
				NewMeshObjectParams.TypeHint = ECreateObjectTypeHint::DynamicMeshActor;

				NewMeshObjectParams.bEnableCollision = ModelingSettings->bEnableCollision;
				NewMeshObjectParams.CollisionMode = ModelingSettings->CollisionMode;
				NewMeshObjectParams.bEnableRaytracingSupport = ModelingSettings->bEnableRayTracing;

				FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
				return Result;
			};


			if (bSeparateActorPerItem)
			{
				for (int32 k = 0; k < NumPatternItems; ++k)
				{
					FTransformSRT3d PatternTransform = CurrentPattern[k];
					FTransform WorldTransform;
					ComputeWorldTransform(WorldTransform, (FTransform)ElementTransform, (FTransform)PatternTransform);
					
					FDynamicMesh3 PatternItemMesh = ElementMesh;
					// TODO: may need to bake nonuniform scale in here, if we allow scaling in transform
					FCreateMeshObjectResult Result = EmitDynamicMeshActor(MoveTemp(PatternItemMesh),
						FString::Printf(TEXT("Pattern_%d_%d"), ElemIdx, k), WorldTransform);
					if (Result.IsOK())
					{
						NewActors.Add(Result.NewActor);
						SetActorComponentsVisibility(Result.NewActor, false);
					}
				}
			}
			else
			{
				FDynamicMesh3 CombinedPatternMesh;
				CombinedPatternMesh.EnableMatchingAttributes(ElementMesh, true);
				FDynamicMeshEditor Editor(&CombinedPatternMesh);
				FMeshIndexMappings Mappings;
				for (int32 k = 0; k < NumPatternItems; ++k)
				{
					FTransformSRT3d PatternTransform = CurrentPattern[k];
					FTransform WorldTransform;
					ComputeWorldTransform(WorldTransform, (FTransform)ElementTransform, (FTransform)PatternTransform);

					// We have world transforms of the components, but need their local transforms in the space of the
					// final mesh, whose pivot is determined by CurrentStartFrameWorld. So we apply the inverse of that
					// while appending them (the inverse FTransform is accurate in this case because
					// CurrentStartFrameWorld can't be nonuniformly scaled).
					FTransformSequence3d TransformSeq;
					TransformSeq.Append(WorldTransform);
					TransformSeq.Append(CurrentStartFrameWorld.ToFTransform().Inverse());
					Mappings.Reset();
					Editor.AppendMesh(&ElementMesh, Mappings,
						[&](int, const FVector3d& Position) { return TransformSeq.TransformPosition(Position); },
						[&](int, const FVector3d& Normal) { return TransformSeq.TransformNormal(Normal); } );
				}
				
				FCreateMeshObjectResult Result = EmitDynamicMeshActor(MoveTemp(CombinedPatternMesh),
				FString::Printf(TEXT("Pattern_%d"), ElemIdx), CurrentStartFrameWorld.ToFTransform());
				if (Result.IsOK())
				{
					NewActors.Add(Result.NewActor);
					SetActorComponentsVisibility(Result.NewActor, false);
				}
			}
		}
		else if (Element.SourceStaticMesh != nullptr)
		{
			UStaticMeshComponent* SourceComponent = Cast<UStaticMeshComponent>(Elements[ElemIdx].SourceComponent);
			
			if (bSeparateActorPerItem)
			{
				for (int32 k = 0; k < NumPatternItems; ++k)
				{
					FTransformSRT3d PatternTransform = CurrentPattern[k];
					FTransform WorldTransform;
					ComputeWorldTransform(WorldTransform, (FTransform)ElementTransform, (FTransform)PatternTransform);

					FCreateActorParams SpawnInfo;
					SpawnInfo.BaseName = FString::Printf(TEXT("Pattern_%d_%d"), ElemIdx, k);
					SpawnInfo.TargetWorld = GetTargetWorld();
					SpawnInfo.TemplateAsset = SourceComponent->GetStaticMesh();
					SpawnInfo.Transform = WorldTransform;

					FCreateActorResult Result = UE::Modeling::CreateNewActor(GetToolManager(), MoveTemp(SpawnInfo));
					if (AStaticMeshActor* NewStaticMeshActor = Cast<AStaticMeshActor>(Result.NewActor); Result.IsOK() && NewStaticMeshActor)
					{
						UStaticMeshComponent* NewStaticMeshComponent = NewStaticMeshActor->GetStaticMeshComponent();

						// Ensure the new static mesh component is properly configured based on the source component
						NewStaticMeshComponent->SetStaticMesh(SourceComponent->GetStaticMesh());
						NewStaticMeshComponent->SetWorldTransform(WorldTransform);
						for (int32 j = 0; j < Element.SourceMaterials.Num(); ++j)
						{
							NewStaticMeshComponent->SetMaterial(j, Element.SourceMaterials[j]);
						}

						NewActors.Add(NewStaticMeshActor);
						SetActorComponentsVisibility(NewStaticMeshActor, false);
					}
				}
			}
			else if (bCreateISMForStaticMeshes)
			{
				FActorSpawnParameters SpawnInfo;
				AActor* NewActor = GetTargetWorld()->SpawnActor<AActor>(SpawnInfo);
				if (NewActor != nullptr)
				{
					NewActors.Add(NewActor);

					UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(NewActor);
					ISMComponent->SetVisibility(false, bPropagateToChildren);
					ISMComponent->SetFlags(RF_Transactional);
					ISMComponent->bHasPerInstanceHitProxies = true;

					ISMComponent->SetStaticMesh(Element.SourceStaticMesh);
					for (int32 j = 0; j < Element.SourceMaterials.Num(); ++j)
					{
						ISMComponent->SetMaterial(j, Element.SourceMaterials[j]);
					}

					NewActor->SetRootComponent(ISMComponent);
					ISMComponent->OnComponentCreated();
					NewActor->AddInstanceComponent(ISMComponent);

					NewActor->SetActorTransform(CurrentStartFrameWorld.ToFTransform());

					for (int32 k = 0; k < NumPatternItems; ++k)
					{
						FTransformSRT3d PatternTransform = CurrentPattern[k];
						FTransform WorldTransform;
						ComputeWorldTransform(WorldTransform, (FTransform)ElementTransform, (FTransform)PatternTransform);

						constexpr bool bTransformInWorldSpace = true;
						ISMComponent->AddInstance(WorldTransform, bTransformInWorldSpace);
					}

					ISMComponent->RegisterComponent();
				}
			}
			else
			{
				AStaticMeshActor* NewStaticMeshActor = nullptr;
				UStaticMeshComponent* TemplateStaticMeshComponent = nullptr;

				for (int32 k = 0; k < NumPatternItems; ++k)
				{
					FTransformSRT3d PatternTransform = CurrentPattern[k];
					FTransform WorldTransform;
					ComputeWorldTransform(WorldTransform, (FTransform)ElementTransform, (FTransform)PatternTransform);

					if (k == 0)
					{
						FCreateActorParams SpawnInfo;
						SpawnInfo.BaseName = FString::Printf(TEXT("Pattern_%d"), ElemIdx);
						SpawnInfo.TargetWorld = GetTargetWorld();
						SpawnInfo.TemplateAsset = SourceComponent->GetStaticMesh();
						SpawnInfo.Transform = FTransform::Identity;

						FCreateActorResult Result = UE::Modeling::CreateNewActor(GetToolManager(), MoveTemp(SpawnInfo));
						if (NewStaticMeshActor = Cast<AStaticMeshActor>(Result.NewActor); Result.IsOK() && NewStaticMeshActor)
						{
							TemplateStaticMeshComponent = NewStaticMeshActor->GetStaticMeshComponent();

							// Ensure the new static mesh component is properly configured based on the source component
							TemplateStaticMeshComponent->SetStaticMesh(SourceComponent->GetStaticMesh());
							TemplateStaticMeshComponent->SetWorldTransform(WorldTransform);
							for (int32 j = 0; j < Element.SourceMaterials.Num(); ++j)
							{
								TemplateStaticMeshComponent->SetMaterial(j, Element.SourceMaterials[j]);
							}

							
							NewActors.Add(NewStaticMeshActor);
							SetActorComponentsVisibility(NewStaticMeshActor, false);
						}
						else
						{
							break;
						}
					}
					else
					{
						// Create new component based on TemplateStaticMeshComponent
						UStaticMeshComponent* NewCloneComponent = DuplicateObject<UStaticMeshComponent>(TemplateStaticMeshComponent, NewStaticMeshActor);
						NewCloneComponent->ClearFlags(RF_DefaultSubObject);
						NewCloneComponent->SetupAttachment(TemplateStaticMeshComponent);
						NewCloneComponent->OnComponentCreated();
						NewStaticMeshActor->AddInstanceComponent(NewCloneComponent);
						NewCloneComponent->RegisterComponent();
						NewCloneComponent->SetWorldTransform(WorldTransform);

						// Using SetVisibility here instead of SetActorComponentsVisibility because in this particular output
						// case with large patterns, there could be a lot of redundant iteration over and casting for components
						// which have already been set to invisible.
						NewCloneComponent->SetVisibility(false, bPropagateToChildren);
					}
				}
			}
		}
	}

	// Make all components visible now that they have all been created
	for (auto Actor : NewActors)
	{
		SetActorComponentsVisibility(Actor, true);
	}

	if (NewActors.Num() > 0)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewActors);
	}

	GetToolManager()->EndUndoTransaction();
}



#undef LOCTEXT_NAMESPACE
