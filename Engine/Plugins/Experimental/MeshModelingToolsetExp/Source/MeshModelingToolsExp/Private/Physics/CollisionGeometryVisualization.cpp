// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/CollisionGeometryVisualization.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Physics/CollisionPropertySets.h"
#include "Physics/ComponentCollisionUtil.h"
#include "Physics/PhysicsDataCollection.h"
#include "Generators/LineSegmentGenerators.h"
#include "Drawing/PreviewGeometryActor.h"

using namespace UE::Geometry;

namespace
{

void InitializePreviewGeometrySolid(
	const FPhysicsDataCollection& PhysicsData,
	UPreviewGeometry* PreviewGeom,
	UMaterialInterface* SolidMaterial,
	TFunctionRef<FColor(int32 GeoSetIndex)> GeoSetIndexToColorFunc,
	bool bVisible,
	int32 SphereStepsResolution,
	int32 FirstGeoSetIndex)
{
	check(PreviewGeom);
	check(SolidMaterial);

	int32 CircleSteps = FMath::Max(4, SphereStepsResolution);
	int32 GeoSetIndex = FirstGeoSetIndex;

	const FKAggregateGeom& AggGeom = PhysicsData.AggGeom;
	FSimpleCollisionTriangulationSettings TriangulationSettings;
	TriangulationSettings.InitFromSphereResolution(CircleSteps);
	TriangulationSettings.bApproximateLevelSetWithCubes = false;

	UE::Geometry::ConvertSimpleCollisionToDynamicMeshes(
		AggGeom, PhysicsData.ExternalScale3D,
		[&](int32 ShapeIndex, const FKShapeElem& ShapeElem, FDynamicMesh3& Mesh)
		{
			FColor Color = GeoSetIndexToColorFunc(ShapeIndex);
			if (UTriangleSetComponent* TriangleSetComponent = PreviewGeom->CreateOrUpdateTriangleSet(FString::Printf(TEXT("Shape %d"), ShapeIndex), 1, [&](int32 Index, TArray<FRenderableTriangle>& TrisOut)
				{
					Mesh.TriangleCount();
					check(Mesh.HasAttributes());
					const UE::Geometry::FDynamicMeshNormalOverlay* PrimaryNormals = Mesh.Attributes()->PrimaryNormals();
					const UE::Geometry::FDynamicMeshUVOverlay* PrimaryUV = Mesh.Attributes()->PrimaryUV();

					TrisOut.Reserve(Mesh.TriangleCount());
					for (int32 TID : Mesh.TriangleIndicesItr())
					{
						FIndex3i MeshTri = Mesh.GetTriangle(TID);
						FRenderableTriangle& RenderTri = TrisOut.Emplace_GetRef();
						RenderTri.Material = SolidMaterial;
						FRenderableTriangleVertex* TriVerts[3]{ &RenderTri.Vertex0, &RenderTri.Vertex1, &RenderTri.Vertex2 };
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							int32 VID = MeshTri[SubIdx];
							Mesh.GetVertex(VID);
							FVector3f Normal;
							PrimaryNormals->GetTriElement(TID, SubIdx, Normal);
							FVector2f UV(0, 0);
							if (PrimaryUV && PrimaryUV->IsSetTriangle(TID))
							{
								PrimaryUV->GetTriElement(TID, SubIdx, UV);
							}
							TriVerts[SubIdx]->Position = Mesh.GetVertex(VID);
							TriVerts[SubIdx]->Color = Color;
							TriVerts[SubIdx]->Normal = (FVector)Normal;
							TriVerts[SubIdx]->UV = (FVector2D)UV;
						}
					}
				}, Mesh.TriangleCount()))
			{
				TriangleSetComponent->SetVisibility(bVisible);
			}
		},
		TriangulationSettings
	);
}

void InitializePreviewGeometryLines(
	const FPhysicsDataCollection& PhysicsData,
	UPreviewGeometry* PreviewGeom,
	UMaterialInterface* LineMaterial,
	TFunctionRef<FColor(int32 LineSetIndex)> LineSetIndexToColorFunc,
	float LineThickness,
	bool bVisible,
	float DepthBias,
	int32 CircleStepResolution,
	int32 FirstLineSetIndex)
{
	check(PreviewGeom);
	check(LineMaterial);

	int32 CircleSteps = FMath::Max(4, CircleStepResolution);
	int32 LineSetIndex = FirstLineSetIndex;

	const FKAggregateGeom& AggGeom = PhysicsData.AggGeom;

	// spheres are draw as 3 orthogonal circles
	for (int32 Index = 0; Index < AggGeom.SphereElems.Num(); Index++)
	{
		if (ULineSetComponent* LineSetComponent = PreviewGeom->CreateOrUpdateLineSet(FString::Printf(TEXT("Spheres %d"), Index), 1, [&](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
		{
			FColor Color = LineSetIndexToColorFunc(LineSetIndex++);

			const FKSphereElem& Sphere = AggGeom.SphereElems[Index];
			FTransform ElemTransform = Sphere.GetTransform();
			ElemTransform.ScaleTranslation(PhysicsData.ExternalScale3D);
			FTransformSRT3f ElemTransformf(ElemTransform);
			float Radius = PhysicsData.ExternalScale3D.GetAbsMin() * Sphere.Radius;
			UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitY(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitY(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
		}))
		{
			LineSetComponent->SetVisibility(bVisible);
			LineSetComponent->SetLineMaterial(LineMaterial);
		}
	}


	// boxes are drawn as boxes
	for (int32 Index = 0; Index < AggGeom.BoxElems.Num(); Index++)
	{
		if (ULineSetComponent* LineSetComponent = PreviewGeom->CreateOrUpdateLineSet(FString::Printf(TEXT("Boxes %d"), Index), 1, [&](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
		{
			FColor Color = LineSetIndexToColorFunc(LineSetIndex++);

			const FKBoxElem& Box = AggGeom.BoxElems[Index];
			FTransform ElemTransform = Box.GetTransform();
			ElemTransform.ScaleTranslation(PhysicsData.ExternalScale3D);
			FTransformSRT3f ElemTransformf(ElemTransform);
			FVector3f HalfDimensions(
				PhysicsData.ExternalScale3D.X * Box.X * 0.5f,
				PhysicsData.ExternalScale3D.Y * Box.Y * 0.5f,
				PhysicsData.ExternalScale3D.Z * Box.Z * 0.5f);
			UE::Geometry::GenerateBoxSegments<float>(HalfDimensions, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitY(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
		}))
		{
			LineSetComponent->SetVisibility(bVisible);
			LineSetComponent->SetLineMaterial(LineMaterial);
		}
	}


	// capsules are draw as two hemispheres (with 3 intersecting arcs/circles) and connecting lines
	for (int32 Index = 0; Index < AggGeom.SphylElems.Num(); Index++)
	{
		if (ULineSetComponent* LineSetComponent = PreviewGeom->CreateOrUpdateLineSet(FString::Printf(TEXT("Capsules %d"), Index), 1, [&](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
		{
			FColor Color = LineSetIndexToColorFunc(LineSetIndex++);

			const FKSphylElem& Capsule = AggGeom.SphylElems[Index];
			FTransform ElemTransform = Capsule.GetTransform();
			ElemTransform.ScaleTranslation(PhysicsData.ExternalScale3D);
			FTransformSRT3f ElemTransformf(ElemTransform);
			const float HalfLength = Capsule.GetScaledCylinderLength(PhysicsData.ExternalScale3D) * .5f;
			const float Radius = Capsule.GetScaledRadius(PhysicsData.ExternalScale3D);
			FVector3f Top(0, 0, HalfLength), Bottom(0, 0, -HalfLength);

			// top and bottom circles
			UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, Top, FVector3f::UnitX(), FVector3f::UnitY(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, Bottom, FVector3f::UnitX(), FVector3f::UnitY(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });

			// top dome
			UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, PI, Top, FVector3f::UnitY(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, PI, Top, FVector3f::UnitX(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });

			// bottom dome
			UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, -PI, Bottom, FVector3f::UnitY(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, -PI, Bottom, FVector3f::UnitX(), FVector3f::UnitZ(), ElemTransformf,
				[&](const FVector3f& A, const FVector3f& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });

			// connecting lines
			for (int k = 0; k < 2; ++k)
			{
				FVector DX = (k < 1) ? FVector(-Radius, 0, 0) : FVector(Radius, 0, 0);
				LinesOut.Add(FRenderableLine(
					ElemTransform.TransformPosition((FVector)Top + DX),
					ElemTransform.TransformPosition((FVector)Bottom + DX), Color, LineThickness, DepthBias));
				FVector DY = (k < 1) ? FVector(0, -Radius, 0) : FVector(0, Radius, 0);
				LinesOut.Add(FRenderableLine(
					ElemTransform.TransformPosition((FVector)Top + DY),
					ElemTransform.TransformPosition((FVector)Bottom + DY), Color, LineThickness, DepthBias));
			}
		}))
		{
			LineSetComponent->SetVisibility(bVisible);
			LineSetComponent->SetLineMaterial(LineMaterial);
		}
	}

	// convexes are drawn as mesh edges
	for (int32 Index = 0; Index < AggGeom.ConvexElems.Num(); Index++)
	{
		if (ULineSetComponent* LineSetComponent = PreviewGeom->CreateOrUpdateLineSet(FString::Printf(TEXT("Convex %d"), Index), 1, [&](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
		{
			FColor Color = LineSetIndexToColorFunc(LineSetIndex++);

			const FKConvexElem& Convex = AggGeom.ConvexElems[Index];
			FTransform ElemTransform = Convex.GetTransform();
			ElemTransform.ScaleTranslation(PhysicsData.ExternalScale3D);
			ElemTransform.MultiplyScale3D(PhysicsData.ExternalScale3D);
			int32 NumTriangles = Convex.IndexData.Num() / 3;
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				FVector A = ElemTransform.TransformPosition(Convex.VertexData[Convex.IndexData[3 * k]]);
				FVector B = ElemTransform.TransformPosition(Convex.VertexData[Convex.IndexData[3 * k + 1]]);
				FVector C = ElemTransform.TransformPosition(Convex.VertexData[Convex.IndexData[3 * k + 2]]);
				LinesOut.Add(FRenderableLine(A, B, Color, LineThickness, DepthBias));
				LinesOut.Add(FRenderableLine(B, C, Color, LineThickness, DepthBias));
				LinesOut.Add(FRenderableLine(C, A, Color, LineThickness, DepthBias));
			}
		}))
		{
			LineSetComponent->SetVisibility(bVisible);
			LineSetComponent->SetLineMaterial(LineMaterial);
		}
	}

	// for Level Sets draw the grid cells where phi < 0
	for (int32 Index = 0; Index < AggGeom.LevelSetElems.Num(); Index++)
	{
		if (ULineSetComponent* LineSetComponent = PreviewGeom->CreateOrUpdateLineSet(FString::Printf(TEXT("Level Set %d"), Index), 1, [&](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
		{
			FColor Color = LineSetIndexToColorFunc(LineSetIndex++);
			const FKLevelSetElem& LevelSet = AggGeom.LevelSetElems[Index];
			
			FTransform ElemTransform = LevelSet.GetTransform();
			ElemTransform.ScaleTranslation(PhysicsData.ExternalScale3D);
			ElemTransform.MultiplyScale3D(PhysicsData.ExternalScale3D);

			auto GenerateBoxSegmentsFromFBox = [&](const FBox& Box)
			{
				const FVector3d Center = Box.GetCenter();
				const FVector3d HalfDimensions = 0.5 * (Box.Max - Box.Min);

				UE::Geometry::GenerateBoxSegments<double>(HalfDimensions, Center, FVector3d::UnitX(), FVector3d::UnitY(), FVector3d::UnitZ(), ElemTransform,
					[&](const FVector3d& A, const FVector3d& B) { LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, Color, LineThickness, DepthBias)); });
			};

			const FBox TotalGridBox = LevelSet.UntransformedAABB();
			GenerateBoxSegmentsFromFBox(TotalGridBox);

			TArray<FBox> CellBoxes;
			const double Threshold = UE_KINDA_SMALL_NUMBER;		// allow slightly greater than zero for visualization purposes
			LevelSet.GetInteriorGridCells(CellBoxes, Threshold);

			for (const FBox& CellBox : CellBoxes)
			{
				GenerateBoxSegmentsFromFBox(CellBox);
			}

		}))
		{
			LineSetComponent->SetVisibility(bVisible);
			LineSetComponent->SetLineMaterial(LineMaterial);
		}
	}

	// Unclear whether we actually use these in the Engine, for UBodySetup? Does not appear to be supported by UxX import system,
	// and online documentation suggests they may only be supported for cloth?
	ensure(AggGeom.TaperedCapsuleElems.Num() == 0);
}

void UpdatePreviewGeometryForCollision(
	UPreviewGeometry* PartialPreviewGeom,
	UCollisionGeometryVisualizationProperties* Settings,
	int32 FirstColorIndex)
{
	check(PartialPreviewGeom);
	check(Settings);

	int32 LineColorIndex = FirstColorIndex;
	PartialPreviewGeom->UpdateAllLineSets([&](ULineSetComponent* LineSet)
	{
		FColor LineColor = Settings->GetLineSetColor(LineColorIndex++);
		LineSet->SetAllLinesColor(LineColor);
		LineSet->SetAllLinesThickness(Settings->LineThickness);
		LineSet->SetLineMaterial(Settings->GetLineMaterial());
		LineSet->SetVisibility(Settings->bShowCollision);
	});

	int32 TriColorIndex = FirstColorIndex;
	PartialPreviewGeom->UpdateAllTriangleSets([&](UTriangleSetComponent* TriangleSet)
	{
		FColor TriangleColor = Settings->GetTriangleSetColor(TriColorIndex++);
		TriangleSet->SetAllTrianglesColor(TriangleColor);
		TriangleSet->SetAllTrianglesMaterial(Settings->GetSolidMaterial());
		TriangleSet->SetVisibility(Settings->bEnableShowSolid && Settings->bShowCollision && Settings->bShowSolid);
	});
}

} // end namespace






void UE::PhysicsTools::InitializeCollisionGeometryVisualization(
	UPreviewGeometry* PreviewGeom,
	UCollisionGeometryVisualizationProperties* Settings,
	const FPhysicsDataCollection& PhysicsData,
	float DepthBias,
	int32 CircleStepResolution,
	bool bClearExistingLinesAndTriangles)
{
	check(PreviewGeom);
	check(Settings);

	if (bClearExistingLinesAndTriangles)
	{
		PreviewGeom->RemoveAllLineSets();
		PreviewGeom->RemoveAllTriangleSets();
	}
	InitializePreviewGeometryLines(
		PhysicsData,
		PreviewGeom,
		Settings->GetLineMaterial(),
		[&Settings](int LineSetIndex) { return Settings->GetLineSetColor(LineSetIndex); },
		Settings->LineThickness,
		Settings->bShowCollision,
		DepthBias,
		CircleStepResolution,
		0);
	InitializePreviewGeometrySolid(
		PhysicsData, PreviewGeom, Settings->GetSolidMaterial(), [&Settings](int SolidSetIndex) { return Settings->GetTriangleSetColor(SolidSetIndex); },
		Settings->bEnableShowSolid && Settings->bShowCollision && Settings->bShowSolid,
		CircleStepResolution,
		0);

	Settings->bVisualizationDirty = false;
}

void UE::PhysicsTools::UpdateCollisionGeometryVisualization(
	UPreviewGeometry* PreviewGeom,
	UCollisionGeometryVisualizationProperties* Settings)
{
	check(PreviewGeom);
	check(Settings);

	if (Settings->bVisualizationDirty)
	{
		UpdatePreviewGeometryForCollision(PreviewGeom, Settings, 0);
		Settings->bVisualizationDirty = false;
	}
}






void UE::PhysicsTools::PartiallyInitializeCollisionGeometryVisualization(
	UPreviewGeometry* PreviewGeom,
	UCollisionGeometryVisualizationProperties* Settings,
	const FPhysicsDataCollection& PhysicsData,
	int32 ColorIndex,
	float DepthBias,
	int32 CircleStepResolution)
{
	check(PreviewGeom);
	check(Settings);

	InitializePreviewGeometryLines(
		PhysicsData,
		PreviewGeom,
		Settings->GetLineMaterial(),
		[&Settings](int LineSetIndex) { return Settings->GetLineSetColor(LineSetIndex); },
		Settings->LineThickness,
		Settings->bShowCollision,
		DepthBias,
		CircleStepResolution,
		ColorIndex);

	InitializePreviewGeometrySolid(
		PhysicsData, PreviewGeom, Settings->GetSolidMaterial(), [&Settings](int SolidSetIndex) { return Settings->GetTriangleSetColor(SolidSetIndex); },
		Settings->bEnableShowSolid && Settings->bShowCollision && Settings->bShowSolid,
		CircleStepResolution,
		ColorIndex);
}

void UE::PhysicsTools::PartiallyUpdateCollisionGeometryVisualization(
	UPreviewGeometry* PartialPreviewGeom,
	UCollisionGeometryVisualizationProperties* Settings,
	int32 ColorIndex)
{
	check(PartialPreviewGeom);
	check(Settings);

	// Note: Solid geometry uses the same coloring as line sets
	UpdatePreviewGeometryForCollision(PartialPreviewGeom, Settings, ColorIndex);
}

