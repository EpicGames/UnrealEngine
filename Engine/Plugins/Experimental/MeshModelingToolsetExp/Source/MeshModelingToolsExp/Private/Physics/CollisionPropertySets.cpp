// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/CollisionPropertySets.h"
#include "Components/ActorComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Physics/PhysicsDataCollection.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionPropertySets)

void UCollisionGeometryVisualizationProperties::Initialize(UInteractiveTool* Tool)
{
	WatchProperty(bShowCollision, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(LineThickness, [this](float NewValue) { bVisualizationDirty = true; });
	WatchProperty(Color, [this](FColor NewValue) { bVisualizationDirty = true; });
	WatchProperty(bRandomColors, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(bShowHidden, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(bShowSolid, [this](bool bNewValue) { bVisualizationDirty = true; });

	LineMaterial = ToolSetupUtil::GetDefaultLineComponentMaterial(Tool->GetToolManager(), true);
	LineMaterialShowingHidden = ToolSetupUtil::GetDefaultLineComponentMaterial(Tool->GetToolManager(), false);
	TriangleMaterial = ToolSetupUtil::GetVertexColorMaterial(Tool->GetToolManager(), false);
}

FColor UCollisionGeometryVisualizationProperties::GetLineSetColor(int32 LineSetIndex) const
{
	return bRandomColors ? UE::Geometry::LinearColors::SelectFColor(LineSetIndex) : Color;
}

void UE::PhysicsTools::InitializePhysicsToolObjectPropertySet(const FPhysicsDataCollection* PhysicsData, UPhysicsObjectToolPropertySet* PropSet)
{
	check(PhysicsData);
	check(PropSet);

	PropSet->ObjectName = FString::Printf(TEXT("%s - %s"),
		*PhysicsData->SourceComponent->GetOwner()->GetName(),
		*PhysicsData->SourceComponent->GetName());

	PropSet->CollisionType = (ECollisionGeometryMode)(int32)PhysicsData->BodySetup->GetCollisionTraceFlag();

	// change object name to above
	//PropSet->Rename(*PropSet->ObjectName, nullptr, REN_NonTransactional | REN_DoNotDirty);

	const FKAggregateGeom& AggGeom = PhysicsData->AggGeom;

	for (const FKSphereElem& Sphere : AggGeom.SphereElems)
	{
		FPhysicsSphereData SphereData;
		SphereData.Radius = Sphere.Radius;
		SphereData.Transform = Sphere.GetTransform();
		SphereData.Element = Sphere;
		PropSet->Spheres.Add(SphereData);
	}

	for (const FKBoxElem& Box : AggGeom.BoxElems)
	{
		FPhysicsBoxData BoxData;
		BoxData.Dimensions = FVector(Box.X, Box.Y, Box.Z);
		BoxData.Transform = Box.GetTransform();
		BoxData.Element = Box;
		PropSet->Boxes.Add(BoxData);
	}

	for (const FKSphylElem& Capsule : AggGeom.SphylElems)
	{
		FPhysicsCapsuleData CapsuleData;
		CapsuleData.Radius = Capsule.Radius;
		CapsuleData.Length = Capsule.Length;
		CapsuleData.Transform = Capsule.GetTransform();
		CapsuleData.Element = Capsule;
		PropSet->Capsules.Add(CapsuleData);
	}

	for (const FKConvexElem& Convex : AggGeom.ConvexElems)
	{
		FPhysicsConvexData ConvexData;
		ConvexData.NumVertices = Convex.VertexData.Num();
		ConvexData.NumFaces = Convex.IndexData.Num() / 3;
		ConvexData.Element = Convex;
		PropSet->Convexes.Add(ConvexData);
	}

	for (const FKLevelSetElem& LevelSet: AggGeom.LevelSetElems)
	{
		FPhysicsLevelSetData LevelSetData;
		LevelSetData.Element = LevelSet;
		PropSet->LevelSets.Add(LevelSetData);
	}
}



void UPhysicsObjectToolPropertySet::Reset()
{
	ObjectName = TEXT("");
	Spheres.Reset();
	Boxes.Reset();
	Capsules.Reset();
	Convexes.Reset();
	LevelSets.Reset();
}
