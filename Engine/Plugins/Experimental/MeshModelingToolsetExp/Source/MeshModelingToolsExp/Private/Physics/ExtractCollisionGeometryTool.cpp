// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/ExtractCollisionGeometryTool.h"
#include "InteractiveToolManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Util/ColorConstants.h"
#include "PreviewMesh.h"

#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshNormals.h"
#include "Generators/SphereGenerator.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"

#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryVisualization.h"
#include "Physics/ComponentCollisionUtil.h"

// physics data
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/AggregateGeom.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExtractCollisionGeometryTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UExtractCollisionGeometryTool"

const FToolTargetTypeRequirements& UExtractCollisionGeometryToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UPrimitiveComponentBackedTarget::StaticClass(),
		UPhysicsDataSource::StaticClass()
		});
	return TypeRequirements;
}


USingleSelectionMeshEditingTool* UExtractCollisionGeometryToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UExtractCollisionGeometryTool>(SceneState.ToolManager);
}


void UExtractCollisionGeometryTool::Setup()
{
	UInteractiveTool::Setup();

	// create preview mesh
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	PreviewMesh->SetMaterial(ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager()));
	PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetSelectionMaterial(GetToolManager()));
	PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
	{
		return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
	});
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->OutputType = UCreateMeshObjectTypeProperties::VolumeIdentifier;		// prefer volumes for extracting simple collision
	OutputTypeProperties->RestoreProperties(this, TEXT("ExtractCollisionTool"));
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	Settings = NewObject<UExtractCollisionToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	// Update input mesh visibility w/ logic that toggles it off when the complex preview is shown
	auto UpdateInputMeshVisibility = [this]()
	{
		bool bShowInput = Settings->bShowInputMesh && (Settings->CollisionType != EExtractCollisionOutputType::Complex || !Settings->bShowPreview);
		UE::ToolTarget::SetSourceObjectVisible(Target, bShowInput);
	};
	Settings->WatchProperty(Settings->CollisionType, [this, UpdateInputMeshVisibility](EExtractCollisionOutputType NewValue) { UpdateInputMeshVisibility(); });
	Settings->WatchProperty(Settings->bWeldEdges, [this](bool bNewValue) { bResultValid = false; });
	Settings->WatchProperty(Settings->bShowPreview, [this, UpdateInputMeshVisibility](bool bNewValue)
	{
		PreviewMesh->SetVisible(bNewValue); 
		UpdateInputMeshVisibility();
	});
	PreviewMesh->SetVisible(Settings->bShowPreview);
	Settings->WatchProperty(Settings->bShowInputMesh, [this, UpdateInputMeshVisibility](bool bNewValue) { UpdateInputMeshVisibility(); });
	UpdateInputMeshVisibility();

	VizSettings = NewObject<UCollisionGeometryVisualizationProperties>(this);
	VizSettings->bEnableShowSolid = false; // This solid visualization is redundant to the 'show preview' option in the general settings section of this tool
	VizSettings->RestoreProperties(this);
	AddToolPropertySource(VizSettings);
	VizSettings->Initialize(this);

	// Enable simple collision visualization and related settings only when extracting simple collision
	VizSettings->bEnableShowCollision = false;
	Settings->WatchProperty(Settings->CollisionType, [this](EExtractCollisionOutputType NewValue)
	{
		bResultValid = false;
		SetToolPropertySourceEnabled(VizSettings, NewValue == EExtractCollisionOutputType::Simple);
		VizSettings->bShowCollision = NewValue == EExtractCollisionOutputType::Simple;
		VizSettings->bVisualizationDirty = true;
		NotifyOfPropertyChangeByTool(VizSettings);
	});
	SetToolPropertySourceEnabled(VizSettings, Settings->CollisionType == EExtractCollisionOutputType::Simple);
	VizSettings->bShowCollision = Settings->CollisionType == EExtractCollisionOutputType::Simple;
	NotifyOfPropertyChangeByTool(VizSettings);
	

	UBodySetup* BodySetup = UE::ToolTarget::GetPhysicsBodySetup(Target);
	if (BodySetup)
	{
		PhysicsInfo = MakeShared<FPhysicsDataCollection>();
		PhysicsInfo->InitializeFromComponent( UE::ToolTarget::GetTargetComponent(Target), true);

		PreviewElements = NewObject<UPreviewGeometry>(this);
		FTransform TargetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
		PhysicsInfo->ExternalScale3D = TargetTransform.GetScale3D();
		TargetTransform.SetScale3D(FVector::OneVector);
		PreviewElements->CreateInWorld(UE::ToolTarget::GetTargetActor(Target)->GetWorld(), TargetTransform);

		UE::PhysicsTools::InitializeCollisionGeometryVisualization(PreviewElements, VizSettings, *PhysicsInfo);

		ObjectProps = NewObject<UPhysicsObjectToolPropertySet>(this);
		UE::PhysicsTools::InitializePhysicsToolObjectPropertySet(PhysicsInfo.Get(), ObjectProps);
		AddToolPropertySource(ObjectProps);
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Collision To Mesh"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Convert Collision Geometry to Mesh Objects"),
		EToolMessageLevel::UserNotification);
}


void UExtractCollisionGeometryTool::OnShutdown(EToolShutdownType ShutdownType)
{
	OutputTypeProperties->SaveProperties(this, TEXT("ExtractCollisionTool"));
	Settings->SaveProperties(this);
	VizSettings->SaveProperties(this);

	FTransform3d ActorTransform(PreviewMesh->GetTransform());

	PreviewElements->Disconnect();
	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	UE::ToolTarget::ShowSourceObject(Target);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		UMaterialInterface* UseMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

		FString TargetName = UE::ToolTarget::GetTargetComponent(Target)->GetName();

		GetToolManager()->BeginUndoTransaction(LOCTEXT("CreateCollisionMesh", "Collision To Mesh"));

		TArray<AActor*> OutputSelection;

		auto EmitNewMesh = [&](FDynamicMesh3&& Mesh, FTransform3d UseTransform, FString UseName)
		{
			FCreateMeshObjectParams NewMeshObjectParams;
			NewMeshObjectParams.TargetWorld = GetTargetWorld();
			NewMeshObjectParams.Transform = (FTransform)UseTransform;
			NewMeshObjectParams.BaseName = UseName;
			NewMeshObjectParams.Materials.Add(UseMaterial);
			NewMeshObjectParams.SetMesh(MoveTemp(Mesh));
			OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
			FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
			if (Result.IsOK() && Result.NewActor != nullptr)
			{
				OutputSelection.Add(Result.NewActor);
			}
		};

		int32 NumParts = CurrentMeshParts.Num();
		if (Settings->bOutputSeparateMeshes && NumParts > 1)
		{
			for ( int32 k = 0; k < NumParts; ++k)
			{
				FDynamicMesh3& MeshPart = *CurrentMeshParts[k];
				FAxisAlignedBox3d Bounds = MeshPart.GetBounds();
				MeshTransforms::Translate(MeshPart, -Bounds.Center());
				FTransform3d CenterTransform = ActorTransform;
				CenterTransform.SetTranslation(CenterTransform.GetTranslation() + ActorTransform.TransformVector(Bounds.Center()));
				FString NewName = FString::Printf(TEXT("%s_Collision%d"), *TargetName, k);
				EmitNewMesh(MoveTemp(MeshPart), CenterTransform, NewName);
			}
		}
		else
		{
			FString NewName = FString::Printf(TEXT("%s_Collision"), *TargetName);
			EmitNewMesh(MoveTemp(CurrentMesh), ActorTransform, NewName);
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), OutputSelection);

		GetToolManager()->EndUndoTransaction();
	}
}


bool UExtractCollisionGeometryTool::CanAccept() const
{
	return Super::CanAccept() && CurrentMesh.TriangleCount() > 0;
}




void UExtractCollisionGeometryTool::OnTick(float DeltaTime)
{
	if (bResultValid == false)
	{
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
		if (Settings->CollisionType == EExtractCollisionOutputType::Simple)
		{
			RecalculateMesh_Simple();
		}
		else
		{
			RecalculateMesh_Complex();
		}
		bResultValid = true;
	}

	UE::PhysicsTools::UpdateCollisionGeometryVisualization(PreviewElements, VizSettings);
}



void UExtractCollisionGeometryTool::RecalculateMesh_Simple()
{
	int32 SphereResolution = 16;

	CurrentMesh = FDynamicMesh3(EMeshComponents::FaceGroups);
	CurrentMesh.EnableAttributes();

	CurrentMeshParts.Reset();

	const FKAggregateGeom& AggGeom = PhysicsInfo->AggGeom;
	UE::Geometry::ConvertSimpleCollisionToMeshes(AggGeom, CurrentMesh,
		FTransformSequence3d(), SphereResolution, true, true,
		[&](int32 ElemType, const FDynamicMesh3& ElemMesh) {
			CurrentMeshParts.Add(MakeShared<FDynamicMesh3>(ElemMesh));
		},
		false /*bApproximateLevelSetWithCubes*/,
		PhysicsInfo->ExternalScale3D);

	for ( int32 k = 0; k < CurrentMeshParts.Num(); ++k)
	{
		FDynamicMesh3& MeshPart = *CurrentMeshParts[k];
		FMeshNormals::InitializeMeshToPerTriangleNormals(&MeshPart);
	}

	FTransform LocalToWorldUnscaled = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	LocalToWorldUnscaled.SetScale3D(FVector::OneVector);
	PreviewMesh->SetTransform(LocalToWorldUnscaled);
	PreviewMesh->UpdatePreview(&CurrentMesh);

	if (CurrentMeshParts.Num() == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("NoSimpleCollisionShapes", "This Mesh has no Simple Collision Shapes"), EToolMessageLevel::UserWarning);
	}
}



void UExtractCollisionGeometryTool::RecalculateMesh_Complex()
{
	CurrentMesh = FDynamicMesh3(EMeshComponents::FaceGroups);
	CurrentMesh.EnableAttributes();
	CurrentMeshParts.Reset();

	bool bMeshErrors = false;

	IInterface_CollisionDataProvider* CollisionProvider = UE::ToolTarget::GetPhysicsCollisionDataProvider(Target);
	if (CollisionProvider)
	{
		FTransformSequence3d Transform;
		UE::Geometry::ConvertComplexCollisionToMeshes(CollisionProvider, CurrentMesh, FTransformSequence3d(), bMeshErrors, Settings->bWeldEdges, true);
	}

	PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	PreviewMesh->UpdatePreview(&CurrentMesh);

	if (CurrentMesh.TriangleCount() == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("EmptyComplexCollision", "This Mesh has no Complex Collision geometry"), EToolMessageLevel::UserWarning);
	}
}

#undef LOCTEXT_NAMESPACE
