// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsInspectorTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Util/ColorConstants.h"

#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryVisualization.h"

// physics data
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsInspectorTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPhysicsInspectorTool"


const FToolTargetTypeRequirements& UPhysicsInspectorToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UPrimitiveComponentBackedTarget::StaticClass(),
		UPhysicsDataSource::StaticClass()
		});
	return TypeRequirements;
}


UMultiSelectionMeshEditingTool* UPhysicsInspectorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UPhysicsInspectorTool>(SceneState.ToolManager);
}


void UPhysicsInspectorTool::Setup()
{
	UInteractiveTool::Setup();

	VizSettings = NewObject<UCollisionGeometryVisualizationProperties>(this);
	VizSettings->RestoreProperties(this);
	AddToolPropertySource(VizSettings);
	VizSettings->bEnableShowCollision = false; // This tool always shows collision geometry

	// Initialize the collision geometry visualization
	{
		VizSettings->Initialize(this);
		InitializePreviewGeometry(false /*bClearExisting*/);
	}

	Settings = NewObject<UPhysicsInspectorToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	Settings->WatchProperty(Settings->bShowTargetMesh, [this](bool bNewValue)
	{
		for (UToolTarget* Target : Targets)
		{
			UE::ToolTarget::SetSourceObjectVisible(Target, bNewValue);
		}
	});

	OnCreatePhysicsDelegateHandle = UActorComponent::GlobalCreatePhysicsDelegate.AddUObject(this, &UPhysicsInspectorTool::OnCreatePhysics);

	SetToolDisplayName(LOCTEXT("ToolName", "Physics Inspector"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Inspect Physics data for the selected Objects"),
		EToolMessageLevel::UserNotification);
}

void UPhysicsInspectorTool::InitializePreviewGeometry(bool bClearExisting)
{
	if (bClearExisting)
	{
		for (UPreviewGeometry* Preview : PreviewElements)
		{
			Preview->Disconnect();
		}
		PreviewElements.Reset();
		for (int32 ObjectDataIdx = 0; ObjectDataIdx < ObjectData.Num(); ++ObjectDataIdx)
		{
			RemoveToolPropertySource(ObjectData[ObjectDataIdx]);
		}
		ObjectData.Reset();
	}

	int32 FirstLineSetIndex = 0;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		const UBodySetup* BodySetup = UE::ToolTarget::GetPhysicsBodySetup(Targets[ComponentIdx]);
		if (BodySetup)
		{
			FPhysicsDataCollection PhysicsData;
			PhysicsData.InitializeFromComponent(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]), true);

			UPreviewGeometry* PreviewGeom = NewObject<UPreviewGeometry>(this);
			FTransform TargetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]);
			PhysicsData.ExternalScale3D = TargetTransform.GetScale3D();
			TargetTransform.SetScale3D(FVector::OneVector);
			PreviewGeom->CreateInWorld(UE::ToolTarget::GetTargetActor(Targets[ComponentIdx])->GetWorld(), TargetTransform);
			PreviewElements.Add(PreviewGeom);

			UE::PhysicsTools::PartiallyInitializeCollisionGeometryVisualization(PreviewGeom, VizSettings, PhysicsData, FirstLineSetIndex);
			FirstLineSetIndex += PreviewGeom->LineSets.Num();

			UPhysicsObjectToolPropertySet* ObjectProps = NewObject<UPhysicsObjectToolPropertySet>(this);
			UE::PhysicsTools::InitializePhysicsToolObjectPropertySet(&PhysicsData, ObjectProps);
			AddToolPropertySource(ObjectProps);
			ObjectData.Add(ObjectProps);
		}
	}
	VizSettings->bVisualizationDirty = false;
	bUnderlyingPhysicsObjectsUpdated = false;
}

void UPhysicsInspectorTool::OnCreatePhysics(UActorComponent* Component)
{
	if (bUnderlyingPhysicsObjectsUpdated) // if flag already set, do not need to re-check
	{
		return;
	}
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ++ComponentIdx)
	{
		if (UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]) == Component)
		{
			bUnderlyingPhysicsObjectsUpdated = true;
			return;
		}
	}
}


void UPhysicsInspectorTool::OnShutdown(EToolShutdownType ShutdownType)
{
	VizSettings->SaveProperties(this);
	Settings->SaveProperties(this);

	UActorComponent::GlobalCreatePhysicsDelegate.Remove(OnCreatePhysicsDelegateHandle);

	for (UPreviewGeometry* Preview : PreviewElements)
	{
		Preview->Disconnect();
	}

	if (!Settings->bShowTargetMesh)
	{
		for (UToolTarget* Target : Targets)
		{
			UE::ToolTarget::ShowSourceObject(Target);
		}
	}
}





void UPhysicsInspectorTool::OnTick(float DeltaTime)
{
	if (bUnderlyingPhysicsObjectsUpdated)
	{
		InitializePreviewGeometry(true /*bClearExisting*/);
	}

	if (VizSettings->bVisualizationDirty)
	{
		int32 FirstLineSetIndex= 0;
		for (UPreviewGeometry* Preview : PreviewElements)
		{
			UE::PhysicsTools::PartiallyUpdateCollisionGeometryVisualization(Preview, VizSettings, FirstLineSetIndex);
			FirstLineSetIndex += Preview->LineSets.Num();
		}

		VizSettings->bVisualizationDirty = false;
	}
}



#undef LOCTEXT_NAMESPACE
