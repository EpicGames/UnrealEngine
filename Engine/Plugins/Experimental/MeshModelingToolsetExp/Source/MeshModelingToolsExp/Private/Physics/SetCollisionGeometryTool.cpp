// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/SetCollisionGeometryTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "GroupTopology.h"
#include "DynamicSubmesh3.h"
#include "Polygroups/PolygroupUtil.h"
#include "Util/ColorConstants.h"

#include "ShapeApproximation/ShapeDetection3.h"
#include "ShapeApproximation/MeshSimpleShapeApproximation.h"

#include "Physics/CollisionGeometryVisualization.h"
#include "Physics/ComponentCollisionUtil.h"

// physics data
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/BodySetup.h"

#include "Async/ParallelFor.h"

#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "UObject/UObjectIterator.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetCollisionGeometryTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USetCollisionGeometryTool"


/*
 * Operators
 */

class FPhysicsCollectionOp : public TGenericDataOperator<FPhysicsDataCollection>
{
public:

	TSharedPtr<FPhysicsDataCollection, ESPMode::ThreadSafe> InitialCollision;
	TSharedPtr<TArray<FPhysicsDataCollection>, ESPMode::ThreadSafe> OtherInputsCollision;
	FTransformSequence3d TargetInverseTransform;
	TSharedPtr<TArray<FTransform3d>, ESPMode::ThreadSafe> OtherInputsTransforms; // null if not using world space

	TUniquePtr<FMeshSimpleShapeApproximation> UseShapeGenerator;
	// Note: UseShapeGenerator holds raw pointers to these meshes, so we keep the array of shared pointers to prevent them from getting deleted while the op runs
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> ActiveInputMeshes;

	ECollisionGeometryType ComputeType;

	bool bUseMaxCount;
	bool bRemoveContained;
	bool bAppendToExisting;
	bool bMergeCollisionShapes;
	bool bUseNegativeSpaceInMerge;

	EProjectedHullAxis SweepAxis;
	int32 MaxCount;

	int32 MergeAboveCount;

	// Begin TGenericDataOperator interface
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		check(UseShapeGenerator.IsValid());
		check(InitialCollision.IsValid());

		// calculate new collision
		TUniquePtr<FPhysicsDataCollection> NewCollision = MakeUnique<FPhysicsDataCollection>();
		NewCollision->InitializeFromExisting(*InitialCollision);
		if (bAppendToExisting)
		{
			NewCollision->CopyGeometryFromExisting(*InitialCollision);
		}

		switch (ComputeType)
		{
		case ECollisionGeometryType::Empty:
			break;
		case ECollisionGeometryType::CopyFromInputs:
			if (ensure(OtherInputsCollision))
			{
				if (OtherInputsTransforms)
				{
					for (int32 i = 0; i < OtherInputsCollision->Num(); ++i)
					{
						const FPhysicsDataCollection& Collision = (*OtherInputsCollision)[i];
						if (!ensure(i < OtherInputsTransforms->Num()))
						{
							break;
						}

						FTransformSequence3d TransformStack;
						TransformStack.Append((*OtherInputsTransforms)[i]);
						TransformStack.Append(TargetInverseTransform);
						NewCollision->Geometry.Append(Collision.Geometry, TransformStack);
					}
				}
				else
				{
					for (const FPhysicsDataCollection& Collision : *OtherInputsCollision)
					{
						NewCollision->Geometry.Append(Collision.Geometry);
					}
				}
			}
			break;
		case ECollisionGeometryType::AlignedBoxes:
			UseShapeGenerator->Generate_AlignedBoxes(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::OrientedBoxes:
			UseShapeGenerator->Generate_OrientedBoxes(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::MinimalSpheres:
			UseShapeGenerator->Generate_MinimalSpheres(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::Capsules:
			UseShapeGenerator->Generate_Capsules(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::ConvexHulls:
			UseShapeGenerator->Generate_ConvexHulls(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::ConvexDecompositions:
			UseShapeGenerator->Generate_ConvexHullDecompositions(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::SweptHulls:
			UseShapeGenerator->Generate_ProjectedHulls(NewCollision->Geometry,
				(FMeshSimpleShapeApproximation::EProjectedHullAxisMode)(int32)SweepAxis);
			break;
		case ECollisionGeometryType::LevelSets:
			UseShapeGenerator->Generate_LevelSets(NewCollision->Geometry);
			break;
		case ECollisionGeometryType::MinVolume:
			UseShapeGenerator->Generate_MinVolume(NewCollision->Geometry);
			break;
		}

		if (!NewCollision)
		{
			ensure(false);
			return;
		}

		if (bRemoveContained)
		{
			NewCollision->Geometry.RemoveContainedGeometry();
		}

		if (bMergeCollisionShapes)
		{
			FSimpleShapeSet3d::FMergeShapesSettings Settings;
			Settings.bMergeShapesProtectNegativeSpace = bUseNegativeSpaceInMerge;
			// Take the UseShapeGenerator negative space settings (same as we use for convex decomposition negative space)
			Settings.bIgnoreInternalNegativeSpace = UseShapeGenerator->bIgnoreInternalNegativeSpace;
			Settings.NegativeSpaceMinRadius = UseShapeGenerator->NegativeSpaceMinRadius;
			Settings.NegativeSpaceTolerance = UseShapeGenerator->NegativeSpaceTolerance;
			NewCollision->Geometry.MergeShapes(MergeAboveCount, Settings);
		}
		
		if (bUseMaxCount)
		{
			NewCollision->Geometry.FilterByVolume(MaxCount);
		}

		NewCollision->CopyGeometryToAggregate();

		SetResult(MoveTemp(NewCollision));
	}
	// End TGenericDataOperator interface
};



const FToolTargetTypeRequirements& USetCollisionGeometryToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMeshDescriptionProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool USetCollisionGeometryToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	UActorComponent* LastValidTarget = nullptr;
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(),
		[&](UActorComponent* Component) { LastValidTarget = Component; });
	if (LastValidTarget != nullptr)
	{
		return (Cast<UStaticMeshComponent>(LastValidTarget) != nullptr) ||
			(Cast<UDynamicMeshComponent>(LastValidTarget) != nullptr);
	}
	return false;
}


void USetCollisionGeometryToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const
{
	const TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, GetTargetRequirements());
	Tool->SetTargets(Targets);
	Tool->SetWorld(SceneState.World);

	if (USetCollisionGeometryTool* CollisionTool = Cast<USetCollisionGeometryTool>(Tool))
	{
		if (Targets.Num() == 1) // Can only have a selection when there is one target
		{
			FGeometrySelection Selection;
			if (GetCurrentGeometrySelectionForTarget(SceneState, Targets[0], Selection))
			{
				CollisionTool->SetGeometrySelection(MoveTemp(Selection));
			}
		}
	}
}


UMultiSelectionMeshEditingTool* USetCollisionGeometryToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USetCollisionGeometryTool>(SceneState.ToolManager);
}

void USetCollisionGeometryTool::Setup()
{
	UInteractiveTool::Setup();

	// if we have one selection, use it as the source, otherwise use all but the last selected mesh
	bSourcesHidden = (Targets.Num() > 1);
	if (Targets.Num() == 1)
	{
		SourceObjectIndices.Add(0);
	}
	else
	{
		for (int32 k = 0; k < Targets.Num() -1; ++k)
		{
			SourceObjectIndices.Add(k);
			UE::ToolTarget::HideSourceObject(Targets[k]);
		}
	}

	UToolTarget* CollisionTarget = Targets[Targets.Num() - 1];
	OrigTargetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(CollisionTarget);
	
	// We need an inverse transform for copying simple collision in CopyFromInputs mode. We have
	// to separate out the scale since we can't represent the inverse properly just in SRT form.
	FTransform OrigTransformRT = OrigTargetTransform;
	OrigTransformRT.SetScale3D(FVector3d::One());

	TargetInverseTransform.Append(OrigTransformRT.Inverse());
	TargetInverseTransform.Append(FTransformSRT3d(FQuaterniond::Identity(), FVector3d::Zero(),
		OrigTargetTransform.GetScale3D().Reciprocal()));

	// The "OtherInputs" variables are only needed if we have multiple meshes selected. They
	// do not include the target mesh
	OtherInputsCollision = MakeShared<TArray<FPhysicsDataCollection>, ESPMode::ThreadSafe>();
	OtherInputsTransforms = MakeShared<TArray<FTransform3d>, ESPMode::ThreadSafe>();
	bool bHaveMultipleInputs = Targets.Num() > 1;
	if (bHaveMultipleInputs)
	{
		// The preallocation is so we can do the initialization in the parallel for below
		OtherInputsCollision->SetNum(SourceObjectIndices.Num());
		OtherInputsTransforms->SetNum(SourceObjectIndices.Num());
	}

	// collect input meshes
	InitialSourceMeshes.SetNum(SourceObjectIndices.Num());
	ParallelFor(SourceObjectIndices.Num(), [&](int32 k)
	{
		InitialSourceMeshes[k] = UE::ToolTarget::GetDynamicMeshCopy(Targets[k]);

		if (bHaveMultipleInputs)
		{
			(*OtherInputsCollision)[k].InitializeFromComponent(UE::ToolTarget::GetTargetComponent(Targets[k]), true);
			(*OtherInputsTransforms)[k] = UE::ToolTarget::GetLocalToWorldTransform(Targets[k]);
		}
	});

	PreviewGeom = NewObject<UPreviewGeometry>(this);
	FTransform PreviewTransform = OrigTargetTransform;
	TargetScale3D = PreviewTransform.GetScale3D();
	PreviewTransform.SetScale3D(FVector::OneVector);
	PreviewGeom->CreateInWorld(UE::ToolTarget::GetTargetActor(CollisionTarget)->GetWorld(), PreviewTransform);

	// initialize initial collision object
	InitialCollision = MakeShared<FPhysicsDataCollection, ESPMode::ThreadSafe>();
	InitialCollision->InitializeFromComponent(UE::ToolTarget::GetTargetComponent(CollisionTarget), true);
	InitialCollision->ExternalScale3D = TargetScale3D;

	// create tool options
	Settings = NewObject<USetCollisionGeometryToolProperties>(this);
	Settings->RestoreProperties(this);
	Settings->bUsingMultipleInputs = Targets.Num() > 1;
	AddToolPropertySource(Settings);
	Settings->WatchProperty(Settings->InputMode, [this](ESetCollisionGeometryInputMode) { OnInputModeChanged(); });
	Settings->WatchProperty(Settings->GeometryType, [this](ECollisionGeometryType) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bUseWorldSpace, [this](bool) { bInputMeshesValid = false; });
	Settings->WatchProperty(Settings->bAppendToExisting, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bRemoveContained, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bEnableMaxCount, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->MaxCount, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->MinThickness, [this](float) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bDetectBoxes, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bDetectSpheres, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bDetectCapsules, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bSimplifyHulls, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->HullTargetFaceCount, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->HullSimplifyGeometryTolerance, [this](double) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->MaxHullsPerShape, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bPreSimplifyToEdgeLength, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->DecompositionTargetEdgeLength, [this](double) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->ConvexDecompositionSearchFactor, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->AddHullsErrorTolerance, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->MinPartThickness, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->DecompositionMethod, [this](EConvexDecompositionMethod) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bLimitHullsPerShape, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->NegativeSpaceMinRadius, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->NegativeSpaceTolerance, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bIgnoreInternalNegativeSpace, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bMergeCollisionShapes, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bUseNegativeSpaceInMerge, [this](bool) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->MergeAboveCount, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->HullTolerance, [this](float) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->SweepAxis, [this](EProjectedHullAxis) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->LevelSetResolution, [this](int32) { InvalidateCompute(); });
	Settings->WatchProperty(Settings->bShowTargetMesh, [this](bool bNewValue) 
	{
		UE::ToolTarget::SetSourceObjectVisible(Targets.Last(), bNewValue);
	});
	UE::ToolTarget::SetSourceObjectVisible(Targets.Last(), Settings->bShowTargetMesh);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this, TEXT("SetCollisionGeometryTool"));
	if (InitialSourceMeshes.Num() == 1)
	{
		PolygroupLayerProperties->InitializeGroupLayers(&InitialSourceMeshes[0]);
	}
	else
	{
		TSet<FName> LayerNames;
		for (int32 MeshIdx = 0; MeshIdx < InitialSourceMeshes.Num(); ++MeshIdx)
		{
			if (InitialSourceMeshes[MeshIdx].Attributes())
			{
				for (int32 k = 0; k < InitialSourceMeshes[MeshIdx].Attributes()->NumPolygroupLayers(); k++)
				{
					FName Name = InitialSourceMeshes[MeshIdx].Attributes()->GetPolygroupLayer(k)->GetName();
					LayerNames.Add(Name);
				}
			}
		}
		PolygroupLayerProperties->InitializeGroupLayers(LayerNames);
	}
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
	AddToolPropertySource(PolygroupLayerProperties);

	VizSettings = NewObject<UCollisionGeometryVisualizationProperties>(this);
	VizSettings->RestoreProperties(this);
	AddToolPropertySource(VizSettings);
	VizSettings->Initialize(this);
	VizSettings->bEnableShowCollision = false; // This tool always shows collision geometry

	// add option for collision properties
	CollisionProps = NewObject<UPhysicsObjectToolPropertySet>(this);
	AddToolPropertySource(CollisionProps);

	if (InputGeometrySelection.IsEmpty() == false)
	{
		GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
		GeometrySelectionVizProperties->RestoreProperties(this);
		AddToolPropertySource(GeometrySelectionVizProperties);
		GeometrySelectionVizProperties->Initialize(this);
		GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(InputGeometrySelection.ElementType);
		GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(InputGeometrySelection.TopologyType);

		// Compute group topology if the selection has Polygroup topology, and do nothing otherwise
		FGroupTopology GroupTopology(&InitialSourceMeshes[0], InputGeometrySelection.TopologyType == EGeometryTopologyType::Polygroup);

		FTransformSRT3d ApplyTransform(UE::ToolTarget::GetLocalToWorldTransform(CollisionTarget));

		GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
		GeometrySelectionViz->CreateInWorld(GetTargetWorld(), ApplyTransform);
		InitializeGeometrySelectionVisualization(
			GeometrySelectionViz,
			GeometrySelectionVizProperties,
			InitialSourceMeshes[0],
			InputGeometrySelection,
			&GroupTopology);
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Mesh To Collision"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Initialize Simple Collision geometry for a Mesh from one or more input Meshes (including itself)."),
		EToolMessageLevel::UserNotification);

	// Make sure we are set to precompute input meshes on first tick
	bInputMeshesValid = false;
}

void USetCollisionGeometryTool::SetGeometrySelection(FGeometrySelection&& SelectionIn)
{
	InputGeometrySelection = MoveTemp(SelectionIn);
}


TUniquePtr<UE::Geometry::TGenericDataOperator<FPhysicsDataCollection>> USetCollisionGeometryTool::MakeNewOperator()
{
	TUniquePtr<FPhysicsCollectionOp> Op = MakeUnique<FPhysicsCollectionOp>();

	Op->InitialCollision = InitialCollision;
	Op->OtherInputsCollision = OtherInputsCollision;
	Op->TargetInverseTransform = TargetInverseTransform;
	if (Settings->bUseWorldSpace)
	{
		Op->OtherInputsTransforms = OtherInputsTransforms;
	}

	// Pick the approximator and input meshes that will be used by the op
	TSharedPtr<UE::Geometry::FMeshSimpleShapeApproximation, ESPMode::ThreadSafe>* Approximator = nullptr;
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>>* Inputs = nullptr;
	if (Settings->InputMode == ESetCollisionGeometryInputMode::CombineAll)
	{
		Approximator = &CombinedInputMeshesApproximator;
		Inputs = &CombinedInputMeshes;
	}
	else if (Settings->InputMode == ESetCollisionGeometryInputMode::PerMeshComponent)
	{
		Approximator = &SeparatedMeshesApproximator;
		Inputs = &SeparatedInputMeshes;
	}
	else if (Settings->InputMode == ESetCollisionGeometryInputMode::PerMeshGroup)
	{
		Approximator = &PerGroupMeshesApproximator;
		Inputs = &PerGroupInputMeshes;
	}
	else
	{
		Approximator = &InputMeshesApproximator;
		Inputs = &InputMeshes;
	}
	Op->UseShapeGenerator = MakeUnique<FMeshSimpleShapeApproximation>(**Approximator);
	Op->ActiveInputMeshes = *Inputs;

	Op->UseShapeGenerator->bDetectSpheres = Settings->bDetectSpheres;
	Op->UseShapeGenerator->bDetectBoxes = Settings->bDetectBoxes;
	Op->UseShapeGenerator->bDetectCapsules = Settings->bDetectCapsules;
	Op->UseShapeGenerator->MinDimension = Settings->MinThickness;
	// SimplifyHulls on the shape generator controls simplification on both swept and convex hull paths, but for Swept Hulls UI we leave simplification always enabled
	Op->UseShapeGenerator->bSimplifyHulls = Settings->GeometryType == ECollisionGeometryType::SweptHulls || Settings->bSimplifyHulls;
	Op->UseShapeGenerator->HullTargetFaceCount = Settings->HullTargetFaceCount;
	Op->UseShapeGenerator->HullGeometricToleranceAfterTarget = Settings->HullSimplifyGeometryTolerance;
	Op->UseShapeGenerator->bDecompositionPreSimplifyWithEdgeLength = Settings->bPreSimplifyToEdgeLength;
	Op->UseShapeGenerator->DecompositionPreSimplifyEdgeLength = Settings->DecompositionTargetEdgeLength;
	Op->UseShapeGenerator->bUseConvexDecompositionMaxPieces = Settings->bLimitHullsPerShape || Settings->DecompositionMethod != EConvexDecompositionMethod::NavigationDriven;
	Op->UseShapeGenerator->ConvexDecompositionMaxPieces = Settings->MaxHullsPerShape;
	Op->UseShapeGenerator->ConvexDecompositionSearchFactor = Settings->ConvexDecompositionSearchFactor;
	Op->UseShapeGenerator->ConvexDecompositionErrorTolerance = Settings->AddHullsErrorTolerance;
	Op->UseShapeGenerator->ConvexDecompositionMinPartThickness = Settings->MinPartThickness;
	Op->UseShapeGenerator->bConvexDecompositionProtectNegativeSpace = Settings->DecompositionMethod == EConvexDecompositionMethod::NavigationDriven;
	Op->UseShapeGenerator->NegativeSpaceMinRadius = Settings->NegativeSpaceMinRadius;
	Op->UseShapeGenerator->NegativeSpaceTolerance = Settings->NegativeSpaceTolerance;
	Op->UseShapeGenerator->bIgnoreInternalNegativeSpace = Settings->bIgnoreInternalNegativeSpace;
	Op->UseShapeGenerator->HullSimplifyTolerance = Settings->HullTolerance;
	Op->UseShapeGenerator->LevelSetGridResolution = Settings->LevelSetResolution;
	Op->bMergeCollisionShapes = Settings->bMergeCollisionShapes;
	Op->bUseNegativeSpaceInMerge = Settings->bUseNegativeSpaceInMerge;
	Op->MergeAboveCount = Settings->MergeAboveCount;

	Op->ComputeType = Settings->GeometryType;
	Op->bAppendToExisting = Settings->bAppendToExisting;

	// If we are in single-target mode, the CopyFromInputs option does not make sense because
	// we don't want to let the user accidentally duplicate all the geometry if bAppendToExisting
	// is true, and if it is false, the user can do the same thing with Empty and appending.
	// TODO: add a way to disable this enum option in the UI.
	if (Targets.Num() <= 1 && Op->ComputeType == ECollisionGeometryType::CopyFromInputs)
	{
		Op->ComputeType = ECollisionGeometryType::Empty;
		Op->bAppendToExisting = true;
	}

	Op->bUseMaxCount = Settings->bEnableMaxCount;
	Op->MaxCount = Settings->MaxCount;
	Op->bRemoveContained = Settings->bRemoveContained;
	Op->SweepAxis = Settings->SweepAxis;

	return Op;
}


void USetCollisionGeometryTool::OnShutdown(EToolShutdownType ShutdownType)
{
	VizSettings->SaveProperties(this);
	Settings->SaveProperties(this);
	if (PolygroupLayerProperties)
	{
		PolygroupLayerProperties->SaveProperties(this, TEXT("SetCollisionGeometryTool"));
	}

	PreviewGeom->Disconnect();

	if (GeometrySelectionViz)
	{
		GeometrySelectionViz->Disconnect();
	}

	if (GeometrySelectionVizProperties)
	{
		GeometrySelectionVizProperties->SaveProperties(this);
	}

	// show hidden sources
	if (bSourcesHidden)
	{
		for (int32 k : SourceObjectIndices)
		{
			UE::ToolTarget::ShowSourceObject(Targets[k]);
		}
	}
	if (!Settings->bShowTargetMesh)
	{
		UE::ToolTarget::ShowSourceObject(Targets.Last());
	}

	if (Compute)
	{
		Compute->Shutdown();
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// Make sure rendering is done so that we are not changing data being used by collision drawing.
		FlushRenderingCommands();

		GetToolManager()->BeginUndoTransaction(LOCTEXT("UpdateCollision", "Update Collision"));


		auto UpdateBodySetup = [this](UBodySetup* BodySetup)
		{
			// mark the BodySetup for modification. Do we need to modify the UStaticMesh??
			BodySetup->Modify();

			// clear existing simple collision. This will call BodySetup->InvalidatePhysicsData()
			BodySetup->RemoveSimpleCollision();

			// set new collision geometry
			BodySetup->AggGeom = GeneratedCollision->AggGeom;

			// update collision type
			BodySetup->CollisionTraceFlag = (ECollisionTraceFlag)(int32)Settings->SetCollisionType;

			// rebuild physics meshes
			BodySetup->CreatePhysicsMeshes();
		};


		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Targets[Targets.Num() - 1]);
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			// code below derived from FStaticMeshEditor::DuplicateSelectedPrims(), FStaticMeshEditor::OnCollisionSphere(), and GeomFitUtils.cpp::GenerateSphylAsSimpleCollision()
			TObjectPtr<UStaticMesh> StaticMesh = (StaticMeshComponent) ? StaticMeshComponent->GetStaticMesh() : nullptr;
			UBodySetup* BodySetup = (StaticMesh) ? StaticMesh->GetBodySetup() : nullptr;
			if (BodySetup != nullptr)
			{
				UpdateBodySetup(BodySetup);

				StaticMesh->RecreateNavCollision();

				// update physics state on all components using this StaticMesh
				for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
				{
					UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(*Iter);
					if (SMComponent->GetStaticMesh() == StaticMesh)
					{
						if (SMComponent->IsPhysicsStateCreated())
						{
							SMComponent->RecreatePhysicsState();
						}
						// Mark the render state dirty to make sure any CollisionTraceFlag changes get picked up
						SMComponent->MarkRenderStateDirty();
					}
				}

				// do we need to do a post edit change here??

				// mark static mesh as dirty so it gets resaved?
				StaticMesh->MarkPackageDirty();

#if WITH_EDITORONLY_DATA
				// mark the static mesh as having customized collision so it is not regenerated on reimport
				StaticMesh->SetCustomizedCollision(true);
#endif // WITH_EDITORONLY_DATA
			}
		}
		else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
		{
			DynamicMeshComponent->Modify();
			if (UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup())
			{
				BodySetup->Modify();
			}
			DynamicMeshComponent->CollisionType = (ECollisionTraceFlag)(int32)Settings->SetCollisionType;
			DynamicMeshComponent->SetSimpleCollisionShapes(GeneratedCollision->AggGeom, true);		
			DynamicMeshComponent->MarkRenderStateDirty();
		}

		// post the undo transaction
		GetToolManager()->EndUndoTransaction();
	}

}




void USetCollisionGeometryTool::OnTick(float DeltaTime)
{
	if (bInputMeshesValid == false)
	{
		PrecomputeInputMeshes();
		bInputMeshesValid = true;
		InvalidateCompute();
	}

	FText DisplayMessage;
	if (!Settings->bAppendToExisting && !InputGeometrySelection.IsEmpty())
	{
		DisplayMessage = LOCTEXT("GeometrySelectionWithoutAppendToExisting", "The tool was invoked with a selection so you may want to enable 'Append to Existing'");
	}
	GetToolManager()->DisplayMessage(DisplayMessage, EToolMessageLevel::UserWarning);

	if (Compute)
	{
		Compute->Tick(DeltaTime);

		if (Compute->HaveValidResult())
		{
			TUniquePtr<FPhysicsDataCollection> Result = Compute->Shutdown();
			if (Result.IsValid())
			{
				GeneratedCollision = MakeShareable<FPhysicsDataCollection>(Result.Release());

				VizSettings->bVisualizationDirty = true;

				// update visualization
				UE::PhysicsTools::InitializeCollisionGeometryVisualization(PreviewGeom, VizSettings, *GeneratedCollision);

				// update property set
				CollisionProps->Reset();
				UE::PhysicsTools::InitializePhysicsToolObjectPropertySet(GeneratedCollision.Get(), CollisionProps);
			}
		}
	}

	UE::PhysicsTools::UpdateCollisionGeometryVisualization(PreviewGeom, VizSettings);

	if (GeometrySelectionViz)
	{
		UpdateGeometrySelectionVisualization(GeometrySelectionViz, GeometrySelectionVizProperties);
	}
}


void USetCollisionGeometryTool::InvalidateCompute()
{
	if (PreviewGeom)
	{
		PreviewGeom->RemoveAllLineSets();
	}

	if (!bInputMeshesValid)
	{
		// InvalidateCompute() will be called again when the input meshes are valid
		return;
	}

	if (!Compute)
	{
		// Initialize background compute
		Compute = MakeUnique<TGenericDataBackgroundCompute<FPhysicsDataCollection>>();
		Compute->Setup(this);
	}
	Compute->InvalidateResult();
}


void USetCollisionGeometryTool::OnInputModeChanged()
{
	if (PolygroupLayerProperties != nullptr)
	{
		SetToolPropertySourceEnabled(PolygroupLayerProperties, Settings->InputMode == ESetCollisionGeometryInputMode::PerMeshGroup);
	}
	InvalidateCompute();
}

void USetCollisionGeometryTool::OnSelectedGroupLayerChanged()
{
	bInputMeshesValid = false;
	InvalidateCompute();
}


FPolygroupSet USetCollisionGeometryTool::GetActiveGroupLayer(const FDynamicMesh3* GroupLayersMesh)
{
	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		return FPolygroupSet(GroupLayersMesh);
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		const FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*GroupLayersMesh, SelectedName);
		return FPolygroupSet(GroupLayersMesh, FoundAttrib);
	}
}







void USetCollisionGeometryTool::InitializeDerivedMeshSet(
	const TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>>& FromInputMeshes,
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>>& ToMeshes,
	TFunctionRef<bool(const FDynamicMesh3* Mesh, int32 Tri0, int32 Tri1)> TrisConnectedPredicate)
{
	// find connected-components on input meshes, under given connectivity predicate
	TArray<TUniquePtr<FMeshConnectedComponents>> ComponentSets;
	ComponentSets.SetNum(FromInputMeshes.Num());
	ParallelFor(FromInputMeshes.Num(), [&](int32 k)
	{
		const FDynamicMesh3* Mesh = FromInputMeshes[k].Get();
		ComponentSets[k] = MakeUnique<FMeshConnectedComponents>(Mesh);
		ComponentSets[k]->FindConnectedTriangles(
			[Mesh, &TrisConnectedPredicate](int32 Tri0, int32 Tri1) 
			{ 
				return TrisConnectedPredicate(Mesh, Tri0, Tri1); 
			}
		);
	});

	// Assemble a list of all the submeshes we want to compute, so we can do them all in parallel
	struct FSubmeshSource
	{
		const FDynamicMesh3* SourceMesh;
		FIndex2i ComponentIdx;
	};
	TArray<FSubmeshSource> AllSubmeshes;
	for (int32 k = 0; k < FromInputMeshes.Num(); ++k)
	{
		const FDynamicMesh3* Mesh = FromInputMeshes[k].Get();
		int32 NumComponents = ComponentSets[k]->Num();
		for ( int32 j = 0; j < NumComponents; ++j )
		{
			const FMeshConnectedComponents::FComponent& Component = ComponentSets[k]->GetComponent(j);
			if (Component.Indices.Num() > 1)		// ignore single triangles
			{
				AllSubmeshes.Add(FSubmeshSource{ Mesh, FIndex2i(k,j) });
			}
		}
	}


	// compute all the submeshes
	ToMeshes.Reset();
	ToMeshes.SetNum(AllSubmeshes.Num());
	ParallelFor(AllSubmeshes.Num(), [&](int32 k)
	{
		const FSubmeshSource& Source = AllSubmeshes[k];
		const FMeshConnectedComponents::FComponent& Component = ComponentSets[Source.ComponentIdx.A]->GetComponent(Source.ComponentIdx.B);
		FDynamicSubmesh3 Submesh(Source.SourceMesh, Component.Indices, (int32)EMeshComponents::None, false);
		ToMeshes[k] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>( MoveTemp(Submesh.GetSubmesh()) );
	});
}


template<typename T>
TArray<const T*> MakeRawPointerList(const TArray<TSharedPtr<T, ESPMode::ThreadSafe>>& InputList)
{
	TArray<const T*> Result;
	Result.Reserve(InputList.Num());
	for (const TSharedPtr<T, ESPMode::ThreadSafe>& Ptr : InputList)
	{
		Result.Add(Ptr.Get());
	}
	return MoveTemp(Result);
}


void USetCollisionGeometryTool::PrecomputeInputMeshes()
{
	UToolTarget* CollisionTarget = Targets[Targets.Num() - 1];
	FTransformSRT3d TargetTransform(UE::ToolTarget::GetLocalToWorldTransform(CollisionTarget));

	// build input meshes.
	InputMeshes.Reset();
	InputMeshes.SetNum(SourceObjectIndices.Num());
	if (!InputGeometrySelection.IsEmpty())
	{
		TSet<int> TriangleROI;
		UE::Geometry::EnumerateSelectionTriangles(
			InputGeometrySelection,
			InitialSourceMeshes[0],
			[&TriangleROI](int32 TriangleID) { TriangleROI.Add(TriangleID); });

		// We dont discard attributes in the Submesh, we need them when building per-group input meshes
		FDynamicSubmesh3 Submesh(&InitialSourceMeshes[0], TriangleROI.Array());

		InputMeshes[0] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(MoveTemp(Submesh.GetSubmesh()));
	}
	else
	{
		bool bCanDiscardAttributes = (PolygroupLayerProperties->HasSelectedPolygroup() == false);
		ParallelFor(SourceObjectIndices.Num(), [this, &TargetTransform, bCanDiscardAttributes](int32 k)
		{
			FDynamicMesh3 SourceMesh = InitialSourceMeshes[k];
			if (Settings->bUseWorldSpace)
			{
				FTransformSRT3d ToWorld(UE::ToolTarget::GetLocalToWorldTransform(Targets[k]));
				MeshTransforms::ApplyTransform(SourceMesh, ToWorld, true);
				MeshTransforms::ApplyTransformInverse(SourceMesh, TargetTransform, true);
			}
			if (bCanDiscardAttributes)
			{
				SourceMesh.DiscardAttributes();
			}

			InputMeshes[k] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(MoveTemp(SourceMesh));
		});
	}
	InputMeshesApproximator = MakeShared<FMeshSimpleShapeApproximation, ESPMode::ThreadSafe>();
	InputMeshesApproximator->InitializeSourceMeshes(MakeRawPointerList<FDynamicMesh3>(InputMeshes));


	// build combined input
	CombinedInputMeshes.Reset();
	FDynamicMesh3 CombinedMesh;
	CombinedMesh.EnableTriangleGroups();
	FDynamicMeshEditor Appender(&CombinedMesh);
	FMeshIndexMappings TmpMappings;
	for (const TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>& InputMesh : InputMeshes)
	{
		TmpMappings.Reset();
		Appender.AppendMesh(InputMesh.Get(), TmpMappings);
	}
	CombinedInputMeshes.Add( MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(MoveTemp(CombinedMesh)) );
	CombinedInputMeshesApproximator = MakeShared<FMeshSimpleShapeApproximation, ESPMode::ThreadSafe>();
	CombinedInputMeshesApproximator->InitializeSourceMeshes(MakeRawPointerList<FDynamicMesh3>(CombinedInputMeshes));

	// build separated input meshes
	SeparatedInputMeshes.Reset();
	InitializeDerivedMeshSet(InputMeshes, SeparatedInputMeshes, 
		[](const FDynamicMesh3* Mesh, int32 Tri0, int32 Tri1)
		{
			return true;
		});
	SeparatedMeshesApproximator = MakeShared<FMeshSimpleShapeApproximation, ESPMode::ThreadSafe>();
	SeparatedMeshesApproximator->InitializeSourceMeshes(MakeRawPointerList<FDynamicMesh3>(SeparatedInputMeshes));

	// build per-group input meshes
	PerGroupInputMeshes.Reset();
	if (InputMeshes.Num() == 1)
	{
		FDynamicMesh3* UseGroupLayerMesh = &InitialSourceMeshes[0];
		if (!InputGeometrySelection.IsEmpty())
		{
			UseGroupLayerMesh = InputMeshes[0].Get();
		}

		// Use the active polygroup layer when there is only one input
		FPolygroupSet ActiveGroupSet = GetActiveGroupLayer(UseGroupLayerMesh);
		InitializeDerivedMeshSet(InputMeshes, PerGroupInputMeshes,
			[this, &ActiveGroupSet](const FDynamicMesh3* Mesh, int32 Tri0, int32 Tri1)
			{
				return ActiveGroupSet.GetTriangleGroup(Tri0) == ActiveGroupSet.GetTriangleGroup(Tri1);
			});
	}
	else
	{
		TMap<const FDynamicMesh3*, FPolygroupSet> ActiveGroupSets;
		for (int32 Idx = 0; Idx < InputMeshes.Num(); ++Idx)
		{
			ActiveGroupSets.Add(InputMeshes[Idx].Get(), GetActiveGroupLayer(InputMeshes[Idx].Get()));
		}

		// Use the default polygroup layer when there is more than one input
		InitializeDerivedMeshSet(InputMeshes, PerGroupInputMeshes,
			[&ActiveGroupSets](const FDynamicMesh3* Mesh, int32 Tri0, int32 Tri1)
			{
				FPolygroupSet* ActiveGroupSet = ActiveGroupSets.Find(Mesh);
				checkSlow(ActiveGroupSet);
				return ActiveGroupSet->GetTriangleGroup(Tri0) == ActiveGroupSet->GetTriangleGroup(Tri1);
			});
	}
	PerGroupMeshesApproximator = MakeShared<FMeshSimpleShapeApproximation, ESPMode::ThreadSafe>();
	PerGroupMeshesApproximator->InitializeSourceMeshes(MakeRawPointerList<FDynamicMesh3>(PerGroupInputMeshes));

}


#undef LOCTEXT_NAMESPACE
