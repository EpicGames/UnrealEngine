// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/SimpleCollisionEditorTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Util/ColorConstants.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryVisualization.h"
#include "Physics/ComponentCollisionUtil.h"

// physics data
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"

#include "UObject/UObjectIterator.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargetManager.h"

#include "Mechanics/CollisionPrimitivesMechanic.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleCollisionEditorTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USimpleCollisionEditorTool"


bool USimpleCollisionEditorToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (Super::CanBuildTool(SceneState)) // let super verify that there is only one targetable component
	{
		// make sure we can also write collision on the target
		bool bCanReadWrite = false;
		SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [&](UActorComponent* TargetComponent)
		{
			bCanReadWrite = Cast<UPrimitiveComponent>(TargetComponent) &&
				UE::Geometry::ComponentTypeSupportsCollision(Cast<UPrimitiveComponent>(TargetComponent), UE::Geometry::EComponentCollisionSupportLevel::ReadWrite);
		});
		return bCanReadWrite;
	}
	return false;
}

const FToolTargetTypeRequirements& USimpleCollisionEditorToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UPrimitiveComponentBackedTarget::StaticClass(),
		UPhysicsDataSource::StaticClass()
		});
	return TypeRequirements;
}


USingleSelectionMeshEditingTool* USimpleCollisionEditorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USimpleCollisionEditorTool>(SceneState.ToolManager);
}

void USimpleCollisionEditorToolActionProperties::PostAction(ESimpleCollisionEditorToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}

void USimpleCollisionEditorTool::Setup()
{
	UInteractiveTool::Setup();

	ActionProperties = NewObject<USimpleCollisionEditorToolActionProperties>(this);
	ActionProperties->Initialize(this);
	ActionProperties->RestoreProperties(this);
	AddToolPropertySource(ActionProperties);

	// collect input mesh
	FDynamicMesh3 InputMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	FAxisAlignedBox3d MeshBounds = InputMesh.GetBounds(true);

	UBodySetup* BodySetup = UE::ToolTarget::GetPhysicsBodySetup(Target);
	if (BodySetup)
	{
		PhysicsInfos = MakeShared<FPhysicsDataCollection>();
		PhysicsInfos->InitializeFromComponent( UE::ToolTarget::GetTargetComponent(Target), true);

		// Set up collision geometry mechanic
		CollisionPrimitivesMechanic = NewObject<UCollisionPrimitivesMechanic>(this);
		CollisionPrimitivesMechanic->Setup(this);
		CollisionPrimitivesMechanic->SetWorld(GetTargetWorld());
		FTransform3d LocalToWorld(Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform());
		CollisionPrimitivesMechanic->Initialize(PhysicsInfos, MeshBounds, LocalToWorld);
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Simple Collision Geometry Editor"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Edit the simple collision geometry for the selected Object."),
		EToolMessageLevel::UserNotification);
}


void USimpleCollisionEditorTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ActionProperties)
	{
		ActionProperties->SaveProperties(this);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// Make sure rendering is done so that we are not changing data being used by collision drawing.
		FlushRenderingCommands();

		GetToolManager()->BeginUndoTransaction(LOCTEXT("UpdateCollision", "Update Collision"));


		auto UpdateBodySetup = [this](UBodySetup* BodySetup)
		{
			// TODO:  mark the BodySetup for modification. Do we need to modify the UStaticMesh??
			BodySetup->Modify();

			// clear existing simple collision. This will call BodySetup->InvalidatePhysicsData()
			BodySetup->RemoveSimpleCollision();

			// set new collision geometry
			BodySetup->AggGeom = PhysicsInfos->AggGeom;

			// rebuild physics meshes
			BodySetup->CreatePhysicsMeshes();
		};


		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Target);
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			// code below derived from FStaticMeshEditor::DuplicateSelectedPrims(), FStaticMeshEditor::OnCollisionSphere(), and GeomFitUtils.cpp::GenerateSphylAsSimpleCollision()
			TObjectPtr<UStaticMesh> StaticMesh = (StaticMeshComponent) ? StaticMeshComponent->GetStaticMesh() : nullptr;
			UBodySetup* BodySetup = (StaticMesh) ? StaticMesh->GetBodySetup() : nullptr;
			if (BodySetup != nullptr)
			{
				UpdateBodySetup(BodySetup);

				StaticMesh->RecreateNavCollision();

				// update physics state on all components using this StaticMesh
				for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
				{
					UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(*Iter);
					if (SMComponent->GetStaticMesh() == StaticMesh)
					{
						if (SMComponent->IsPhysicsStateCreated())
						{
							SMComponent->RecreatePhysicsState();
						}
						// Mark the render state dirty to make sure any CollisionTraceFlag changes get picked up
						SMComponent->MarkRenderStateDirty();
					}
				}

				//  TODO: do we need to do a post edit change here??

				// mark static mesh as dirty so it gets resaved?
				StaticMesh->MarkPackageDirty();

#if WITH_EDITORONLY_DATA
				// mark the static mesh as having customized collision so it is not regenerated on reimport
				StaticMesh->SetCustomizedCollision(true);
#endif // WITH_EDITORONLY_DATA
			}
		}
		else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
		{
			DynamicMeshComponent->Modify();
			if (UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup())
			{
				BodySetup->Modify();
			}
			DynamicMeshComponent->SetSimpleCollisionShapes(PhysicsInfos->AggGeom, true);
			DynamicMeshComponent->MarkRenderStateDirty();
		}

		// post the undo transaction
		GetToolManager()->EndUndoTransaction();
	}

	if (CollisionPrimitivesMechanic)
	{
		CollisionPrimitivesMechanic->Shutdown();
	}

}

void USimpleCollisionEditorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (CollisionPrimitivesMechanic != nullptr)
	{
		CollisionPrimitivesMechanic->Render(RenderAPI);
	}
}

void USimpleCollisionEditorTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (CollisionPrimitivesMechanic != nullptr)
	{
		CollisionPrimitivesMechanic->DrawHUD(Canvas, RenderAPI);
	}
}

void USimpleCollisionEditorTool::OnTick(float DeltaTime)
{

	if (PendingAction != ESimpleCollisionEditorToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = ESimpleCollisionEditorToolAction::NoAction;
	}
}

void USimpleCollisionEditorTool::InitializeObjectProperties(const FPhysicsDataCollection& PhysicsData, UPhysicsObjectToolPropertySet* PropSet)
{
	UE::PhysicsTools::InitializePhysicsToolObjectPropertySet(&PhysicsData, PropSet);
}

void USimpleCollisionEditorTool::RequestAction(ESimpleCollisionEditorToolAction Action)
{
	PendingAction = Action;
}

void USimpleCollisionEditorTool::ApplyAction(ESimpleCollisionEditorToolAction Action)
{
	switch (Action)
	{
	case ESimpleCollisionEditorToolAction::AddSphere:
		CollisionPrimitivesMechanic->AddSphere();
		break;
	case ESimpleCollisionEditorToolAction::AddBox:
		CollisionPrimitivesMechanic->AddBox();
		break;
	case ESimpleCollisionEditorToolAction::AddCapsule:
		CollisionPrimitivesMechanic->AddCapsule();
		break;
	case ESimpleCollisionEditorToolAction::Duplicate:
		CollisionPrimitivesMechanic->DuplicateSelectedPrimitive();
		break;
	case ESimpleCollisionEditorToolAction::DeleteSelected:
		CollisionPrimitivesMechanic->DeleteSelectedPrimitive();
		break;
	case ESimpleCollisionEditorToolAction::DeleteAll:
		CollisionPrimitivesMechanic->DeleteAllPrimitives();
		break;
	default:
		break;
	}
}

#undef LOCTEXT_NAMESPACE
