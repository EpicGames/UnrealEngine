// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlaneCutTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "DynamicMeshEditor.h"
#include "Selection/SelectClickedAction.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "Drawing/MeshDebugDrawing.h"
#include "ModelingObjectsCreationAPI.h"

#include "Changes/ToolCommandChangeSequence.h"

#include "CuttingOps/PlaneCutOp.h"

#include "Misc/MessageDialog.h"

#include "ModelingToolTargetUtil.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlaneCutTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPlaneCutTool"

/*
 * ToolBuilder
 */


UMultiSelectionMeshEditingTool* UPlaneCutToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UPlaneCutTool>(SceneState.ToolManager);
}



/*
 * Tool
 */

UPlaneCutTool::UPlaneCutTool()
{
}

void UPlaneCutTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponents
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::HideSourceObject(Targets[ComponentIdx]);
	}

	TArray<int32> MapToFirstOccurrences;
	bool bAnyHaveSameSource = GetMapToSharedSourceData(MapToFirstOccurrences);

	if (bAnyHaveSameSource)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("PlaneCutMultipleAssetWithSameSource", "WARNING: Multiple meshes in your selection use the same source asset!  Plane cuts apply to the source asset, and this tool will not duplicate assets for you, so the tool typically cannot give a correct result in this case.  Please consider exiting the tool and duplicating the source assets."),
			EToolMessageLevel::UserWarning);
	}

	// Convert input targets to dynamic mesh
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		FDynamicMesh3* OriginalDynamicMesh = new FDynamicMesh3;
		*OriginalDynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[Idx]);
		OriginalDynamicMesh->EnableAttributes();
		TDynamicMeshScalarTriangleAttribute<int>* SubObjectIDs = new TDynamicMeshScalarTriangleAttribute<int>(OriginalDynamicMesh);
		SubObjectIDs->Initialize(0);
		OriginalDynamicMesh->Attributes()->AttachAttribute(FPlaneCutOp::ObjectIndexAttribute, SubObjectIDs);

		/// fill in the MeshesToCut array
		UDynamicMeshReplacementChangeTarget* Target = MeshesToCut.Add_GetRef(NewObject<UDynamicMeshReplacementChangeTarget>());
		// store a UV scale based on the original mesh bounds (we don't want to recompute this between cuts b/c we want consistent UV scale)
		MeshUVScaleFactor.Add(1.0 / OriginalDynamicMesh->GetBounds().MaxDim());

		// Set callbacks so previews are invalidated on undo/redo changing the meshes
		Target->SetMesh(TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>(OriginalDynamicMesh));
		Target->OnMeshChanged.AddLambda([this, Idx]() { Previews[Idx]->InvalidateResult(); });
	}

	// initialize our properties
	BasicProperties = NewObject<UPlaneCutToolProperties>(this, TEXT("Plane Cut Settings"));
	BasicProperties->RestoreProperties(this);
	AddToolPropertySource(BasicProperties);

	ToolPropertyObjects.Add(this);

	// initialize the PreviewMesh+BackgroundCompute object
	SetupPreviews();

	// set initial cut plane (also attaches gizmo/proxy)
	FBox CombinedBounds; CombinedBounds.Init();
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		FVector ComponentOrigin, ComponentExtents;
		UE::ToolTarget::GetTargetActor(Targets[Idx])->GetActorBounds(false, ComponentOrigin, ComponentExtents);
		CombinedBounds += FBox::BuildAABB(ComponentOrigin, ComponentExtents);
	}

	CutPlaneWorld.Origin = (FVector3d)CombinedBounds.GetCenter();
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), CutPlaneWorld);
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		CutPlaneWorld = PlaneMechanic->Plane;
		InvalidatePreviews();
		});
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		PlaneMechanic->SetPlaneCtrlClickBehaviorTarget->InvisibleComponentsToHitTest.Add(UE::ToolTarget::GetTargetComponent(Targets[Idx]));
	}

	InvalidatePreviews();

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Plane Cut"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartPlaneCutTool", "Press 'T' or use the Cut button to cut the mesh without leaving the tool.  Press 'R' to flip the plane direction."),
		EToolMessageLevel::UserNotification);
}




void UPlaneCutTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 101,
		TEXT("Do Plane Cut"), 
		LOCTEXT("DoPlaneCut", "Do Plane Cut"),
		LOCTEXT("DoPlaneCutTooltip", "Cut the mesh with the current cutting plane, without exiting the tool"),
		EModifierKey::None, EKeys::T,
		[this]() { this->Cut(); } );
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 102,
		TEXT("Flip Cutting Plane"),
		LOCTEXT("FlipCutPlane", "Flip Cutting Plane"),
		LOCTEXT("FlipCutPlaneTooltip", "Flip the cutting plane"),
		EModifierKey::None, EKeys::R,
		[this]() { this->FlipPlane(); });
}




void UPlaneCutTool::SetupPreviews()
{
	int32 CurrentNumPreview = Previews.Num();
	int32 NumSourceMeshes = MeshesToCut.Num();
	int32 TargetNumPreview = NumSourceMeshes;
	for (int32 PreviewIdx = CurrentNumPreview; PreviewIdx < TargetNumPreview; PreviewIdx++)
	{
		UPlaneCutOperatorFactory *CutSide = NewObject<UPlaneCutOperatorFactory>();
		CutSide->CutTool = this;
		CutSide->ComponentIndex = PreviewIdx;
		UMeshOpPreviewWithBackgroundCompute* Preview = Previews.Add_GetRef(NewObject<UMeshOpPreviewWithBackgroundCompute>(CutSide, "Preview"));
		Preview->Setup(GetTargetWorld(), CutSide);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
		Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

		const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[PreviewIdx]);
		Preview->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
		);

		// set initial preview to un-processed mesh, so stuff doesn't just disappear if the first cut takes a while
		Preview->PreviewMesh->UpdatePreview(MeshesToCut[PreviewIdx]->GetMesh().Get());
		Preview->PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[PreviewIdx]));
	}
	UpdateVisibility();
}


void UPlaneCutTool::DoFlipPlane()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("FlipPlaneTransactionName", "Flip Plane"));

	FVector3d Origin = CutPlaneWorld.Origin;
	FVector3d Normal = CutPlaneWorld.GetAxis(2);
	PlaneMechanic->SetDrawPlaneFromWorldPos(Origin, -Normal, false);

	GetToolManager()->EndUndoTransaction();
}


void UPlaneCutTool::DoCut()
{
	if (!CanAccept())
	{
		return;
	}

	

	TUniquePtr<FToolCommandChangeSequence> ChangeSeq = MakeUnique<FToolCommandChangeSequence>();

	TArray<FDynamicMeshOpResult> Results;
	for (int Idx = 0, N = MeshesToCut.Num(); Idx < N; Idx++)
	{
		UMeshOpPreviewWithBackgroundCompute* Preview = Previews[Idx];
		TUniquePtr<FDynamicMesh3> ResultMesh = Preview->PreviewMesh->ExtractPreviewMesh();
		ChangeSeq->AppendChange(MeshesToCut[Idx], MeshesToCut[Idx]->ReplaceMesh(
			TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>(ResultMesh.Release()))
		);
	}

	// emit combined change sequence
	GetToolManager()->EmitObjectChange(this, MoveTemp(ChangeSeq), LOCTEXT("MeshPlaneCut", "Cut Mesh with Plane"));
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}



void UPlaneCutTool::OnShutdown(EToolShutdownType ShutdownType)
{
	PlaneMechanic->Shutdown();
	BasicProperties->SaveProperties(this);

	// Restore (unhide) the source meshes
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[Idx]);
	}

	TArray<FDynamicMeshOpResult> Results;
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Results.Emplace(Preview->Shutdown());
	}
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Results);
	}
}

TUniquePtr<FDynamicMeshOperator> UPlaneCutOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FPlaneCutOp> CutOp = MakeUnique<FPlaneCutOp>();
	CutOp->bFillCutHole = CutTool->BasicProperties->bFillCutHole;
	CutOp->bFillSpans = CutTool->BasicProperties->bFillSpans;
	CutOp->bSimplifyAlongNewEdges = CutTool->BasicProperties->bSimplifyAlongCut;

	FTransform LocalToWorld = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(CutTool->Targets[ComponentIndex]);
	CutOp->SetTransform(LocalToWorld);
	// for all plane computation, change LocalToWorld to not have any zero scale dims
	FVector LocalToWorldScale = LocalToWorld.GetScale3D();
	for (int i = 0; i < 3; i++)
	{
		float DimScale = FMathf::Abs(LocalToWorldScale[i]);
		float Tolerance = KINDA_SMALL_NUMBER;
		if (DimScale < Tolerance)
		{
			LocalToWorldScale[i] = Tolerance * FMathf::SignNonZero(LocalToWorldScale[i]);
		}
	}
	LocalToWorld.SetScale3D(LocalToWorldScale);

	FVector LocalOrigin = LocalToWorld.InverseTransformPosition((FVector)CutTool->CutPlaneWorld.Origin);
	FVector3d WorldNormal = CutTool->CutPlaneWorld.GetAxis(2);
	FTransformSRT3d L2WForNormal(LocalToWorld);
	FVector LocalNormal = (FVector)L2WForNormal.InverseTransformNormal(WorldNormal);
	FVector BackTransformed = LocalToWorld.TransformVector(LocalNormal);
	float NormalScaleFactor = FVector::DotProduct(BackTransformed, (FVector)WorldNormal);
	if (NormalScaleFactor >= FLT_MIN)
	{
		NormalScaleFactor = 1.0 / NormalScaleFactor;
	}
	CutOp->LocalPlaneOrigin = (FVector3d)LocalOrigin;
	CutOp->LocalPlaneNormal = (FVector3d)LocalNormal;
	CutOp->OriginalMesh = CutTool->MeshesToCut[ComponentIndex]->GetMesh();
	CutOp->bKeepBothHalves = CutTool->BasicProperties->bKeepBothHalves;
	CutOp->CutPlaneLocalThickness = CutTool->BasicProperties->SpacingBetweenHalves * NormalScaleFactor;
	CutOp->UVScaleFactor = CutTool->MeshUVScaleFactor[ComponentIndex];
	

	return CutOp;
}



void UPlaneCutTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	PlaneMechanic->Render(RenderAPI);
}

void UPlaneCutTool::OnTick(float DeltaTime)
{
	PlaneMechanic->Tick(DeltaTime);

	if (PendingAction != EPlaneCutToolActions::NoAction)
	{
		if (PendingAction == EPlaneCutToolActions::Cut)
		{
			DoCut();
		}
		else if (PendingAction == EPlaneCutToolActions::FlipPlane)
		{
			DoFlipPlane();
		}

		PendingAction = EPlaneCutToolActions::NoAction;
	}

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}
}


#if WITH_EDITOR
void UPlaneCutTool::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InvalidatePreviews();
}
#endif


void UPlaneCutTool::UpdateVisibility()
{
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		UE::ToolTarget::SetSourceObjectVisible(Targets[Idx], !BasicProperties->bShowPreview);
	}
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->SetVisibility(BasicProperties->bShowPreview);
	}
}


void UPlaneCutTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPlaneCutToolProperties, bShowPreview)))
	{
		UpdateVisibility();
	}

	InvalidatePreviews();
}




void UPlaneCutTool::InvalidatePreviews()
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}





bool UPlaneCutTool::CanAccept() const
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}


void UPlaneCutTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	if (Results.Num() == 0)
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("PlaneCutToolTransactionName", "Plane Cut Tool"));
	

	// currently in-place replaces the first half, and adds a new actor for the second half (if it was generated)
	// TODO: options to support other choices re what should be a new actor

	ensure(Results.Num() > 0);
	int32 NumSourceMeshes = MeshesToCut.Num();
	TArray<TArray<TUniquePtr<FDynamicMesh3>>> AllSplitMeshes; AllSplitMeshes.SetNum(NumSourceMeshes);

	// build a selection change starting w/ the original selection (used if objects are added below)
	FSelectedObjectsChangeList NewSelection;
	NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
	for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
	{
		NewSelection.Actors.Add(UE::ToolTarget::GetTargetActor(Targets[OrigMeshIdx]));
	}

	// check if we entirely cut away any meshes
	bool bWantDestroy = false;
	for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
	{
		bWantDestroy = bWantDestroy || (Results[OrigMeshIdx].Mesh->TriangleCount() == 0);
	}
	// if so ask user what to do
	if (bWantDestroy)
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, 
			LOCTEXT("PlaneCutDestroyQuestion", "Plane cuts have entirely cut away at least one mesh. Do you actually want to delete these mesh components? Note that either way all actors will remain, and meshes that are not fully cut away will still be cut as normal."), 
			LOCTEXT("PlaneCutDestroyTitle", "Delete mesh components?"));
		if (Ret == EAppReturnType::No || Ret == EAppReturnType::Cancel)
		{
			bWantDestroy = false; // quell destructive urge
		}
	}
	
	bool bNeedToAdd = false; // will be set to true if any mesh will be partly split out into a new generated asset
	for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
	{
		FDynamicMesh3* UseMesh = Results[OrigMeshIdx].Mesh.Get();
		check(UseMesh != nullptr);

		if (UseMesh->TriangleCount() == 0)
		{
			if (bWantDestroy)
			{
				UE::ToolTarget::GetTargetComponent(Targets[OrigMeshIdx])->DestroyComponent();
			}
			continue;
		}

		if (BasicProperties->bExportSeparatedPiecesAsNewMeshAssets)
		{
			TDynamicMeshScalarTriangleAttribute<int>* SubMeshIDs =
				static_cast<TDynamicMeshScalarTriangleAttribute<int>*>(UseMesh->Attributes()->GetAttachedAttribute(
					FPlaneCutOp::ObjectIndexAttribute));
			TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes = AllSplitMeshes[OrigMeshIdx];
			bool bWasSplit = FDynamicMeshEditor::SplitMesh(UseMesh, SplitMeshes, false, [SubMeshIDs](int TID)
			{
				return SubMeshIDs->GetValue(TID);
			}
			);
			if (bWasSplit)
			{
				// split mesh did something but has no meshes in the output array??
				if (!ensure(SplitMeshes.Num() > 0))
				{
					continue;
				}
				bNeedToAdd = bNeedToAdd || (SplitMeshes.Num() > 1);
				UseMesh = SplitMeshes[0].Get();
			}
		}

		UE::ToolTarget::CommitDynamicMeshUpdate(Targets[OrigMeshIdx], *UseMesh, true);
	}

	if (bNeedToAdd)
	{
		for (int OrigMeshIdx = 0; OrigMeshIdx < NumSourceMeshes; OrigMeshIdx++)
		{
			TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes = AllSplitMeshes[OrigMeshIdx];
			if (SplitMeshes.Num() < 2)
			{
				continue;
			}

			// get materials for both the component and the asset
			const FComponentMaterialSet ComponentMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[OrigMeshIdx], false);
			const FComponentMaterialSet AssetMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[OrigMeshIdx], true /*prefer asset materials*/);

			// add all the additional meshes
			for (int AddMeshIdx = 1; AddMeshIdx < SplitMeshes.Num(); AddMeshIdx++)
			{
				FCreateMeshObjectParams NewMeshObjectParams;
				NewMeshObjectParams.TargetWorld = GetTargetWorld();
				NewMeshObjectParams.Transform = (FTransform)Results[OrigMeshIdx].Transform;
				NewMeshObjectParams.BaseName = TEXT("PlaneCutOtherPart");
				NewMeshObjectParams.Materials = ComponentMaterialSet.Materials;
				NewMeshObjectParams.AssetMaterials = AssetMaterialSet.Materials;
				NewMeshObjectParams.SetMesh(MoveTemp(*SplitMeshes[AddMeshIdx]));
				UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[OrigMeshIdx], NewMeshObjectParams);
				FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
				if (Result.IsOK() && Result.NewActor != nullptr)
				{
					NewSelection.Actors.Add(Result.NewActor);
				}
			}
		}

		if (NewSelection.Actors.Num() > 0)
		{
			GetToolManager()->RequestSelectionChange(NewSelection);
		}
	}


	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE

