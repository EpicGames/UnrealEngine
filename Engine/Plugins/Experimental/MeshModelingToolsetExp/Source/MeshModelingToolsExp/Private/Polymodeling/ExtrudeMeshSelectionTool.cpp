// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polymodeling/ExtrudeMeshSelectionTool.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "MeshQueries.h"

#include "Selections/MeshFaceSelection.h"
#include "DynamicSubmesh3.h"
#include "Operations/MeshRegionOperator.h"
#include "Operations/OffsetMeshRegion.h"

#include "ModelingOperators.h"
#include "PolyModelingOps/LinearExtrusionOp.h"
#include "Operations/PolyModeling/PolyModelingMaterialUtil.h"
#include "Operations/PolyModeling/PolyModelingFaceUtil.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExtrudeMeshSelectionTool)


using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UExtrudeMeshSelectionTool"



USingleTargetWithSelectionTool* UExtrudeMeshSelectionToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UExtrudeMeshSelectionTool>(SceneState.ToolManager);
}



class FExtrudeMeshSelectionOpFactory : public IDynamicMeshOperatorFactory
{
public:
	UExtrudeMeshSelectionTool* SourceTool;

	FExtrudeMeshSelectionOpFactory(UExtrudeMeshSelectionTool* Tool)
	{
		SourceTool = Tool;
	}

	TUniquePtr<FLinearExtrusionOp> MakeNewExtrudeOp(
		TSharedPtr<FSharedConstDynamicMesh3> SourceMesh,
		const TArray<int32>& ExtrudeROI	)
	{
		TUniquePtr<FLinearExtrusionOp> Op = MakeUnique<FLinearExtrusionOp>();
		Op->OriginalMeshShared = SourceTool->EditRegionSharedMesh;
		Op->TriangleSelection = ExtrudeROI;

		Op->bShellsToSolids = SourceTool->ExtrudeProperties->bShellsToSolids;

		Op->bInferGroupsFromNeighbours = SourceTool->ExtrudeProperties->bInferGroupsFromNbrs;
		Op->bNewGroupPerSubdivision = SourceTool->ExtrudeProperties->bGroupPerSubdivision;
		Op->bUseColinearityForSettingBorderGroups = false;		// what is this??
		Op->bRemapExtrudeGroups = ! SourceTool->ExtrudeProperties->bReplaceSelectionGroups;

		Op->CreaseAngleThresholdDeg = SourceTool->ExtrudeProperties->CreaseAngle;

		Op->UVScaleFactor = SourceTool->ExtrudeProperties->UVScale;
		Op->bUVIslandPerGroup = SourceTool->ExtrudeProperties->bUVIslandPerGroup;

		Op->bInferMaterialID = SourceTool->ExtrudeProperties->bInferMaterialID;
		Op->SetMaterialID = FMath::Min(SourceTool->ExtrudeProperties->SetMaterialID, SourceTool->MaxMaterialID);

		Op->RegionModifierMode = (FLinearExtrusionOp::ESelectionShapeModifierMode)(int32)SourceTool->ExtrudeProperties->RegionMode;
		Op->NumSubdivisions = SourceTool->ExtrudeProperties->NumSubdivisions;

		Op->RaycastMaxDistance = SourceTool->ExtrudeProperties->RaycastMaxDistance;

		if (SourceTool->ExtrudeProperties->InputMode == EExtrudeMeshSelectionInteractionMode::Interactive)
		{
			Op->StartFrame = SourceTool->InitialFrameLocal;
			Op->ToFrame = SourceTool->ExtrudeFrameLocal;
			Op->Scale = SourceTool->LocalScale;
		}
		else
		{
			Op->StartFrame = SourceTool->InitialFrameLocal;
			Op->ToFrame = FFrame3d(Op->StartFrame.Origin + SourceTool->ExtrudeProperties->ExtrudeDistance * Op->StartFrame.Z(), Op->StartFrame.Rotation);
			Op->Scale = FVector3d::One();
		}

		Op->SetResultTransform(SourceTool->WorldTransform);

		return Op;
	}

	virtual TUniquePtr<FDynamicMeshOperator> MakeNewOperator() override
	{
		return MakeNewExtrudeOp(SourceTool->EditRegionSharedMesh, SourceTool->RegionExtrudeROI.Array());
	}
};


UExtrudeMeshSelectionTool::UExtrudeMeshSelectionTool()
{
	SetToolDisplayName(LOCTEXT("ToolName", "Extrude Selection"));
}


void UExtrudeMeshSelectionTool::Setup()
{
	UToolTarget* UseTarget = Super::GetTarget();
	CurrentMesh = UE::ToolTarget::GetDynamicMeshCopy(UseTarget);

	// need valid MaterialIDs for MaterialID parameter
	FInterval1i MatIDRange;
	UE::Geometry::ComputeMaterialIDRange(CurrentMesh, MatIDRange);
	MaxMaterialID = MatIDRange.Max;

	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(UseTarget);
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(UseTarget);

	ExtrudeProperties = NewObject<UExtrudeMeshSelectionToolProperties>(this);
	ExtrudeProperties->RestoreProperties(this);
	ExtrudeProperties->WatchProperty(ExtrudeProperties->InputMode,
		[this](EExtrudeMeshSelectionInteractionMode NewMode) { UpdateInteractionMode(NewMode); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->ExtrudeDistance,
		[this](double) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->RegionMode,
		[this](EExtrudeMeshSelectionRegionModifierMode) { EditCompute->InvalidateResult();});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->NumSubdivisions,
		[this](int) { EditCompute->InvalidateResult();});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bShellsToSolids,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->UVScale,
		[this](double) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bInferGroupsFromNbrs,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bGroupPerSubdivision,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bUVIslandPerGroup,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bReplaceSelectionGroups,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->CreaseAngle,
		[this](double) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->RaycastMaxDistance,
		[this](double) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bInferMaterialID,
		[this](bool) { EditCompute->InvalidateResult(); });
	ExtrudeProperties->WatchProperty(ExtrudeProperties->SetMaterialID,
		[this](int) { EditCompute->InvalidateResult(); });

	ExtrudeProperties->WatchProperty(ExtrudeProperties->bShowInputMaterials,
		[this](bool) { UpdateVisualizationSettings(); });

	AddToolPropertySource(ExtrudeProperties);

	// extract selection
	FMeshFaceSelection TriSelection(&CurrentMesh);
	const FGeometrySelection& Selection = GetGeometrySelection();
	UE::Geometry::EnumerateSelectionTriangles(Selection, CurrentMesh,
		[&](int32 tid) { TriSelection.Select(tid); },
		nullptr  /* no support for group layers yet */);
	ExtrudeROI = TriSelection.AsArray();

	// if we have an empty selection, just abort here
	if (ExtrudeROI.Num() == 0)
	{
		GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel,
			true, LOCTEXT("InvalidSelectionMessage", "ExtrudeMeshSelectionTool: input face selection was empty, cannot Extrude"));
		return;
	}

	TriSelection.ExpandToOneRingNeighbours(2);
	ModifiedROI = TriSelection.AsSet();		// could steal here if it was possible

	SelectionBoundsWorld = (FBox)TMeshQueries<FDynamicMesh3>::GetTrianglesBounds(CurrentMesh, ExtrudeROI, WorldTransform);

	// create the preview object for the unmodified area
	// (if input was a UDynamicMesh we could do this w/o making a copy...)
	SourcePreview = NewObject<UPreviewMesh>();
	SourcePreview->CreateInWorld(GetTargetWorld(), WorldTransform);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(SourcePreview, UseTarget);
	SourcePreview->SetMaterials(MaterialSet.Materials);
	SourcePreview->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			return ExtrudeROI.Contains(TriangleID);		// hide triangles in extrude area in base mesh
		});
	SourcePreview->SetSecondaryBuffersVisibility(false);
	SourcePreview->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	SourcePreview->UpdatePreview(&CurrentMesh);

	// initialize a region operator for the modified area
	RegionOperator = MakePimpl<FMeshRegionOperator>(&CurrentMesh, ModifiedROI.Array());
	for (int32 tid : ExtrudeROI)
	{
		RegionExtrudeROI.Add( RegionOperator->Region.MapTriangleToSubmesh(tid) );
	}
	EditRegionMesh = RegionOperator->Region.GetSubmesh();
	for (int32 tid : EditRegionMesh.TriangleIndicesItr())
	{
		if (RegionExtrudeROI.Contains(tid) == false)
		{
			RegionBorderTris.Add(tid);
		}
	}
	EditRegionSharedMesh = MakeShared<FSharedConstDynamicMesh3>(&EditRegionMesh);

	// try to guess selection frame...
	SelectionFrameLocal = UE::Geometry::ComputeFaceSelectionFrame(EditRegionMesh, RegionExtrudeROI.Array(), false);
	InitialFrameLocal = SelectionFrameLocal;
	InitialFrameWorld = InitialFrameLocal;
	InitialFrameWorld.Transform(WorldTransform);

	this->OperatorFactory = MakePimpl<FExtrudeMeshSelectionOpFactory>(this);

	// Create the preview compute for the extrusion operation
	EditCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	EditCompute->Setup(GetTargetWorld(), this->OperatorFactory.Get());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(EditCompute->PreviewMesh, UseTarget);
	EditCompute->PreviewMesh->SetTransform((FTransform)WorldTransform);
	EditCompute->ConfigureMaterials(MaterialSet.Materials, nullptr, nullptr);

	// hide the triangles in the 'border' region outside the extrude area, although they are included in
	// the extrude region operator submesh, they are still visible in the base mesh
	EditCompute->PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID) { return RegionBorderTris.Contains(TriangleID); });
	EditCompute->PreviewMesh->SetSecondaryBuffersVisibility(false);

	UpdateVisualizationSettings();

	// is this the right tangents behavior?
	EditCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	EditCompute->PreviewMesh->UpdatePreview( & RegionOperator->Region.GetSubmesh() );
	EditCompute->SetVisibility(true);

	// hide input Component
	UE::ToolTarget::HideSourceObject(Target);

	// initialize UI stuff
	Initialize_GizmoMechanic();
	UpdateInteractionMode(ExtrudeProperties->InputMode);
}


void UExtrudeMeshSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if ( ExtrudeProperties )
	{
		ExtrudeProperties->SaveProperties(this);
	}

	if (SourcePreview != nullptr )
	{
		SourcePreview->Disconnect();
		SourcePreview = nullptr;
	}

	if (EditCompute != nullptr)
	{
		// shut down background computation
		// (we don't actually care about compute result here because it was only for the live preview...)
		FDynamicMeshOpResult ComputeResult = EditCompute->Shutdown();
		EditCompute->ClearOpFactory();
		EditCompute->OnOpCompleted.RemoveAll(this);
		EditCompute = nullptr;

		UToolTarget* UseTarget = GetTarget();
		UE::ToolTarget::ShowSourceObject(UseTarget);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("ExtrudeSelection", "Extrude Selection"));

			TArray<int32> SelectOutputTriangles;

			// this function computes the final Extrude on full Mesh and cleans it up
			auto ComputeFinalResult = [this, &SelectOutputTriangles](FDynamicMesh3& EditMesh) -> bool
			{
				TSharedPtr<FSharedConstDynamicMesh3> FullMeshShared = MakeShared<FSharedConstDynamicMesh3>(&EditMesh);
				TUniquePtr<FLinearExtrusionOp> FinalOp = OperatorFactory->MakeNewExtrudeOp(FullMeshShared, this->ExtrudeROI);
				SelectOutputTriangles = this->ExtrudeROI;
				bool bResult = FinalOp->CalculateResultInPlace(EditMesh, nullptr);
				if (EditMesh.IsCompact() == false)
				{
					FCompactMaps CompactMaps;
					EditMesh.CompactInPlace(&CompactMaps);
					if (CompactMaps.TriangleMapIsSet())
					{
						for (int32& tid : SelectOutputTriangles)
						{
							tid = CompactMaps.GetTriangleMapping(tid);
						}
					}
				}

				FullMeshShared->ReleaseSharedObject();
				return bResult;
			};

			// try to emit an incremental FMeshChange, to avoid storing full mesh before/after copies in undo history
			bool bChangeApplied = false;
			if (UE::ToolTarget::SupportsIncrementalMeshChanges(UseTarget))
			{
				bChangeApplied = UE::ToolTarget::ApplyIncrementalMeshEditChange(UseTarget,
					[&](FDynamicMesh3& EditMesh, UObject* TransactionTarget)
					{
						FDynamicMeshChangeTracker ChangeTracker(&EditMesh);
						ChangeTracker.BeginChange();
						ChangeTracker.SaveTriangles(ExtrudeROI, true);
						if (ComputeFinalResult(EditMesh))
						{
							TUniquePtr<FDynamicMeshChange> MeshEditChange = ChangeTracker.EndChange();
							GetToolManager()->GetContextTransactionsAPI()->AppendChange( TransactionTarget,
								MakeUnique<FMeshChange>(MoveTemp(MeshEditChange)), LOCTEXT("ExtrudeSelection", "Extrude Selection") );
							return true;
						}
						return false;
					});

				ensureMsgf(bChangeApplied, TEXT("UExtrudeMeshSelectionTool::OnShutdown : incremental mesh edit failed!"));
			}

			// if we could not apply incremental change, apply a full-mesh update
			if ( bChangeApplied == false )
			{
				ComputeFinalResult(CurrentMesh);
				const bool bModifiedTopology = true;
				UE::ToolTarget::CommitDynamicMeshUpdate(UseTarget, CurrentMesh, bModifiedTopology);
			}

			// Construct and send output triangle selection, assume selection system will convert to correct type.
			// (todo: maybe be smarter about this, as for (eg) an input vertex selection, it will grow the selection...)
			// Note that we cannot do this before we commit/complete the mesh update, as the selection system needs a chance
			// to respond to the mesh change. Possibly should use UE::Geometry::InitializeSelectionFromTriangles() here...need to ProcessMesh() then though
			FGeometrySelection OutputSelection;
			OutputSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
			for (int32 tid : SelectOutputTriangles)
			{
				OutputSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
			}
			UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, UseTarget, OutputSelection);


			GetToolManager()->EndUndoTransaction();
		}
	}

	if ( TransformGizmo != nullptr )
	{
		GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
		TransformProxy = nullptr;
		TransformGizmo = nullptr;
	}

	if ( EditRegionSharedMesh.IsValid() )
	{
		EditRegionSharedMesh->ReleaseSharedObject();
	}

	CurrentMesh.Clear();
}



void UExtrudeMeshSelectionTool::OnTick(float DeltaTime)
{
	EditCompute->Tick(DeltaTime);
}


void UExtrudeMeshSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

FBox UExtrudeMeshSelectionTool::GetWorldSpaceFocusBox() 
{ 
	FAxisAlignedBox3d CurBox = SelectionBoundsWorld;
	FVector3d Translation = ExtrudeFrameWorld.Origin - InitialFrameWorld.Origin;
	CurBox.Min += Translation; CurBox.Max += Translation;
	CurBox.Contain(SelectionBoundsWorld);
	return (FBox)CurBox;
}

FTransform UExtrudeMeshSelectionTool::GetTransform() const
{
	return TransformProxy->GetTransform();
}

void UExtrudeMeshSelectionTool::SetTransform(const FTransform& NewTransform)
{
	FTransformProxyChangeSource ChangeSource(TransformProxy);
	ChangeSource.BeginChange();

	TransformProxy->SetTransform(NewTransform);

	TUniquePtr<FToolCommandChange> Change = ChangeSource.EndChange();
	if (Change.IsValid())
	{
		GetToolManager()->EmitObjectChange(this, MoveTemp(Change), LOCTEXT("SetTransform", "Set Transform"));
	}
}

void UExtrudeMeshSelectionTool::Initialize_GizmoMechanic()
{
	// Set up the gizmo.
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform( InitialFrameWorld.ToFTransform() );

	//TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(
		GetToolManager()->GetPairedGizmoManager(),
		ETransformGizmoSubElements::FullTranslateRotateScale, this);

	TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());
	TransformProxy->OnTransformChanged.AddUObject(this, &UExtrudeMeshSelectionTool::GizmoTransformChanged);

	ExtrudeFrameWorld = InitialFrameWorld;
	ExtrudeFrameLocal = ExtrudeFrameWorld;
	ExtrudeFrameLocal.Transform( WorldTransform.InverseUnsafe() );
	LocalScale = FVector3d::One();
}

void UExtrudeMeshSelectionTool::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	ExtrudeFrameWorld = (FFrame3d)Transform;
	ExtrudeFrameLocal = ExtrudeFrameWorld;
	ExtrudeFrameLocal.Transform( WorldTransform.InverseUnsafe() );

	LocalScale = Transform.GetScale3D();

	EditCompute->InvalidateResult();
}


void UExtrudeMeshSelectionTool::UpdateVisualizationSettings()
{
	if (ExtrudeProperties->bShowInputMaterials)
	{
		EditCompute->DisablePreviewMaterials();
		Cast<UDynamicMeshComponent>(EditCompute->PreviewMesh->GetRootComponent())->SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode::None);
	}
	else
	{
		EditCompute->ConfigurePreviewMaterials(
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()),
			ToolSetupUtil::GetSelectionMaterial(FLinearColor::Yellow, GetToolManager())  );
		Cast<UDynamicMeshComponent>(EditCompute->PreviewMesh->GetRootComponent())->SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode::Polygroups);
	}
}


void UExtrudeMeshSelectionTool::UpdateInteractionMode(EExtrudeMeshSelectionInteractionMode InteractionMode)
{
	if ( InteractionMode == EExtrudeMeshSelectionInteractionMode::Interactive )
	{
		TransformGizmo->SetVisibility(true);
	}
	else
	{
		TransformGizmo->SetVisibility(false);
	}

	EditCompute->InvalidateResult();
}


#undef LOCTEXT_NAMESPACE
