// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polymodeling/OffsetMeshSelectionTool.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "MeshQueries.h"

#include "Selections/MeshFaceSelection.h"
#include "DynamicSubmesh3.h"
#include "Operations/MeshRegionOperator.h"
#include "Operations/OffsetMeshRegion.h"

#include "ModelingOperators.h"
#include "PolyModelingOps/RegionOffsetOp.h"
#include "Operations/PolyModeling/PolyModelingMaterialUtil.h"
#include "Operations/PolyModeling/PolyModelingFaceUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OffsetMeshSelectionTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UOffsetMeshSelectionTool"



USingleTargetWithSelectionTool* UOffsetMeshSelectionToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UOffsetMeshSelectionTool>(SceneState.ToolManager);
}



class FOffsetMeshSelectionOpFactory : public IDynamicMeshOperatorFactory
{
public:
	UOffsetMeshSelectionTool* SourceTool;

	FOffsetMeshSelectionOpFactory(UOffsetMeshSelectionTool* Tool)
	{
		SourceTool = Tool;
	}

	TUniquePtr<FRegionOffsetOp> MakeNewOffsetOp(
		TSharedPtr<FSharedConstDynamicMesh3> SourceMesh,
		const TArray<int32>& OffsetROI	)
	{
		TUniquePtr<FRegionOffsetOp> Op = MakeUnique<FRegionOffsetOp>();
		Op->OriginalMeshShared = SourceTool->EditRegionSharedMesh;
		Op->TriangleSelection = OffsetROI;

		Op->bShellsToSolids = SourceTool->OffsetProperties->bShellsToSolids;

		Op->bInferGroupsFromNeighbours = SourceTool->OffsetProperties->bInferGroupsFromNbrs;
		Op->bNewGroupPerSubdivision = SourceTool->OffsetProperties->bGroupPerSubdivision;
		Op->bUseColinearityForSettingBorderGroups = false;		// what is this??
		Op->bRemapExtrudeGroups = ! SourceTool->OffsetProperties->bReplaceSelectionGroups;

		Op->OffsetMode = FRegionOffsetOp::EOffsetComputationMode::VertexNormals;
		if ( SourceTool->OffsetProperties->Direction == EOffsetMeshSelectionDirectionMode::ConstantWidth )
		{
			Op->OffsetMode = FRegionOffsetOp::EOffsetComputationMode::ApproximateConstantThickness;
		}
		else if (SourceTool->OffsetProperties->Direction == EOffsetMeshSelectionDirectionMode::FaceNormals)
		{
			Op->OffsetMode = FRegionOffsetOp::EOffsetComputationMode::FaceNormals;
		}

		Op->OffsetDistance = SourceTool->OffsetProperties->OffsetDistance;

		//Op->bUseColinearityForSettingBorderGroups = OffsetProperties->bUseColinearityForSettingBorderGroups;
		//Op->MaxScaleForAdjustingTriNormalsOffset = OffsetProperties->MaxDistanceScaleFactor;

		Op->CreaseAngleThresholdDeg = SourceTool->OffsetProperties->CreaseAngle;

		Op->UVScaleFactor = SourceTool->OffsetProperties->UVScale;
		Op->bUVIslandPerGroup = SourceTool->OffsetProperties->bUVIslandPerGroup;

		Op->bInferMaterialID = SourceTool->OffsetProperties->bInferMaterialID;
		Op->SetMaterialID = FMath::Min(SourceTool->OffsetProperties->SetMaterialID, SourceTool->MaxMaterialID);

		Op->NumSubdivisions = SourceTool->OffsetProperties->NumSubdivisions;

		Op->SetResultTransform(SourceTool->WorldTransform);

		return Op;
	}

	virtual TUniquePtr<FDynamicMeshOperator> MakeNewOperator() override
	{
		return MakeNewOffsetOp(SourceTool->EditRegionSharedMesh, SourceTool->RegionOffsetROI.Array());
	}
};


UOffsetMeshSelectionTool::UOffsetMeshSelectionTool()
{
	SetToolDisplayName(LOCTEXT("ToolName", "Offset Selection"));
}


void UOffsetMeshSelectionTool::Setup()
{
	UToolTarget* UseTarget = Super::GetTarget();
	CurrentMesh = UE::ToolTarget::GetDynamicMeshCopy(UseTarget);

	// need valid MaterialIDs for MaterialID parameter
	FInterval1i MatIDRange;
	UE::Geometry::ComputeMaterialIDRange(CurrentMesh, MatIDRange);
	MaxMaterialID = MatIDRange.Max;

	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(UseTarget);
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(UseTarget);

	OffsetProperties = NewObject<UOffsetMeshSelectionToolProperties>(this);
	OffsetProperties->RestoreProperties(this);
	OffsetProperties->WatchProperty(OffsetProperties->OffsetDistance,
		[this](double) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->Direction,
		[this](EOffsetMeshSelectionDirectionMode) { EditCompute->InvalidateResult();});
	OffsetProperties->WatchProperty(OffsetProperties->NumSubdivisions,
		[this](int) { EditCompute->InvalidateResult();});
	OffsetProperties->WatchProperty(OffsetProperties->bShellsToSolids,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->UVScale,
		[this](double) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->bInferGroupsFromNbrs,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->bGroupPerSubdivision,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->bUVIslandPerGroup,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->bReplaceSelectionGroups,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->CreaseAngle,
		[this](double) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->bInferMaterialID,
		[this](bool) { EditCompute->InvalidateResult(); });
	OffsetProperties->WatchProperty(OffsetProperties->SetMaterialID,
		[this](int) { EditCompute->InvalidateResult(); });

	OffsetProperties->WatchProperty(OffsetProperties->bShowInputMaterials,
		[this](bool) { UpdateVisualizationSettings(); });

	AddToolPropertySource(OffsetProperties);

	// extract selection
	FMeshFaceSelection TriSelection(&CurrentMesh);
	const FGeometrySelection& Selection = GetGeometrySelection();
	UE::Geometry::EnumerateSelectionTriangles(Selection, CurrentMesh,
		[&](int32 tid) { TriSelection.Select(tid); },
		nullptr  /* no support for group layers yet */);
	OffsetROI = TriSelection.AsArray();

	// if we have an empty selection, just abort here
	if (OffsetROI.Num() == 0)
	{
		GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel,
			true, LOCTEXT("InvalidSelectionMessage", "OffsetMeshSelectionTool: input face selection was empty, cannot Offset"));
		return;
	}

	TriSelection.ExpandToOneRingNeighbours(2);
	ModifiedROI = TriSelection.AsSet();		// could steal here if it was possible

	SelectionBoundsWorld = (FBox)TMeshQueries<FDynamicMesh3>::GetTrianglesBounds(CurrentMesh, OffsetROI, WorldTransform);

	// create the preview object for the unmodified area
	// (if input was a UDynamicMesh we could do this w/o making a copy...)
	SourcePreview = NewObject<UPreviewMesh>();
	SourcePreview->CreateInWorld(GetTargetWorld(), WorldTransform);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(SourcePreview, UseTarget);
	SourcePreview->SetMaterials(MaterialSet.Materials);
	SourcePreview->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			return OffsetROI.Contains(TriangleID);		// hide triangles in Offset area in base mesh
		});
	SourcePreview->SetSecondaryBuffersVisibility(false);
	SourcePreview->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	SourcePreview->UpdatePreview(&CurrentMesh);

	// initialize a region operator for the modified area
	RegionOperator = MakePimpl<FMeshRegionOperator>(&CurrentMesh, ModifiedROI.Array());
	for (int32 tid : OffsetROI)
	{
		RegionOffsetROI.Add( RegionOperator->Region.MapTriangleToSubmesh(tid) );
	}
	EditRegionMesh = RegionOperator->Region.GetSubmesh();
	for (int32 tid : EditRegionMesh.TriangleIndicesItr())
	{
		if (RegionOffsetROI.Contains(tid) == false)
		{
			RegionBorderTris.Add(tid);
		}
	}
	EditRegionSharedMesh = MakeShared<FSharedConstDynamicMesh3>(&EditRegionMesh);

	// try to guess selection frame...
	SelectionFrameLocal = UE::Geometry::ComputeFaceSelectionFrame(EditRegionMesh, RegionOffsetROI.Array(), false);
	InitialFrameLocal = SelectionFrameLocal;
	InitialFrameWorld = InitialFrameLocal;
	InitialFrameWorld.Transform(WorldTransform);

	this->OperatorFactory = MakePimpl<FOffsetMeshSelectionOpFactory>(this);

	// Create the preview compute for the extrusion operation
	EditCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	EditCompute->Setup(GetTargetWorld(), this->OperatorFactory.Get());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(EditCompute->PreviewMesh, UseTarget);
	EditCompute->PreviewMesh->SetTransform((FTransform)WorldTransform);
	EditCompute->ConfigureMaterials(MaterialSet.Materials, nullptr, nullptr);

	// hide the triangles in the 'border' region outside the Offset area, although they are included in
	// the Offset region operator submesh, they are still visible in the base mesh
	EditCompute->PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID) { return RegionBorderTris.Contains(TriangleID); });
	EditCompute->PreviewMesh->SetSecondaryBuffersVisibility(false);

	UpdateVisualizationSettings();

	// is this the right tangents behavior?
	EditCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	EditCompute->PreviewMesh->UpdatePreview( & RegionOperator->Region.GetSubmesh() );
	EditCompute->SetVisibility(true);

	// hide input Component
	UE::ToolTarget::HideSourceObject(Target);

	// are these needed?
	OffsetFrameWorld = InitialFrameWorld;
	OffsetFrameLocal = OffsetFrameWorld;
	OffsetFrameLocal.Transform( WorldTransform.InverseUnsafe() );
	LocalScale = FVector3d::One();

	EditCompute->InvalidateResult();
}


void UOffsetMeshSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if ( OffsetProperties )
	{
		OffsetProperties->SaveProperties(this);
	}

	if (SourcePreview != nullptr )
	{
		SourcePreview->Disconnect();
		SourcePreview = nullptr;
	}

	if (EditCompute != nullptr)
	{
		// shut down background computation
		// (we don't actually care about compute result here because it was only for the live preview...)
		FDynamicMeshOpResult ComputeResult = EditCompute->Shutdown();
		EditCompute->ClearOpFactory();
		EditCompute->OnOpCompleted.RemoveAll(this);
		EditCompute = nullptr;

		UToolTarget* UseTarget = GetTarget();
		UE::ToolTarget::ShowSourceObject(UseTarget);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("OffsetSelection", "Offset Selection"));

			TArray<int32> SelectOutputTriangles;

			// this function computes the final Offset on full Mesh and cleans it up
			auto ComputeFinalResult = [this, &SelectOutputTriangles](FDynamicMesh3& EditMesh) -> bool
			{
				TSharedPtr<FSharedConstDynamicMesh3> FullMeshShared = MakeShared<FSharedConstDynamicMesh3>(&EditMesh);
				TUniquePtr<FRegionOffsetOp> FinalOp = OperatorFactory->MakeNewOffsetOp(FullMeshShared, this->OffsetROI);
				SelectOutputTriangles = this->OffsetROI;
				bool bResult = FinalOp->CalculateResultInPlace(EditMesh, nullptr);
				if (EditMesh.IsCompact() == false)
				{
					FCompactMaps CompactMaps;
					EditMesh.CompactInPlace(&CompactMaps);
					if (CompactMaps.TriangleMapIsSet())
					{
						for (int32& tid : SelectOutputTriangles)
						{
							tid = CompactMaps.GetTriangleMapping(tid);
						}
					}
				}

				FullMeshShared->ReleaseSharedObject();
				return bResult;
			};

			// try to emit an incremental FMeshChange, to avoid storing full mesh before/after copies in undo history
			bool bChangeApplied = false;
			if (UE::ToolTarget::SupportsIncrementalMeshChanges(UseTarget))
			{
				bChangeApplied = UE::ToolTarget::ApplyIncrementalMeshEditChange(UseTarget,
					[&](FDynamicMesh3& EditMesh, UObject* TransactionTarget)
					{
						FDynamicMeshChangeTracker ChangeTracker(&EditMesh);
						ChangeTracker.BeginChange();
						ChangeTracker.SaveTriangles(OffsetROI, true);
						if (ComputeFinalResult(EditMesh))
						{
							TUniquePtr<FDynamicMeshChange> MeshEditChange = ChangeTracker.EndChange();
							GetToolManager()->GetContextTransactionsAPI()->AppendChange( TransactionTarget,
								MakeUnique<FMeshChange>(MoveTemp(MeshEditChange)), LOCTEXT("OffsetSelection", "Offset Selection") );
							return true;
						}
						return false;
					});

				ensureMsgf(bChangeApplied, TEXT("UOffsetMeshSelectionTool::OnShutdown : incremental mesh edit failed!"));
			}

			// if we could not apply incremental change, apply a full-mesh update
			if ( bChangeApplied == false )
			{
				ComputeFinalResult(CurrentMesh);
				const bool bModifiedTopology = true;
				UE::ToolTarget::CommitDynamicMeshUpdate(UseTarget, CurrentMesh, bModifiedTopology);
			}

			// Construct and send output triangle selection, assume selection system will convert to correct type.
			// (todo: maybe be smarter about this, as for (eg) an input vertex selection, it will grow the selection...)
			// Note that we cannot do this before we commit/complete the mesh update, as the selection system needs a chance
			// to respond to the mesh change. Possibly should use UE::Geometry::InitializeSelectionFromTriangles() here...need to ProcessMesh() then though
			FGeometrySelection OutputSelection;
			OutputSelection.InitializeTypes(EGeometryElementType::Face, EGeometryTopologyType::Triangle);
			for (int32 tid : SelectOutputTriangles)
			{
				OutputSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
			}
			UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, UseTarget, OutputSelection);


			GetToolManager()->EndUndoTransaction();
		}
	}

	if ( EditRegionSharedMesh.IsValid() )
	{
		EditRegionSharedMesh->ReleaseSharedObject();
	}

	CurrentMesh.Clear();
}



void UOffsetMeshSelectionTool::OnTick(float DeltaTime)
{
	EditCompute->Tick(DeltaTime);
}


void UOffsetMeshSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

FBox UOffsetMeshSelectionTool::GetWorldSpaceFocusBox() 
{ 
	FAxisAlignedBox3d CurBox = SelectionBoundsWorld;
	FVector3d Translation = OffsetFrameWorld.Origin - InitialFrameWorld.Origin;
	CurBox.Min += Translation; CurBox.Max += Translation;
	CurBox.Contain(SelectionBoundsWorld);
	return (FBox)CurBox;
}


void UOffsetMeshSelectionTool::UpdateVisualizationSettings()
{
	if (OffsetProperties->bShowInputMaterials)
	{
		EditCompute->DisablePreviewMaterials();
		Cast<UDynamicMeshComponent>(EditCompute->PreviewMesh->GetRootComponent())->SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode::None);
	}
	else
	{
		EditCompute->ConfigurePreviewMaterials(
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()),
			ToolSetupUtil::GetSelectionMaterial(FLinearColor::Yellow, GetToolManager())  );
		Cast<UDynamicMeshComponent>(EditCompute->PreviewMesh->GetRootComponent())->SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode::Polygroups);
	}
}




#undef LOCTEXT_NAMESPACE
