// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshAnalysisProperties.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshQueries.h"
#include "MeshAdapter.h"
#include "DynamicMesh/MeshAdapterUtil.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Internationalization/Text.h"
#include "Math/BasicMathExpressionEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshAnalysisProperties)

#define LOCTEXT_NAMESPACE "UMeshAnalysisProperites"

using namespace UE::Geometry;

/** 
 * Utility function that converts a number to a string. The precision of the fractional part can be adjusted based on 
 * the magnitude of the input number.
 * 
 * @param Number to convert to a string
 * @param MagnitudeToFractionalDigits Array of tuples (x, y) such that if the Number is less than x then set the fractional precision to y. 
 * @param DefaultMaxFractionalDigits Default precision if the Number doesn't fall into any range specified in MagnitudeToFractionalDigits
 */
template<typename RealType>
FString NumberToStringWithFractionalPrecision(const RealType Number, 
											  const TArray<TTuple<RealType, int32>>& MagnitudeToFractionalDigits, 
											  const int DefaultMaxFractionalDigits = 6) 
{
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.SetUseGrouping(false);
	NumberFormattingOptions.SetMaximumFractionalDigits(DefaultMaxFractionalDigits);
	
	if (MagnitudeToFractionalDigits.Num() > 0) 
	{
		TArray<TTuple<RealType, int32>> SortedMagnitudeToFractionalDigits = MagnitudeToFractionalDigits;
		SortedMagnitudeToFractionalDigits.Sort([](const TTuple<RealType, int32>& LHS, const TTuple<RealType, int32>& RHS) { return LHS.Key < RHS.Key; });

		if (Number < SortedMagnitudeToFractionalDigits.Last().Key) // First check if the number falls into any range
		{
			for (const TTuple<RealType, int32>& Tuple : SortedMagnitudeToFractionalDigits) 
			{
				if (Number < Tuple.Key)
				{
					NumberFormattingOptions.SetMaximumFractionalDigits(Tuple.Value);
					break;
				}
			}
		}
	}

	return FastDecimalFormat::NumberToString(Number, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
}

void UMeshAnalysisProperties::Update(const FDynamicMesh3& MeshIn, const FTransform& Transform)
{
	FTriangleMeshAdapterd TransformedMesh = UE::Geometry::MakeTransformedDynamicMeshAdapter(&MeshIn, (FTransform3d)Transform);
	FVector2d VolArea = TMeshQueries<FTriangleMeshAdapterd>::GetVolumeArea(TransformedMesh);
	
	TArray<TTuple<double, int32>> MagnitudeToFractionalDigits;
	MagnitudeToFractionalDigits.Add(MakeTuple(0.1, 8)); // If the number is less than 0.1 show 8 fractional digits
	MagnitudeToFractionalDigits.Add(MakeTuple(1, 6)); // If the number is less than 1 show 6 fractional digits
	
	const double VolInMeters = VolArea[0]/1000000;
	const double AreaInMeters = VolArea[1]/10000;
	
	this->Volume = NumberToStringWithFractionalPrecision(VolInMeters, MagnitudeToFractionalDigits, 4);
	this->SurfaceArea = NumberToStringWithFractionalPrecision(AreaInMeters, MagnitudeToFractionalDigits, 4);
}



#undef LOCTEXT_NAMESPACE

