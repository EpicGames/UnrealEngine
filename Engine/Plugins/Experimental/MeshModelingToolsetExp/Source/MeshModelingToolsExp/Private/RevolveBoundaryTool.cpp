// Copyright Epic Games, Inc. All Rights Reserved.

#include "RevolveBoundaryTool.h"

#include "ModelingObjectsCreationAPI.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "CoreMinimal.h"
#include "CompositionOps/CurveSweepOp.h"
#include "InteractiveToolManager.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "Selection/PolygonSelectionMechanic.h"
#include "GroupTopology.h"
#include "ToolBuilderUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingToolTargetUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevolveBoundaryTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "URevolveBoundaryTool"

// Tool builder

USingleSelectionMeshEditingTool* URevolveBoundaryToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<URevolveBoundaryTool>(SceneState.ToolManager);
}
bool URevolveBoundaryToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// We're disallowing volumes here because volumes won't have open boundaries intrinsically.
	return USingleSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

// Operator factory

TUniquePtr<FDynamicMeshOperator> URevolveBoundaryOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FCurveSweepOp> CurveSweepOp = MakeUnique<FCurveSweepOp>();
	
	// Assemble profile curve
	const FGroupTopologySelection& ActiveSelection = RevolveBoundaryTool->SelectionMechanic->GetActiveSelection();
	if (ActiveSelection.SelectedEdgeIDs.Num() == 1)
	{
		int32 EdgeID = ActiveSelection.GetASelectedEdgeID();
		if (RevolveBoundaryTool->Topology->IsBoundaryEdge(EdgeID))
		{
			const TArray<int32>& VertexIndices = RevolveBoundaryTool->Topology->GetGroupEdgeVertices(EdgeID);
			FTransform ToWorld = Cast<IPrimitiveComponentBackedTarget>(RevolveBoundaryTool->Target)->GetWorldTransform();

			// If boundary loop includes the last vertex as first, stop early.
			// (Note: This is generally true unless the mesh has bowties that confuse the boundary walk.)
			bool bIsLoop = VertexIndices.Num() && VertexIndices.Last() == VertexIndices[0];
			CurveSweepOp->ProfileCurve.Reserve(VertexIndices.Num() - (int32)bIsLoop);
			for (int32 i = 0; i < VertexIndices.Num() - (int32)bIsLoop; ++i)
			{
				int32 VertIndex = VertexIndices[i];

				FVector3d NewPos = (FVector3d)ToWorld.TransformPosition((FVector)RevolveBoundaryTool->OriginalMesh->GetVertex(VertIndex));
				CurveSweepOp->ProfileCurve.Add(NewPos);
			}
			CurveSweepOp->bProfileCurveIsClosed = bIsLoop;
		}
	}

	RevolveBoundaryTool->Settings->ApplyToCurveSweepOp(*RevolveBoundaryTool->MaterialProperties,
		RevolveBoundaryTool->RevolutionAxisOrigin, RevolveBoundaryTool->RevolutionAxisDirection, *CurveSweepOp);

	return CurveSweepOp;
}


// Tool itself

void URevolveBoundaryTool::Setup()
{
	UMeshBoundaryToolBase::Setup();

	// We're actually going to handle the selection clicks ourselves so that we can align axis if
	// we want to.
	SelectionMechanic->DisableBehaviors(this);
	SelectionMechanic->SetShouldAddToSelectionFunc([]() {return false; });
	SelectionMechanic->SetShouldRemoveFromSelectionFunc([]() {return false; });

	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ClickBehavior->Modifiers.RegisterModifier(CtrlModifier, FInputDeviceState::IsCtrlKeyDown);
	ClickBehavior->Modifiers.RegisterModifier(ShiftModifier, FInputDeviceState::IsShiftKeyDown);
	AddInputBehavior(ClickBehavior);
	
	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	Settings = NewObject<URevolveBoundaryToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	AddToolPropertySource(MaterialProperties);
	MaterialProperties->RestoreProperties(this);


	FTransform LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
	Settings->AxisOrigin = LocalToWorld.GetTranslation();

	UpdateRevolutionAxis();

	// The plane mechanic is used for the revolution axis
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), FFrame3d(Settings->AxisOrigin, 
		FRotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0).Quaternion()));
	PlaneMechanic->UpdateClickPriority(ClickBehavior->GetPriority().MakeLower());
	PlaneMechanic->bShowGrid = false;
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		Settings->AxisOrigin = (FVector)PlaneMechanic->Plane.Origin;
		FRotator AxisOrientation = ((FQuat)PlaneMechanic->Plane.Rotation).Rotator();
		Settings->AxisOrientation.X = AxisOrientation.Pitch;
		Settings->AxisOrientation.Y = AxisOrientation.Yaw;
		NotifyOfPropertyChangeByTool(Settings);
		UpdateRevolutionAxis();
		});

	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(Settings->bDisplayInputMesh);

	SetToolDisplayName(LOCTEXT("ToolName", "Revolve Boundary"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartRevolveBoundaryTool", "Revolve an open mesh boundary loop around an axis to create a new mesh. Ctrl+Click to align the axis to a surface/edge."),
		EToolMessageLevel::UserNotification);
	if (Topology->Edges.Num() == 1)
	{
		FGroupTopologySelection Selection;
		Selection.SelectedEdgeIDs.Add(0);
		SelectionMechanic->SetSelection(Selection);
		StartPreview();
	}
	else if (Topology->Edges.Num() == 0)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoBoundaryLoops", "This mesh does not have any boundary loops to display and revolve. Delete faces to create a boundary or use a different mesh."),
			EToolMessageLevel::UserWarning);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartRevolveBoundaryToolMultipleBoundaries", "Your mesh has multiple boundaries. Select the one you wish to use."),
			EToolMessageLevel::UserWarning);
	}
}

void URevolveBoundaryTool::OnUpdateModifierState(int ModifierId, bool bIsOn)
{
	if (ModifierId == CtrlModifier)
	{
		// Like with the plane mechanic, clicking an edge while holding ctrl should move the axis
		// to that point and (by default) align it.
		bMoveAxisOnClick = bIsOn;
	}
	else if (ModifierId == ShiftModifier)
	{
		// Like with the plane mechanic, holding ctrl + shift shifts the axis to the point without
		// changing the orientation
		bAlignAxisOnClick = !bIsOn;
	}
}

FInputRayHit URevolveBoundaryTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FHitResult OutHit;
	if (SelectionMechanic->TopologyHitTest(ClickPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return FInputRayHit(); // bHit is false
}

void URevolveBoundaryTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	// Update selection only if we clicked on something. We don't want to be able to
	// clear a selection with a click.
	FHitResult HitResult;
	if (SelectionMechanic->TopologyHitTest(ClickPos.WorldRay, HitResult))
	{
		FVector3d LocalHitPosition, LocalHitNormal;
		SelectionMechanic->UpdateSelection(ClickPos.WorldRay, LocalHitPosition, LocalHitNormal);

		// Clear the "multiple boundaries" warning, since we've selected one.
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

		// Act on Ctrl/Shift modifiers the same way that the plane mechanic does.
		if (bMoveAxisOnClick)
		{
			const FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();
			int32 ClickedEid = Topology->GetGroupEdgeEdges(Selection.GetASelectedEdgeID())[HitResult.Item];
			
			FVector3d VertexA, VertexB;
			OriginalMesh->GetEdgeV(ClickedEid, VertexA, VertexB);
			FTransform ToWorldTranform = Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform();
			FLine3d EdgeLine = FLine3d::FromPoints((FVector3d)ToWorldTranform.TransformPosition((FVector)VertexA), 
				(FVector3d)ToWorldTranform.TransformPosition((FVector)VertexB));
			
			// New frame starts as the old one with modified origin
			FFrame3d RevolutionAxisFrame(EdgeLine.NearestPoint((FVector3d)HitResult.ImpactPoint), PlaneMechanic->Plane.Rotation);
			
			if (bAlignAxisOnClick)
			{
				RevolutionAxisFrame.AlignAxis(0, EdgeLine.Direction);
			}

			PlaneMechanic->SetPlaneWithoutBroadcast(RevolutionAxisFrame);

			Settings->AxisOrigin = (FVector)RevolutionAxisFrame.Origin;
			FRotator AxisOrientation = ((FQuat)RevolutionAxisFrame.Rotation).Rotator();
			Settings->AxisOrientation.X = AxisOrientation.Pitch;
			Settings->AxisOrientation.Y = AxisOrientation.Yaw;
			UpdateRevolutionAxis();
		}

		// Update the preview
		if (Preview == nullptr)
		{
			StartPreview();
		}
		else
		{
			Preview->InvalidateResult();
		}
	}
}

bool URevolveBoundaryTool::CanAccept() const
{
	return Preview != nullptr && Preview->HaveValidNonEmptyResult();
}

/** 
 * Uses the settings stored in the properties object to update the revolution axis
 */
void URevolveBoundaryTool::UpdateRevolutionAxis()
{
	RevolutionAxisOrigin = (FVector3d)Settings->AxisOrigin;
	RevolutionAxisDirection = (FVector3d)FRotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0).RotateVector(FVector(1, 0, 0));
	if (Preview)
	{
		Preview->InvalidateResult();
	}
}

void URevolveBoundaryTool::StartPreview()
{
	URevolveBoundaryOperatorFactory* RevolveBoundaryOpCreator = NewObject<URevolveBoundaryOperatorFactory>();
	RevolveBoundaryOpCreator->RevolveBoundaryTool = this;

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(RevolveBoundaryOpCreator);
	Preview->Setup(GetTargetWorld(), RevolveBoundaryOpCreator);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

	Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);

	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute* UpdatedPreview)
		{
			UpdateAcceptWarnings(UpdatedPreview->HaveEmptyResult() ? EAcceptWarning::EmptyForbidden : EAcceptWarning::NoWarning);
		}
	);

	Preview->SetVisibility(true);
	Preview->InvalidateResult();
}

void URevolveBoundaryTool::OnShutdown(EToolShutdownType ShutdownType)
{
	UMeshBoundaryToolBase::OnShutdown(ShutdownType);

	OutputTypeProperties->SaveProperties(this);

	Settings->SaveProperties(this);
	MaterialProperties->SaveProperties(this);

	PlaneMechanic->Shutdown();

	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	if (Preview)
	{
		if (ShutdownType == EToolShutdownType::Accept)
		{
			Preview->PreviewMesh->CalculateTangents();
			GenerateAsset(Preview->Shutdown());
		}
		else
		{
			Preview->Cancel();
		}
	}
}

void URevolveBoundaryTool::GenerateAsset(const FDynamicMeshOpResult& OpResult)
{
	if (OpResult.Mesh->TriangleCount() <= 0)
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("RevolveBoundaryToolTransactionName", "Boundary Revolve Tool"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = GetTargetWorld();
	NewMeshObjectParams.Transform = (FTransform)OpResult.Transform;
	NewMeshObjectParams.BaseName = TEXT("Revolve");
	NewMeshObjectParams.Materials.Add(MaterialProperties->Material.Get());
	NewMeshObjectParams.SetMesh(OpResult.Mesh.Get());
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();
}

void URevolveBoundaryTool::OnTick(float DeltaTime)
{
	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Tick(DeltaTime);
	}

	if (Preview)
	{
		Preview->Tick(DeltaTime);
	}
}

void URevolveBoundaryTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshBoundaryToolBase::Render(RenderAPI);

	FViewCameraState CameraState;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Render(RenderAPI);

		// Draw the axis of rotation
		float PdiScale = CameraState.GetPDIScalingFactor();
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		FColor AxisColor(240, 16, 240);
		double AxisThickness = 1 * PdiScale;
		double AxisHalfLength = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, RevolutionAxisOrigin, 90);

		FVector3d StartPoint = RevolutionAxisOrigin - (RevolutionAxisDirection * (AxisHalfLength * PdiScale));
		FVector3d EndPoint = RevolutionAxisOrigin + (RevolutionAxisDirection * (AxisHalfLength * PdiScale));

		PDI->DrawLine((FVector)StartPoint, (FVector)EndPoint, AxisColor, SDPG_Foreground,
			AxisThickness, 0.0f, true);
	}
}

void URevolveBoundaryTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	PlaneMechanic->SetPlaneWithoutBroadcast(FFrame3d(Settings->AxisOrigin, 
		FRotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0).Quaternion()));
	UpdateRevolutionAxis();

	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(Settings->bDisplayInputMesh);

	if (Preview)
	{
		if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNewMeshMaterialProperties, Material)))
		{
			Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
				ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
		}

		Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
		Preview->InvalidateResult();
	}
}

#undef LOCTEXT_NAMESPACE

