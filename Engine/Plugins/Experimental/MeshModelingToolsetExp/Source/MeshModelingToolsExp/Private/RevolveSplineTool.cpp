// Copyright Epic Games, Inc. All Rights Reserved.

#include "RevolveSplineTool.h"

#include "CompGeom/PolygonTriangulation.h"
#include "CompositionOps/CurveSweepOp.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GameFramework/Actor.h"
#include "InteractiveToolManager.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "Selection/ToolSelectionUtil.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevolveSplineTool)

#define LOCTEXT_NAMESPACE "URevolveSplineTool"

using namespace UE::Geometry;



void URevolveSplineTool::Setup()
{
	UInteractiveTool::Setup();

	Settings = NewObject<URevolveSplineToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	ToolActions = NewObject<URevolveSplineToolActionPropertySet>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	SetToolDisplayName(LOCTEXT("RevolveSplineToolName", "Revolve Spline"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("RevolveSplineToolDescription", "Revolve the selected spline to create a mesh."),
		EToolMessageLevel::UserNotification);

	// TODO: We'll probably want a click behavior someday for clicking on the spline to align to a tangent at a point

	// The plane mechanic is used for the revolution axis.
	// Note: The only thing we really end up using from it is the gizmo and the control+click. We
	// could use our own gizmo directly.
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(GetTargetWorld(), FFrame3d(Settings->AxisOrigin, 
		FRotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0).Quaternion()));
	PlaneMechanic->bShowGrid = false;
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		Settings->AxisOrigin = (FVector)PlaneMechanic->Plane.Origin;
		FRotator AxisOrientation = ((FQuat)PlaneMechanic->Plane.Rotation).Rotator();
		Settings->AxisOrientation.X = AxisOrientation.Pitch;
		Settings->AxisOrientation.Y = AxisOrientation.Yaw;
		NotifyOfPropertyChangeByTool(Settings);
		UpdateRevolutionAxis();
		});
	// Add if we get our own click behavior:
	//PlaneMechanic->UpdateClickPriority(ClickBehavior->GetPriority().MakeLower());

	// TODO: It would be nice to have a drag alignment mechanic for the above gizmo, but we currently
	// don't have a way to pass in a custom alignment raycast, which we would want in order to snap
	// and align to spline points.

	if (Settings->bResetAxisOnStart)
	{
		ResetAxis();
	}
	else
	{
		UpdateRevolutionAxis();
	}

	Super::Setup();
}

void URevolveSplineTool::ResetAxis()
{
	USplineComponent* Spline = GetFirstSpline();
	if (!Spline)
	{
		return;
	}
	int32 NumSplinePoints = Spline->GetNumberOfSplinePoints();
	if (NumSplinePoints == 0)
	{
		return;
	}
	Settings->AxisOrigin = Spline->GetLocationAtSplinePoint(0, ESplineCoordinateSpace::World);

	// Our axis is the X axis of the frame, and we align it to Last-First
	FVector3d PlaneX = Spline->GetLocationAtSplinePoint(NumSplinePoints-1, ESplineCoordinateSpace::World) - Settings->AxisOrigin;
	PlaneX.Normalize();
	FFrame3d PlaneFrame(Settings->AxisOrigin, SplineFitPlaneNormal);
	if (!PlaneX.IsZero())
	{
		FVector3d PlaneY = SplineFitPlaneNormal.Cross(PlaneX);
		FVector3d PlaneZ = PlaneX.Cross(PlaneY);
		PlaneFrame = FFrame3d(Settings->AxisOrigin, PlaneX, PlaneY, PlaneZ);
	}
	FRotator AxisOrientation = ((FQuat)PlaneFrame.Rotation).Rotator();
	Settings->AxisOrientation.X = AxisOrientation.Pitch;
	Settings->AxisOrientation.Y = AxisOrientation.Yaw;

	NotifyOfPropertyChangeByTool(Settings);
	UpdateRevolutionAxis();
}

void URevolveSplineTool::OnSplineUpdate()
{
	USplineComponent* Spline = GetFirstSpline();
	if (!Spline)
	{
		return;
	}

	bProfileCurveIsClosed = Spline->IsClosedLoop();

	// Update the curve plane
	TArray<FVector> SplineControlPoints;
	int32 NumSplinePoints = Spline->GetNumberOfSplinePoints();
	for (int32 i = 0; i < NumSplinePoints; ++i)
	{
		SplineControlPoints.Add(Spline->GetLocationAtSplinePoint(i, ESplineCoordinateSpace::World));
	}
	PolygonTriangulation::ComputePolygonPlane<double>(SplineControlPoints, SplineFitPlaneNormal, SplineFitPlaneOrigin);

	// Update the points we actually revolve
	ProfileCurve.Reset();
	switch (Settings->SampleMode)
	{
	case ERevolveSplineSampleMode::ControlPointsOnly:
	{
		ProfileCurve = SplineControlPoints;
	}
	break;
	case ERevolveSplineSampleMode::PolyLineMaxError:
	{
		Spline->ConvertSplineToPolyLine(ESplineCoordinateSpace::World,
			Settings->ErrorTolerance * Settings->ErrorTolerance,
			ProfileCurve);
	};
	break;
	case ERevolveSplineSampleMode::UniformSpacingAlongCurve:
	{
		double Length = Spline->GetSplineLength();
		int32 NumSegments = FMath::RoundFromZero(Length / FMath::Max(0.01, Settings->MaxSampleDistance));
		double SegmentLengthToUse = Length / NumSegments;
		for (int32 i = 0; i <= NumSegments; ++i)
		{
			double DistanceToSampleAt = Length * ((double)i / NumSegments); //-V609
			ProfileCurve.Add(Spline->GetLocationAtDistanceAlongSpline(DistanceToSampleAt, 
				ESplineCoordinateSpace::World));
		}
	}
	break;
	}

	Preview->InvalidateResult();
}

void URevolveSplineTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (PlaneMechanic)
	{
		PlaneMechanic->Tick(DeltaTime);
	}
}

void URevolveSplineTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);

	FViewCameraState CameraState;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (PlaneMechanic)
	{
		PlaneMechanic->Render(RenderAPI);

		// Draw the axis of rotation
		float PdiScale = CameraState.GetPDIScalingFactor();
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		FColor AxisColor(240, 16, 240);
		double AxisThickness = 1 * PdiScale;
		double AxisHalfLength = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, RevolutionAxisOrigin, 90);

		FVector3d StartPoint = RevolutionAxisOrigin - (RevolutionAxisDirection * (AxisHalfLength * PdiScale));
		FVector3d EndPoint = RevolutionAxisOrigin + (RevolutionAxisDirection * (AxisHalfLength * PdiScale));

		PDI->DrawLine((FVector)StartPoint, (FVector)EndPoint, AxisColor, SDPG_Foreground,
			AxisThickness, 0.0f, true);
	}
}

void URevolveSplineTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(URevolveSplineToolProperties, SampleMode)
			|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(URevolveSplineToolProperties, ErrorTolerance)
			|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(URevolveSplineToolProperties, MaxSampleDistance))
		{
			OnSplineUpdate();
		}

		// Checking the name for these settings doesn't work, since the reported names are the low level components, like "X" or "Y"
		// So we'll simply update the axis whenever any property changes. It's overkill but probably not too bad.
		PlaneMechanic->SetPlaneWithoutBroadcast(FFrame3d(Settings->AxisOrigin,
			FRotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0).Quaternion()));
		UpdateRevolutionAxis();
	}

	Super::OnPropertyModified(PropertySet, Property);
}

void URevolveSplineTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);

	Super::Shutdown(ShutdownType);

	PlaneMechanic->Shutdown();

	Settings = nullptr;
	ToolActions = nullptr;
	PlaneMechanic = nullptr;
}

FString URevolveSplineTool::GeneratedAssetBaseName() const
{
	return TEXT("RevolveSpline");
}
FText URevolveSplineTool::TransactionName() const
{
	return LOCTEXT("RevolveSplinesAction", "Revolve Spline");
}

TUniquePtr<FDynamicMeshOperator> URevolveSplineTool::MakeNewOperator()
{
	TUniquePtr<FCurveSweepOp> CurveSweepOp = MakeUnique<FCurveSweepOp>();

	// Assemble profile curve
	CurveSweepOp->ProfileCurve = ProfileCurve;
	CurveSweepOp->bProfileCurveIsClosed = bProfileCurveIsClosed;

	// If we are capping the top and bottom, we just add a couple extra vertices and mark the curve as being closed
	if (!bProfileCurveIsClosed && Settings->bClosePathToAxis && ProfileCurve.Num() > 1)
	{
		// Project first and last points onto the revolution axis.
		FVector3d FirstPoint = CurveSweepOp->ProfileCurve[0];
		FVector3d LastPoint = CurveSweepOp->ProfileCurve.Last();
		double DistanceAlongAxis = RevolutionAxisDirection.Dot(LastPoint - RevolutionAxisOrigin);
		FVector3d ProjectedPoint = RevolutionAxisOrigin + (RevolutionAxisDirection * DistanceAlongAxis);

		CurveSweepOp->ProfileCurve.Add(ProjectedPoint);

		DistanceAlongAxis = RevolutionAxisDirection.Dot(FirstPoint - RevolutionAxisOrigin);
		ProjectedPoint = RevolutionAxisOrigin + (RevolutionAxisDirection * DistanceAlongAxis);

		CurveSweepOp->ProfileCurve.Add(ProjectedPoint);
		CurveSweepOp->bProfileCurveIsClosed = true;
	}

	Settings->ApplyToCurveSweepOp(*MaterialProperties,
		RevolutionAxisOrigin, RevolutionAxisDirection, *CurveSweepOp);

	return CurveSweepOp;
}


//Uses the settings stored in the properties object to update the revolution axis
void URevolveSplineTool::UpdateRevolutionAxis()
{
	RevolutionAxisOrigin = (FVector3d)Settings->AxisOrigin;
	
	FRotator Rotator(Settings->AxisOrientation.X, Settings->AxisOrientation.Y, 0);
	RevolutionAxisDirection = (FVector3d)Rotator.RotateVector(FVector(1, 0, 0));

	PlaneMechanic->SetPlaneWithoutBroadcast(FFrame3d(Settings->AxisOrigin, Rotator.Quaternion()));

	if (Preview)
	{
		Preview->InvalidateResult();
	}
}

void URevolveSplineTool::RequestAction(ERevolveSplineToolAction Action)
{
	if (Action == ERevolveSplineToolAction::ResetAxis)
	{
		ResetAxis();
	}
}


/// Tool builder:

UInteractiveTool* URevolveSplineToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	URevolveSplineTool* NewTool = NewObject<URevolveSplineTool>(SceneState.ToolManager);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

// Action set:

void URevolveSplineToolActionPropertySet::PostAction(ERevolveSplineToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


#undef LOCTEXT_NAMESPACE
