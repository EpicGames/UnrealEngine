// Copyright Epic Games, Inc. All Rights Reserved.

#include "SeamSculptTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ToolDataVisualizer.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Util/ColorConstants.h"
#include "Changes/ToolCommandChangeSequence.h"
#include "Changes/MeshChange.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMeshEditor.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshUVTransforms.h"
#include "Parameterization/MeshDijkstra.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshToMeshDescription.h"
#include "ToolTargetManager.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include "ModelingToolTargetUtil.h"

#include "BaseGizmos/BrushStampIndicator.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SeamSculptTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshSelectionTool"


/*
 * ToolBuilder
 */
UMeshSurfacePointTool* USeamSculptToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	USeamSculptTool* SeamSculptTool = NewObject<USeamSculptTool>(SceneState.ToolManager);
	SeamSculptTool->SetWorld(SceneState.World);
	return SeamSculptTool;
}


bool USeamSculptToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState) && 
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) > 0;
}



void USeamSculptTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

USeamSculptTool::USeamSculptTool()
{
	SetToolDisplayName(LOCTEXT("SeamSculptToolName", "Add UV Seams"));
}

void USeamSculptTool::Setup()
{
	UDynamicMeshBrushTool::Setup();

	// hide strength and falloff
	BrushProperties->bShowStrength = BrushProperties->bShowFalloff = false;
	BrushProperties->RestoreProperties(this);

	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	MeshTransform = FTransform3d(TargetComponent->GetWorldTransform());
	InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*PreviewMesh->GetMesh());
	FMeshNormals::QuickComputeVertexNormals(*InputMesh);
	FAxisAlignedBox3d MeshBounds = InputMesh->GetBounds(true);
	NormalOffset = MeshBounds.MinDim() * 0.001;
	MeshMaxDim = MeshBounds.MaxDim();

	// disable shadows
	//PreviewMesh->GetRootComponent()->bCastDynamicShadow = false;

	Settings = NewObject<USeamSculptToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	SetToolPropertySourceEnabled(BrushProperties, false);

	Settings->WatchProperty(Settings->bShowWireframe,
		[this](bool bNewValue) { PreviewMesh->EnableWireframe(bNewValue); });

	RecalculateBrushRadius();

	PreviewGeom = NewObject<UPreviewGeometry>(this);
	PreviewGeom->CreateInWorld(TargetComponent->GetOwnerActor()->GetWorld(), TargetComponent->GetWorldTransform());
	InitPreviewGeometry();

	// regenerate preview geo if mesh changes due to undo/redo/etc
	PreviewMesh->GetOnMeshChanged().AddLambda([this]() { bPreviewGeometryNeedsUpdate = true; });

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartSeamSculptTool", "Draw UV seams on the mesh by left click-dragging between mesh vertices."),
		EToolMessageLevel::UserNotification);

	// We don't need this to be visible for the seam editing process.
	BrushStampIndicator->bVisible = false;
}




void USeamSculptTool::OnShutdown(EToolShutdownType ShutdownType)
{
	PreviewGeom->Disconnect();

	Settings->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// force seams to be disjoint
		FDynamicMesh3 ResultMesh(*PreviewMesh->GetMesh());
		UE::MeshUVTransforms::MakeSeamsDisjoint(ResultMesh.Attributes()->GetUVLayer(0));

		// remove bowties?

		GetToolManager()->BeginUndoTransaction(LOCTEXT("SeamSculptTransactionName", "UV Seam Edit"));

		UE::ToolTarget::CommitDynamicMeshUpdate(Target, ResultMesh, true);

		GetToolManager()->EndUndoTransaction();
	}

}


void USeamSculptTool::OnTick(float DeltaTime)
{
	if (bPreviewGeometryNeedsUpdate)
	{
		UpdatePreviewGeometry();
		bPreviewGeometryNeedsUpdate = false;
	}
}


void USeamSculptTool::OnBeginDrag(const FRay& Ray)
{
	UBaseBrushTool::OnBeginDrag(Ray);

	if (CurrentSnapVertex >= 0)
	{
		CaptureState = EActiveCaptureState::DrawNewPath;
		DrawPathStartVertex = CurrentSnapVertex;
		DrawPathStartPositionLocal = CurrentSnapPositionLocal;
		LongTransactions.Open(LOCTEXT("CreateSeamChange", "Add UV Seam"), GetToolManager());

		CurDrawPath.Reset();
	}
}

void USeamSculptTool::OnUpdateDrag(const FRay& Ray)
{
	UBaseBrushTool::OnUpdateDrag(Ray);

	FVector3d LocalRayDir = MeshTransform.InverseTransformVector((FVector3d)Ray.Direction);
	FVector3d LocalPos = MeshTransform.InverseTransformPosition((FVector3d)LastBrushStamp.WorldPosition);
	double NearDistSqr;
	int32 NewSnapVertex = PreviewMesh->GetSpatial()->FindNearestVertex(LocalPos, NearDistSqr);
	FVector3d SnapNormal = (FVector3d)InputMesh->GetVertexNormal(NewSnapVertex);
	if (Settings->bHitBackFaces == false && SnapNormal.Dot(LocalRayDir) > 0)
	{
		return;
	}

	FVector3d NewSnapPositionLocal = PreviewMesh->GetMesh()->GetVertex(NewSnapVertex);

	if (CaptureState == EActiveCaptureState::DrawNewPath)
	{
		CurrentSnapVertex = NewSnapVertex;
		CurrentSnapPositionLocal = NewSnapPositionLocal;
		UpdateCurrentDrawPath();
	}
}

void USeamSculptTool::OnEndDrag(const FRay& Ray)
{
	UBaseBrushTool::OnEndDrag(Ray);

	if (CaptureState == EActiveCaptureState::DrawNewPath)
	{
		CreateSeamAlongPath();
		CurDrawPath.Reset();
		LongTransactions.Close(GetToolManager());
	}

	CaptureState = EActiveCaptureState::NoState;
}

bool USeamSculptTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	bool bHit = UDynamicMeshBrushTool::OnUpdateHover(DevicePos);
	if (bHit)
	{
		FVector3d LocalRayDir = MeshTransform.InverseTransformVector((FVector3d)DevicePos.WorldRay.Direction);
		FVector3d LocalPos = MeshTransform.InverseTransformPosition((FVector3d)LastBrushStamp.WorldPosition);
		double NearDistSqr;
		int32 NewSnapVertex = PreviewMesh->GetSpatial()->FindNearestVertex(LocalPos, NearDistSqr);
		FVector3d SnapNormal = (FVector3d)InputMesh->GetVertexNormal(NewSnapVertex);
		if (Settings->bHitBackFaces || SnapNormal.Dot(LocalRayDir) < 0)
		{
			CurrentSnapVertex = NewSnapVertex;
			CurrentSnapPositionLocal = PreviewMesh->GetMesh()->GetVertex(CurrentSnapVertex);
		}
	}

	return bHit;
}


void USeamSculptTool::UpdateCurrentDrawPath()
{
	CurDrawPath.Reset();
	if (DrawPathStartVertex == CurrentSnapVertex || CurrentSnapVertex < 0 || DrawPathStartVertex < 0)
	{
		return;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	TMeshDijkstra<FDynamicMesh3> PathFinder(Mesh);

	if (Settings->PathSimilarityWeight > 0 && MeshMaxDim > 0)
	{
		PathFinder.bEnableDistanceWeighting = true;

		FLine3d DirectLine = FLine3d::FromPoints(Mesh->GetVertex(DrawPathStartVertex), Mesh->GetVertex(CurrentSnapVertex));
		PathFinder.GetWeightedDistanceFunc = [this, Mesh, DirectLine](int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance)
		{
			double SimilarityMetric = SquaredDistanceFromLineIntegratedAlongSegment(
				DirectLine, FSegment3d(Mesh->GetVertex(FromVid), Mesh->GetVertex(ToVid)));

			// We want the relative path weights to not change if the mesh is uniformly scaled by a positive scalar s.
			//  Our similarity metric is proportional to s^3, so divide by max dim squared to make it proportional
			//  to s like EuclideanDistance, so that the final result is just scaled by s. We don't do cube root 
			//  because our adjustment needs to be distributive, to avoid being sensitive to path tesselation.
			SimilarityMetric /= (MeshMaxDim * MeshMaxDim);

			return EuclideanDistance + Settings->PathSimilarityWeight * SimilarityMetric;
		};
	}

	TArray<TMeshDijkstra<FDynamicMesh3>::FSeedPoint> SeedPoints;
	SeedPoints.Add({ DrawPathStartVertex, DrawPathStartVertex, 0 });
	if ( PathFinder.ComputeToTargetPoint(SeedPoints, CurrentSnapVertex) )
	{
		PathFinder.FindPathToNearestSeed(CurrentSnapVertex, CurDrawPath);
	}
}




void USeamSculptTool::ApplyStamp(const FBrushStampData& Stamp)
{
}



void USeamSculptTool::InitPreviewGeometry()
{
	FColor BoundaryColor = FColor(15, 15, 255);
	float BoundaryThickness = 6.0;
	FColor SeamColor = FColor(25, 150, 25);
	float SeamThickness = 4.0;

	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();

	int32 UVLayer = 0;
	const FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->GetUVLayer(UVLayer);


	ULineSetComponent* BoundaryLines = PreviewGeom->AddLineSet(TEXT("BoundaryEdges"));
	for (int32 eid : Mesh->EdgeIndicesItr())
	{
		if (Mesh->IsBoundaryEdge(eid))
		{
			FVector3d A, B;
			Mesh->GetEdgeV(eid, A, B);
			BoundaryLines->AddLine((FVector)A, (FVector)B, BoundaryColor, BoundaryThickness);
		}
	}

	ULineSetComponent* SeamLines = PreviewGeom->AddLineSet(TEXT("Seams"));
	UpdatePreviewGeometry();
}


void USeamSculptTool::UpdatePreviewGeometry()
{
	FColor SeamColor = FColor(25, 150, 25);
	float SeamThickness = 4.0;

	const FDynamicMesh3* BaseMesh = InputMesh.Get();
	const FDynamicMesh3* CurMesh = PreviewMesh->GetMesh();

	int32 UVLayer = 0;
	const FDynamicMeshUVOverlay* UVOverlay = CurMesh->Attributes()->GetUVLayer(UVLayer);

	ULineSetComponent* SeamLines = PreviewGeom->FindLineSet(TEXT("Seams"));
	SeamLines->Clear();
	for (int32 eid : CurMesh->EdgeIndicesItr())
	{
		if (UVOverlay->IsSeamEdge(eid))
		{
			FIndex2i EdgeV = CurMesh->GetEdgeV(eid);
			FVector3d V0 = BaseMesh->GetVertex(EdgeV.A) + NormalOffset * (FVector3d)BaseMesh->GetVertexNormal(EdgeV.A);
			FVector3d V1 = BaseMesh->GetVertex(EdgeV.B) + NormalOffset * (FVector3d)BaseMesh->GetVertexNormal(EdgeV.B);
			SeamLines->AddLine((FVector)V0, (FVector)V1, SeamColor, SeamThickness);
		}
	}
}




void USeamSculptTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FToolDataVisualizer Draw;
	Draw.BeginFrame(RenderAPI);
	Draw.PushTransform(PreviewMesh->GetTransform());

	if (CurrentSnapVertex >= 0)
	{
		Draw.DrawPoint(CurrentSnapPositionLocal, FLinearColor(0, 1, 0), 8.0, false);
	}

	if (CaptureState == EActiveCaptureState::DrawNewPath)
	{
		Draw.DrawPoint(DrawPathStartPositionLocal, FLinearColor(0, 1, 1), 10.0, false);

		const FDynamicMesh3* Mesh = InputMesh.Get();
		int32 PathLen = CurDrawPath.Num();
		for (int32 k = 0; k < PathLen - 1; ++k)
		{
			FVector3d V0 = Mesh->GetVertex(CurDrawPath[k]) + NormalOffset*(FVector3d)Mesh->GetVertexNormal(CurDrawPath[k]);
			FVector3d V1 = Mesh->GetVertex(CurDrawPath[k+1]) + NormalOffset*(FVector3d)Mesh->GetVertexNormal(CurDrawPath[k+1]);
			Draw.DrawLine( (FVector)V0, (FVector)V1, FLinearColor(0,1,0), 2.0, false);
		}
	}

	Draw.EndFrame();
}





void USeamSculptTool::CreateSeamAlongPath()
{
	if (CurDrawPath.Num() > 1)
	{
		TUniquePtr<FMeshChange> MeshEditChange = PreviewMesh->TrackedEditMesh([&](FDynamicMesh3& Mesh, FDynamicMeshChangeTracker& Tracker)
		{
			Tracker.SaveVertexOneRingTriangles(CurDrawPath, true);

			TSet<int32> PathEids;
			for (int32 i = 0; i < CurDrawPath.Num() - 1; ++i)
			{
				int32 Eid = Mesh.FindEdge(CurDrawPath[i], CurDrawPath[i + 1]);
				if (ensure(Eid != IndexConstants::InvalidID))
				{
					PathEids.Add(Eid);
				}
			}
			FDynamicMeshUVEditor UVEditor(&Mesh, 0, false);
			UVEditor.CreateSeamsAtEdges(PathEids);
		});

		GetToolManager()->EmitObjectChange(PreviewMesh, MoveTemp(MeshEditChange), LOCTEXT("CreateSeamChange", "Add UV Seam"));
		CurDrawPath.Reset();
	}

	bPreviewGeometryNeedsUpdate = true;
}



#undef LOCTEXT_NAMESPACE
