// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelfUnionMeshesTool.h"
#include "CompositionOps/SelfUnionMeshesOp.h"
#include "InteractiveToolManager.h"
#include "ToolSetupUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "ModelingToolTargetUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshTransforms.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "TargetInterfaces/MeshDescriptionProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelfUnionMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USelfUnionMeshesTool"


void USelfUnionMeshesTool::SetupProperties()
{
	Super::SetupProperties();
	Properties = NewObject<USelfUnionMeshesToolProperties>(this);
	Properties->RestoreProperties(this);
	AddToolPropertySource(Properties);

	SetToolDisplayName(LOCTEXT("ToolName", "Merge"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Compute a Self-Union of the input meshes, to resolve self-intersections. Use the transform gizmos to tweak the positions of the input objects (can help to resolve errors/failures)"),
		EToolMessageLevel::UserNotification);
}


void USelfUnionMeshesTool::SaveProperties()
{
	Super::SaveProperties();
	Properties->SaveProperties(this);
}


void USelfUnionMeshesTool::TransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	ConvertInputsAndSetPreviewMaterials(false); // have to redo the conversion because the transforms are all baked there
	Preview->InvalidateResult();
}


void USelfUnionMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	FComponentMaterialSet AllMaterialSet;
	TMap<UMaterialInterface*, int> KnownMaterials;
	TArray<TArray<int>> MaterialRemap; MaterialRemap.SetNum(Targets.Num());

	if (!Properties->bOnlyUseFirstMeshMaterials)
	{
		for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			const FComponentMaterialSet ComponentMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[ComponentIdx]);
			for (UMaterialInterface* Mat : ComponentMaterialSet.Materials)
			{
				int* FoundMatIdx = KnownMaterials.Find(Mat);
				int MatIdx;
				if (FoundMatIdx)
				{
					MatIdx = *FoundMatIdx;
				}
				else
				{
					MatIdx = AllMaterialSet.Materials.Add(Mat);
					KnownMaterials.Add(Mat, MatIdx);
				}
				MaterialRemap[ComponentIdx].Add(MatIdx);
			}
		}
	}
	else
	{
		AllMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
		for (int MatIdx = 0; MatIdx < AllMaterialSet.Materials.Num(); MatIdx++)
		{
			MaterialRemap[0].Add(MatIdx);
		}
		for (int ComponentIdx = 1; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			MaterialRemap[ComponentIdx].Init(0, Cast<IMaterialProvider>(Targets[ComponentIdx])->GetNumMaterials());
		}
	}

	CombinedSourceMeshes = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	CombinedSourceMeshes->EnableAttributes();
	CombinedSourceMeshes->EnableTriangleGroups(0);
	CombinedSourceMeshes->Attributes()->EnableMaterialID();
	CombinedSourceMeshes->Attributes()->EnablePrimaryColors();
	FDynamicMeshEditor AppendEditor(CombinedSourceMeshes.Get());

	CombinedCenter = FVector3d(0, 0, 0);
	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		CombinedCenter += TransformProxies[ComponentIdx]->GetTransform().GetTranslation();
	}
	CombinedCenter /= double(Targets.Num());
	

	bool bNeedColorAttr = false;
	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FDynamicMesh3 ComponentMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx]);
		bNeedColorAttr = bNeedColorAttr || (ComponentMesh.Attributes()->HasPrimaryColors());
		// ensure materials and attributes are always enabled
		ComponentMesh.EnableAttributes();
		ComponentMesh.Attributes()->EnableMaterialID();
		FDynamicMeshMaterialAttribute* MaterialIDs = ComponentMesh.Attributes()->GetMaterialID();
		int32 NumComponentMaterials = MaterialRemap[ComponentIdx].Num();
		// If component had valid materials, remap them. (If it had no materials, no IDs are valid, so just leave them as they were.)
		if (NumComponentMaterials > 0)
		{
			for (int TID : ComponentMesh.TriangleIndicesItr())
			{
				int32 MID = MaterialIDs->GetValue(TID);
				MaterialIDs->SetValue(TID, MaterialRemap[ComponentIdx][MID >= 0 && MID < NumComponentMaterials ? MID : 0]);
			}
		}
		FTransformSRT3d WorldTransform = TransformProxies[ComponentIdx]->GetTransform();
		if (WorldTransform.GetDeterminant() < 0)
		{
			ComponentMesh.ReverseOrientation(false);
		}

		// ensure all UV layers are transferred
		const int NumUVLayers = FMath::Max(CombinedSourceMeshes->Attributes()->NumUVLayers(), ComponentMesh.Attributes()->NumUVLayers());
		CombinedSourceMeshes->Attributes()->SetNumUVLayers(NumUVLayers);
		
		FMeshIndexMappings IndexMaps;
		AppendEditor.AppendMesh(&ComponentMesh, IndexMaps,
			[WorldTransform, this](int VID, const FVector3d& Pos)
			{
				return WorldTransform.TransformPosition(Pos) - CombinedCenter;
			},
			[WorldTransform](int VID, const FVector3d& Normal)
			{
				return WorldTransform.TransformNormal(Normal);
			}
			);
	}
	if (!bNeedColorAttr)
	{
		CombinedSourceMeshes->Attributes()->DisablePrimaryColors();
	}

	Preview->ConfigureMaterials(AllMaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	if (bSetPreviewMesh)
	{
		Preview->PreviewMesh->UpdatePreview(CombinedSourceMeshes.Get());
		Preview->PreviewMesh->SetTransform(FTransform(CombinedCenter));
	}
}


void USelfUnionMeshesTool::SetPreviewCallbacks()
{
	DrawnLineSet = NewObject<ULineSetComponent>(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetupAttachment(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	DrawnLineSet->RegisterComponent();

	Preview->OnOpCompleted.AddLambda(
		[this](const FDynamicMeshOperator* Op)
		{
			const FSelfUnionMeshesOp* UnionOp = (const FSelfUnionMeshesOp*)(Op);
			CreatedBoundaryEdges = UnionOp->GetCreatedBoundaryEdges();
		}
	);
	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute*)
		{
			GetToolManager()->PostInvalidation();
			UpdateVisualization();
		}
	);
}


void USelfUnionMeshesTool::UpdateVisualization()
{
	FColor BoundaryEdgeColor(240, 15, 15);
	float BoundaryEdgeThickness = 2.0;
	float BoundaryEdgeDepthBias = 2.0f;

	const FDynamicMesh3* TargetMesh = Preview->PreviewMesh->GetPreviewDynamicMesh();
	FVector3d A, B;

	DrawnLineSet->Clear();
	DrawnLineSet->SetWorldTransform(Preview->PreviewMesh->GetTransform());
	if (Properties->bShowNewBoundaryEdges)
	{
		for (int EID : CreatedBoundaryEdges)
		{
			TargetMesh->GetEdgeV(EID, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}
}


TUniquePtr<FDynamicMeshOperator> USelfUnionMeshesTool::MakeNewOperator()
{
	TUniquePtr<FSelfUnionMeshesOp> Op = MakeUnique<FSelfUnionMeshesOp>();
	
	Op->bAttemptFixHoles = Properties->bTryFixHoles;
	Op->bTryCollapseExtraEdges = Properties->bTryCollapseEdges;
	Op->WindingNumberThreshold = Properties->WindingThreshold;
	Op->bTrimFlaps = Properties->bTrimFlaps;

	Op->SetResultTransform(FTransformSRT3d(CombinedCenter));
	Op->CombinedMesh = CombinedSourceMeshes;

	return Op;
}


void USelfUnionMeshesTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(USelfUnionMeshesToolProperties, bOnlyUseFirstMeshMaterials)))
	{
		if (!AreAllTargetsValid())
		{
			GetToolManager()->DisplayMessage(LOCTEXT("InvalidTargets", "Target meshes are no longer valid"), EToolMessageLevel::UserWarning);
			return;
		}
		ConvertInputsAndSetPreviewMaterials(false);
		Preview->InvalidateResult();
	}
	else if (PropertySet == HandleSourcesProperties)
	{
		// nothing
	}
	else if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(USelfUnionMeshesToolProperties, bShowNewBoundaryEdges)))
	{
		GetToolManager()->PostInvalidation();
		UpdateVisualization();
	}
	else
	{
		Super::OnPropertyModified(PropertySet, Property);
	}
}


FString USelfUnionMeshesTool::GetCreatedAssetName() const
{
	return TEXT("Merge");
}


FText USelfUnionMeshesTool::GetActionName() const
{
	return LOCTEXT("SelfUnionMeshes", "Merge Meshes");
}






#undef LOCTEXT_NAMESPACE

