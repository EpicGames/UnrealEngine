// Copyright Epic Games, Inc. All Rights Reserved.

#include "Spline/BaseMeshFromSplinesTool.h"

#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Properties/MeshMaterialProperties.h"

#include "Engine/World.h"

#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseMeshFromSplinesTool)

#define LOCTEXT_NAMESPACE "UBaseMeshFromSplinesTool"

using namespace UE::Geometry;



void UBaseMeshFromSplinesTool::Setup()
{
	UInteractiveTool::Setup();

	// initialize our properties

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	AddToolPropertySource(MaterialProperties);
	MaterialProperties->RestoreProperties(this);

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), this);
	Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
	Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);

	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute* UpdatedPreview)
		{
			UpdateAcceptWarnings(UpdatedPreview->HaveEmptyResult() ? EAcceptWarning::EmptyForbidden : EAcceptWarning::NoWarning);
		}
	);

	PollSplineUpdates();
}

void UBaseMeshFromSplinesTool::PollSplineUpdates()
{
	if (bLostInputSpline)
	{
		return;
	}

	bool bSplinesUpdated = false;
	int32 SplineIdx = 0;
	EnumerateSplines([this, &bSplinesUpdated, &SplineIdx](USplineComponent* SplineComponent)
	{
		int32 Version = SplineComponent->GetVersion();
		FTransform Transform = SplineComponent->GetComponentTransform();
		if (SplineIdx >= LastSplineVersions.Num())
		{
			bSplinesUpdated = true;
			LastSplineVersions.Add(Version);
			LastSplineTransforms.Add(Transform);
		}
		else if (LastSplineVersions[SplineIdx] != Version || !LastSplineTransforms[SplineIdx].Equals(Transform))
		{
			bSplinesUpdated = true;
		}
		LastSplineVersions[SplineIdx] = Version;
		LastSplineTransforms[SplineIdx] = Transform;
		++SplineIdx;
	});
	if (LastSplineVersions.Num() != SplineIdx)
	{
		if (SplineIdx < LastSplineVersions.Num())
		{
			bLostInputSpline = true;
			GetToolManager()->DisplayMessage(
				LOCTEXT("LostSpline", "Tool lost reference to an input spline; cannot respond to further spline changes."),
				EToolMessageLevel::UserWarning);
			return;
		}
		LastSplineVersions.SetNum(SplineIdx);
		LastSplineTransforms.SetNum(SplineIdx);
		bSplinesUpdated = true;
	}
	if (bSplinesUpdated)
	{
		OnSplineUpdate();

		Preview->InvalidateResult();
	}
}

USplineComponent* UBaseMeshFromSplinesTool::GetFirstSpline() const
{
	USplineComponent* ToRet = nullptr;
	for (int32 ActorIdx = 0; !ToRet && ActorIdx < ActorsWithSplines.Num(); ++ActorIdx)
	{
		if (AActor* Actor = ActorsWithSplines[ActorIdx].Get())
		{
			Actor->ForEachComponent<USplineComponent>(false, [&ToRet](USplineComponent* SplineComponent)
			{
				if (!ToRet)
				{
					ToRet = SplineComponent;
				}
			});
		}
	}
	return ToRet;
}

USplineComponent* UBaseMeshFromSplinesTool::GetLastSpline() const
{
	USplineComponent* ToRet = nullptr;
	for (int32 ActorIdx = ActorsWithSplines.Num() - 1; !ToRet && ActorIdx >= 0; --ActorIdx)
	{
		if (AActor* Actor = ActorsWithSplines[ActorIdx].Get())
		{
			Actor->ForEachComponent<USplineComponent>(false, [&ToRet](USplineComponent* SplineComponent)
			{
				ToRet = SplineComponent;
			});
		}
	}
	return ToRet;
}

void UBaseMeshFromSplinesTool::OnTick(float DeltaTime)
{
	PollSplineUpdates();

	if (Preview)
	{
		Preview->Tick(DeltaTime);
	}
}

void UBaseMeshFromSplinesTool::SetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* UBaseMeshFromSplinesTool::GetTargetWorld()
{
	return TargetWorld.Get();
}

FTransform3d UBaseMeshFromSplinesTool::HandleOperatorTransform(const FDynamicMeshOpResult& OpResult)
{
	FTransform3d NewTransform;
	if (ActorsWithSplines.Num() == 1 && ActorsWithSplines[0].IsValid()) // in the single-actor case, shove the result back into the original actor transform space
	{
		FTransform3d ActorToWorld = (FTransform3d)ActorsWithSplines[0]->GetTransform();
		MeshTransforms::ApplyTransform(*OpResult.Mesh, OpResult.Transform, true);
		MeshTransforms::ApplyTransformInverse(*OpResult.Mesh, ActorToWorld, true);
		NewTransform = ActorToWorld;
	}
	else // in the multi-selection case, center the pivot for the combined result
	{
		FVector3d Center = OpResult.Mesh->GetBounds().Center();
		double Rescale = OpResult.Transform.GetScale().X;
		FTransform3d LocalTransform(-Center * Rescale);
		LocalTransform.SetScale3D(FVector3d(Rescale, Rescale, Rescale));
		MeshTransforms::ApplyTransform(*OpResult.Mesh, LocalTransform, true);
		NewTransform = OpResult.Transform;
		NewTransform.SetScale3D(FVector3d::One());
		NewTransform.SetTranslation(NewTransform.GetTranslation() + NewTransform.TransformVector(Center * Rescale));
	}
	return NewTransform;
}

void UBaseMeshFromSplinesTool::GenerateAsset(const FDynamicMeshOpResult& OpResult)
{
	if (OpResult.Mesh.Get() == nullptr) return;

	FTransform3d NewTransform = HandleOperatorTransform(OpResult);

	FString BaseName = GeneratedAssetBaseName();

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = GetTargetWorld();
	NewMeshObjectParams.Transform = (FTransform)NewTransform;
	NewMeshObjectParams.BaseName = BaseName;
	NewMeshObjectParams.Materials.Add(MaterialProperties->Material.Get());
	NewMeshObjectParams.SetMesh(OpResult.Mesh.Get());
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}
}

void UBaseMeshFromSplinesTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (PropertySet == OutputTypeProperties))
	{
		return;
	}

	if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNewMeshMaterialProperties, Material))
	{
		Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	}

	Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);

	Preview->InvalidateResult();
}

void UBaseMeshFromSplinesTool::Shutdown(EToolShutdownType ShutdownType)
{
	OutputTypeProperties->SaveProperties(this);
	MaterialProperties->SaveProperties(this);

	FDynamicMeshOpResult Result = Preview->Shutdown();

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SweepSplineAction", "Spline Triangulation"));

		// Generate the result asset
		GenerateAsset(Result);

		GetToolManager()->EndUndoTransaction();
	}

	TargetWorld = nullptr;
	Preview = nullptr;
	MaterialProperties = nullptr;
	OutputTypeProperties = nullptr;

	Super::Shutdown(ShutdownType);
}

bool UBaseMeshFromSplinesTool::CanAccept() const
{
	return Preview->HaveValidNonEmptyResult();
}

TUniquePtr<FDynamicMeshOperator> UBaseMeshFromSplinesTool::MakeNewOperator()
{
	checkNoEntry();
	return TUniquePtr<FDynamicMeshOperator>();
}

/// Tool builder

bool UBaseMeshFromSplinesToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int32 NumSplines = ToolBuilderUtil::CountComponents(SceneState, [&](UActorComponent* Object) -> bool
		{
			return Object && Object->IsA<USplineComponent>();
		});
	FIndex2i SupportedRange = GetSupportedSplineCountRange();
	return (NumSplines >= SupportedRange.A && (SupportedRange.B == -1 || NumSplines <= SupportedRange.B));
}

void UBaseMeshFromSplinesToolBuilder::InitializeNewTool(UBaseMeshFromSplinesTool* NewTool, const FToolBuilderState& SceneState) const
{
	TArray<UActorComponent*> Components = ToolBuilderUtil::FindAllComponents(SceneState, [&](UActorComponent* Object)
		{
			return Object && Object->IsA<USplineComponent>();
		});
	TArray<TWeakObjectPtr<AActor>> ActorsWithSplines;
	TSet<AActor*> FoundActors;
	for (UActorComponent* Component : Components)
	{
		AActor* ActorWithSpline = Component->GetOwner();
		if (!FoundActors.Contains(ActorWithSpline))
		{
			FoundActors.Add(ActorWithSpline);
			ActorsWithSplines.Add(ActorWithSpline);
		}
	}
	NewTool->SetSplineActors(MoveTemp(ActorsWithSplines));
	NewTool->SetWorld(SceneState.World);
}



#undef LOCTEXT_NAMESPACE

