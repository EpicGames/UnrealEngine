// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplitMeshesTool.h"
#include "ComponentSourceInterfaces.h"
#include "Drawing/PreviewGeometryActor.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "VertexConnectedComponents.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicSubmesh3.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Util/ColorConstants.h"

#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplitMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USplitMeshesTool"

/*
 * ToolBuilder
 */
const FToolTargetTypeRequirements& USplitMeshesToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

UMultiTargetWithSelectionTool* USplitMeshesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USplitMeshesTool>(SceneState.ToolManager);
}



/*
 * Tool
 */


void USplitMeshesTool::Setup()
{
	UInteractiveTool::Setup();

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->InitializeDefaultWithAuto();
	OutputTypeProperties->OutputType = UCreateMeshObjectTypeProperties::AutoIdentifier;
	OutputTypeProperties->RestoreProperties(this, TEXT("OutputTypeFromInputTool"));
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	BasicProperties = NewObject<USplitMeshesToolProperties>(this);
	BasicProperties->RestoreProperties(this);
	BasicProperties->WatchProperty(BasicProperties->SplitMethod, [this](ESplitMeshesMethod) { UpdateSplitMeshes(); });
	BasicProperties->WatchProperty(BasicProperties->ConnectVerticesThreshold, [this](double) { UpdateSplitMeshes(); });
	BasicProperties->WatchProperty(BasicProperties->bShowPreview, [this](bool bShowPreview) { UpdatePreviewVisibility(bShowPreview); });
	AddToolPropertySource(BasicProperties);

	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	
	SourceMeshes.SetNum(Targets.Num());
	bool bHasSelection = false;
	for (int32 k = 0; k < Targets.Num(); ++k)
	{
		SourceMeshes[k].Mesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[k], GetMeshParams);
		SourceMeshes[k].Materials = UE::ToolTarget::GetMaterialSet(Targets[k]).Materials;
		bHasSelection = bHasSelection || HasGeometrySelection(k);
	}
	BasicProperties->bIsInSelectionMode = bHasSelection;

	PerTargetPreviews.Reserve(Targets.Num());
	for (int32 TargetIdx = 0; TargetIdx < Targets.Num(); ++TargetIdx)
	{
		UPreviewGeometry* PreviewGeom = PerTargetPreviews.Add_GetRef(NewObject<UPreviewGeometry>(this));
		PreviewGeom->CreateInWorld(UE::ToolTarget::GetTargetActor(Targets[TargetIdx])->GetWorld(), UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]));
	}
	PreviewMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), false);

	UpdateSplitMeshes();

	SetToolDisplayName(LOCTEXT("ToolName", "Split"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Split Meshes into parts"),
		EToolMessageLevel::UserNotification);
}

void USplitMeshesTool::UpdatePreviewVisibility(bool bShowPreview)
{
	checkSlow(Targets.Num() == SplitMeshes.Num());
	for (int32 PreviewIdx = 0; PreviewIdx < PerTargetPreviews.Num(); ++PreviewIdx)
	{
		PerTargetPreviews[PreviewIdx]->SetAllVisible(bShowPreview);
		UE::ToolTarget::SetSourceObjectVisible(Targets[PreviewIdx], !bShowPreview || SplitMeshes[PreviewIdx].bNoComponents);
	}
}

bool USplitMeshesTool::CanAccept() const
{
	return Super::CanAccept();
}

void USplitMeshesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	for (UPreviewGeometry* PreviewGeom : PerTargetPreviews)
	{
		PreviewGeom->Disconnect();
	}

	// make sure source objects are visible
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	OutputTypeProperties->SaveProperties(this, TEXT("OutputTypeFromInputTool"));
	BasicProperties->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SplitMeshesToolTransactionName", "Split Meshes"));

		TArray<AActor*> NewSelectedActors;
		TSet<AActor*> DeleteActors;

		for (int32 ti = 0; ti < Targets.Num(); ++ti)
		{
			FComponentsInfo& SplitInfo = SplitMeshes[ti];
			if (SplitInfo.bNoComponents)
			{
				continue;
			}
			AActor* TargetActor = UE::ToolTarget::GetTargetActor(Targets[ti]);
			check(TargetActor != nullptr);
			DeleteActors.Add(TargetActor);

			FTransform3d SourceTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[ti]);
			FString AssetName = TargetActor->GetActorNameOrLabel();

			FCreateMeshObjectParams BaseMeshObjectParams;
			BaseMeshObjectParams.TargetWorld = GetTargetWorld();

			if (OutputTypeProperties->OutputType == UCreateMeshObjectTypeProperties::AutoIdentifier)
			{
				UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[ti], BaseMeshObjectParams);
			}
			else
			{
				OutputTypeProperties->ConfigureCreateMeshObjectParams(BaseMeshObjectParams);
			}

			int32 NumComponents = SplitInfo.Meshes.Num();
			for (int32 k = 0; k < NumComponents; ++k)
			{
				FCreateMeshObjectParams NewMeshObjectParams = BaseMeshObjectParams;
				NewMeshObjectParams.BaseName = FString::Printf(TEXT("%s_%d"), *AssetName, k);
				FTransform3d PartTransform = SourceTransform;
				PartTransform.SetTranslation(SourceTransform.GetTranslation() + SplitInfo.Origins[k]);
				NewMeshObjectParams.Transform = (FTransform)PartTransform;
				if (BasicProperties->bTransferMaterials)
				{
					NewMeshObjectParams.Materials = SplitInfo.Materials[k];
				}
				NewMeshObjectParams.SetMesh(MoveTemp(SplitInfo.Meshes[k]));

				FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
				if (Result.IsOK())
				{
					NewSelectedActors.Add(Result.NewActor);
				}
			}
		}

		for (AActor* DeleteActor : DeleteActors)
		{
			DeleteActor->Destroy();
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewSelectedActors);

		GetToolManager()->EndUndoTransaction();
	}
}




void USplitMeshesTool::UpdateSplitMeshes()
{
	SplitMeshes.Reset();
	SplitMeshes.SetNum(SourceMeshes.Num());
	NoSplitCount = 0;

	int32 VisColorIdx = 0;

	for (int32 si = 0; si < SourceMeshes.Num(); ++si)
	{
		FComponentsInfo& SplitInfo = SplitMeshes[si];
		const FDynamicMesh3* SourceMesh = &SourceMeshes[si].Mesh;
		const TArray<UMaterialInterface*> SourceMaterials = SourceMeshes[si].Materials;

		TArray<TArray<int32>> ComponentTriIndices;
		int32 NumComponents = 0;

		auto FillComponentTriIndicesFromTriIDs = [&ComponentTriIndices, &NumComponents, SourceMesh](TFunctionRef<int32(int32)> TIDtoID)
			{
				TMap<int32, int32> ComponentIDMap;
				for (int32 TID : SourceMesh->TriangleIndicesItr())
				{
					int32 CompID = TIDtoID(TID);
					int32* FoundIdx = ComponentIDMap.Find(CompID);
					int32 UseIdx = -1;
					if (FoundIdx)
					{
						UseIdx = *FoundIdx;
					}
					else
					{
						UseIdx = ComponentTriIndices.AddDefaulted();
						ComponentIDMap.Add(CompID, UseIdx);
					}
					ComponentTriIndices[UseIdx].Add(TID);
				}
				NumComponents = ComponentTriIndices.Num();
			};

		const bool bMeshHasGeometrySelection = HasGeometrySelection(si);
		if (bMeshHasGeometrySelection)
		{
			// when there is a geometry selection, ignore any computations the Split Tool would normally have done to
			// decide where to split the mesh; instead split into 2 meshes regardless: the selected geometry, and everything else
			NumComponents = 2;
		}
		else if (BasicProperties->SplitMethod == ESplitMeshesMethod::ByMeshTopology || BasicProperties->SplitMethod == ESplitMeshesMethod::ByVertexOverlap)
		{
			FVertexConnectedComponents Components(SourceMesh->MaxVertexID());
			Components.ConnectTriangles(*SourceMesh);
			if (BasicProperties->SplitMethod == ESplitMeshesMethod::ByVertexOverlap)
			{
				Components.ConnectCloseVertices(*SourceMesh, BasicProperties->ConnectVerticesThreshold, 2);
			}
			FillComponentTriIndicesFromTriIDs([&](int32 TID)->int32 { return Components.GetComponent(SourceMesh->GetTriangle(TID).A); });
		}
		else if (BasicProperties->SplitMethod == ESplitMeshesMethod::ByPolyGroup)
		{
			FillComponentTriIndicesFromTriIDs([&](int32 TID)->int32 { return SourceMesh->GetTriangleGroup(TID); });
		}
		else if (BasicProperties->SplitMethod == ESplitMeshesMethod::ByMaterialID)
		{
			if (SourceMesh->HasAttributes())
			{
				if (const FDynamicMeshMaterialAttribute* MaterialID = SourceMesh->Attributes()->GetMaterialID())
				{
					FillComponentTriIndicesFromTriIDs([&](int32 TID)->int32 { return MaterialID->GetValue(TID); });
				}
			}
		}
		
		if (NumComponents < 2)
		{
			PerTargetPreviews[si]->RemoveAllTriangleSets();
			SplitInfo.bNoComponents = true;
			NoSplitCount++;
			continue;
		}
		SplitInfo.bNoComponents = false;

		SplitInfo.Meshes.SetNum(NumComponents);
		SplitInfo.Materials.SetNum(NumComponents);
		SplitInfo.Origins.SetNum(NumComponents);
		for (int32 k = 0; k < NumComponents; ++k)
		{
			FDynamicSubmesh3 SubmeshCalc;
			if (bMeshHasGeometrySelection)
			{
				// retrieve the triangles in the current selection
				// when using Edge or Vertex selection mode, any triangle touching the selected edges/vertices will be included
				TSet<int> SelectionTriangles;
				const FGeometrySelection& InputSelection = GetGeometrySelection(si);
				
				UE::Geometry::EnumerateSelectionTriangles(InputSelection, *SourceMesh,
			[&](int32 TriangleID) { SelectionTriangles.Add(TriangleID); });

				TArray<int> SelectionTrianglesArray = SelectionTriangles.Array();
				
				if (k == 0) // component made of the selected triangles
				{
					SubmeshCalc = FDynamicSubmesh3(SourceMesh, SelectionTrianglesArray);
				}
				else if (k == 1) // component made of the rest of the mesh (unselected triangles)
				{
					TArray<int> NonSelectedTriangles;
					for (int TID = 0; TID < SourceMesh->MaxTriangleID(); TID++)
					{
						if (SourceMesh->IsTriangle(TID) && !SelectionTriangles.Contains(TID))
						{
							NonSelectedTriangles.Add(TID);
						}
					}
					SubmeshCalc = FDynamicSubmesh3(SourceMesh, NonSelectedTriangles);
				}
			}
			else
			{
				// if statement should always be true- components should always have been calculated & populated when there's no geometry selection
				if (ensure(!ComponentTriIndices.IsEmpty()))
				{
					SubmeshCalc = FDynamicSubmesh3(SourceMesh, ComponentTriIndices[k]);
				}
			}

			FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
			TArray<UMaterialInterface*> NewMaterials;

			// remap materials
			FDynamicMeshMaterialAttribute* MaterialIDs = Submesh.HasAttributes() ? Submesh.Attributes()->GetMaterialID() : nullptr;
			if (MaterialIDs)
			{
				TArray<int32> UniqueIDs;
				for (int32 tid : Submesh.TriangleIndicesItr())
				{
					int32 MaterialID = MaterialIDs->GetValue(tid);
					int32 Index = UniqueIDs.IndexOfByKey(MaterialID);
					if (Index == INDEX_NONE)
					{
						int32 NewMaterialID = UniqueIDs.Num();
						UniqueIDs.Add(MaterialID);
						NewMaterials.Add(SourceMaterials.IsValidIndex(MaterialID) ? SourceMaterials[MaterialID] : nullptr);
						MaterialIDs->SetValue(tid, NewMaterialID);
					}
					else
					{
						MaterialIDs->SetValue(tid, Index);
					}
				}
			}
			
			// TODO: Consider whether to expose bCenterPivots as an option to the user
			constexpr bool bCenterPivots = false;
			FVector3d Origin = FVector3d::ZeroVector;
			if (bCenterPivots)
			{
				// reposition mesh
				FAxisAlignedBox3d Bounds = Submesh.GetBounds();
				Origin = Bounds.Center();
				MeshTransforms::Translate(Submesh, -Origin);
			}

			SplitInfo.Meshes[k] = MoveTemp(Submesh);
			SplitInfo.Materials[k] = MoveTemp(NewMaterials);
			SplitInfo.Origins[k] = Origin;
		}

		PerTargetPreviews[si]->CreateOrUpdateTriangleSet(TEXT("Components"), 1, [&](int32, TArray<FRenderableTriangle>& Triangles)
			{	
				for (const UE::Geometry::FDynamicMesh3& Mesh : SplitInfo.Meshes)
				{
					++VisColorIdx;
					FColor MeshColor = LinearColors::SelectFColor(VisColorIdx);

					for (int32 TID : Mesh.TriangleIndicesItr())
					{
						FVector3d Normal = Mesh.GetTriNormal(TID);
						FIndex3i Tri = Mesh.GetTriangle(TID);
						FRenderableTriangleVertex A(Mesh.GetVertex(Tri.A), FVector2D(0, 0), Normal, MeshColor);
						FRenderableTriangleVertex B(Mesh.GetVertex(Tri.B), FVector2D(1, 0), Normal, MeshColor);
						FRenderableTriangleVertex C(Mesh.GetVertex(Tri.C), FVector2D(1, 1), Normal, MeshColor);
						Triangles.Add(FRenderableTriangle(PreviewMaterial, A, B, C));
					}
				}
			}, SourceMeshes[si].Mesh.TriangleCount());
	}

	if (NoSplitCount > 0)
	{
		GetToolManager()->DisplayMessage(
			FText::Format(LOCTEXT("NoComponentsMessage", "{0} of {1} Input Meshes cannot be Split."), NoSplitCount, SourceMeshes.Num()), EToolMessageLevel::UserWarning);
	}
	else
	{
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
	}

	UpdatePreviewVisibility(BasicProperties->bShowPreview);

}



#undef LOCTEXT_NAMESPACE

