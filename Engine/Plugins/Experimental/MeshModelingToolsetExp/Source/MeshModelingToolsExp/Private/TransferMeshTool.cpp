// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransferMeshTool.h"
#include "ComponentSourceInterfaces.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "Physics/ComponentCollisionUtil.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransferMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UTransferMeshTool"

/*
 * ToolBuilder
 */

bool UTransferMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 2;
}

UMultiSelectionMeshEditingTool* UTransferMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UTransferMeshTool>(SceneState.ToolManager);
}

const FToolTargetTypeRequirements& UTransferMeshToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

/*
 * Tool
 */

void UTransferMeshTool::Setup()
{
	UInteractiveTool::Setup();

	BasicProperties = NewObject<UTransferMeshToolProperties>(this);
	BasicProperties->RestoreProperties(this);
	AddToolPropertySource(BasicProperties);

	SetToolDisplayName(LOCTEXT("ToolName", "Transfer"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Copy Mesh from Source object to Target object"),
		EToolMessageLevel::UserNotification);

	// currently we can only query available LODs via the UStaticMesh, improvements TBD
#if WITH_EDITOR
	if (IStaticMeshBackedTarget* StaticMeshTarget = Cast<IStaticMeshBackedTarget>(Targets[0]))
	{
		if (UStaticMesh* StaticMesh = StaticMeshTarget->GetStaticMesh())
		{
			BasicProperties->bIsStaticMeshSource = true;
			int32 NumSourceModels = StaticMesh->GetNumSourceModels();
			for (int32 si = 0; si < NumSourceModels; ++si)
			{
				if (StaticMesh->IsMeshDescriptionValid(si))
				{
					BasicProperties->SourceLODNamesList.Add(FString::Printf(TEXT("LOD %d"), si));
					BasicProperties->SourceLODEnums.Add(static_cast<EMeshLODIdentifier>(si));
				}
			}

			// hires LOD slot
			if (StaticMesh->IsHiResMeshDescriptionValid())
			{
				BasicProperties->SourceLODNamesList.Add(TEXT("HiRes"));
				BasicProperties->SourceLODEnums.Add(EMeshLODIdentifier::HiResSource);
			}

			BasicProperties->SourceLOD = BasicProperties->SourceLODNamesList[0];
		}
	}	

	if (IStaticMeshBackedTarget* StaticMeshTarget = Cast<IStaticMeshBackedTarget>(Targets[1]))
	{
		if (UStaticMesh* StaticMesh = StaticMeshTarget->GetStaticMesh())
		{
			BasicProperties->bIsStaticMeshTarget = true;
			int32 NumSourceModels = StaticMesh->GetNumSourceModels();
			for (int32 si = 0; si < NumSourceModels; ++si)
			{
				BasicProperties->TargetLODNamesList.Add( FString::Printf(TEXT("LOD %d"), si) );
				BasicProperties->TargetLODEnums.Add(static_cast<EMeshLODIdentifier>(si));
			}

			// option to add one additional lod
			if (NumSourceModels <= 7)
			{
				BasicProperties->TargetLODNamesList.Add(FString::Printf(TEXT("LOD %d (New)"), NumSourceModels));
				BasicProperties->TargetLODEnums.Add(static_cast<EMeshLODIdentifier>(NumSourceModels));
			}

			// hires LOD slot
			BasicProperties->TargetLODNamesList.Add(TEXT("HiRes"));
			BasicProperties->TargetLODEnums.Add(EMeshLODIdentifier::HiResSource);

			BasicProperties->TargetLOD = BasicProperties->TargetLODNamesList[0];
		}
	}
#endif

}

bool UTransferMeshTool::CanAccept() const
{
	return Super::CanAccept();
}

void UTransferMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	BasicProperties->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("TransferMeshToolTransactionName", "Transfer Mesh"));

		FComponentMaterialSet Materials = UE::ToolTarget::GetMaterialSet(Targets[0]);
		const FComponentMaterialSet* TransferMaterials = (BasicProperties->bTransferMaterials) ? &Materials : nullptr;

		FMeshDescription SourceMesh;

		FGetMeshParameters GetMeshParams;
		GetMeshParams.bWantMeshTangents = true;

		if (IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Targets[0]))
		{
			FString SelectedLOD = BasicProperties->SourceLOD;
			if (SelectedLOD.StartsWith(TEXT("HiRes")))
			{
				GetMeshParams.bHaveRequestLOD = true;
				GetMeshParams.RequestLOD = EMeshLODIdentifier::HiResSource;
			}
			else
			{
				int32 FoundIndex = BasicProperties->SourceLODNamesList.IndexOfByKey(BasicProperties->SourceLOD);
				if (FoundIndex != INDEX_NONE)
				{
					GetMeshParams.bHaveRequestLOD = true;
					GetMeshParams.RequestLOD =  BasicProperties->SourceLODEnums[FoundIndex];
				}
			}
		}
		SourceMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[0], GetMeshParams);


		IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Targets[1]);
		if (BasicProperties->bIsStaticMeshTarget && MeshDescriptionCommitter)
		{
			FCommitMeshParameters CommitParams;
			FString SelectedLOD = BasicProperties->TargetLOD;
			if (SelectedLOD.StartsWith(TEXT("HiRes")))
			{
				CommitParams.bHaveTargetLOD = true;
				CommitParams.TargetLOD = EMeshLODIdentifier::HiResSource;
			}
			else
			{
				int32 FoundIndex = BasicProperties->TargetLODNamesList.IndexOfByKey(BasicProperties->TargetLOD);
				if (FoundIndex != INDEX_NONE)
				{
					CommitParams.bHaveTargetLOD = true;
					CommitParams.TargetLOD =  BasicProperties->TargetLODEnums[FoundIndex];
				}
			}

			if (TransferMaterials)
			{
				UE::ToolTarget::CommitMaterialSetUpdate(Targets[1], *TransferMaterials, true);
			}

			MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(SourceMesh), CommitParams);
		}
		else 
		{
			UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[1], &SourceMesh, TransferMaterials);
		}


		if (BasicProperties->bTransferCollision)
		{
			UPrimitiveComponent* SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[0]);
			if (UE::Geometry::ComponentTypeSupportsCollision(SourceComponent, UE::Geometry::EComponentCollisionSupportLevel::ReadOnly))
			{
				UPrimitiveComponent* TargetComponent = UE::ToolTarget::GetTargetComponent(Targets[1]);
				if (UE::Geometry::ComponentTypeSupportsCollision(TargetComponent))
				{
					FComponentCollisionSettings SourceCollisionSettings = UE::Geometry::GetCollisionSettings(SourceComponent);

					FSimpleShapeSet3d SourceShapeSet;
					if (UE::Geometry::GetCollisionShapes(SourceComponent, SourceShapeSet))
					{
						UE::Geometry::SetSimpleCollision(TargetComponent, &SourceShapeSet, SourceCollisionSettings);
					}
				}
			}
		}

		GetToolManager()->EndUndoTransaction();
	}
}





#undef LOCTEXT_NAMESPACE

