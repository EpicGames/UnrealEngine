// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformMeshesTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "ToolSceneQueriesUtil.h"
#include "ModelingToolTargetUtil.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include "Components/PrimitiveComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/World.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UTransformMeshesTool"


/*
 * ToolBuilder
 */


const FToolTargetTypeRequirements& UTransformMeshesToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(
		UPrimitiveComponentBackedTarget::StaticClass()
		);
	return TypeRequirements;
}

UMultiSelectionMeshEditingTool* UTransformMeshesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UTransformMeshesTool>(SceneState.ToolManager);
}




/*
 * Tool
 */

UTransformMeshesTool::UTransformMeshesTool()
{
}


void UTransformMeshesTool::Setup()
{
	UInteractiveTool::Setup();

	// Must be done before creating gizmos, so that we can bind the mechanic to them.
	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);

	UClickDragInputBehavior* ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
	ClickDragBehavior->Initialize(this);
	AddInputBehavior(ClickDragBehavior);

	TransformProps = NewObject<UTransformMeshesToolProperties>();
	AddToolPropertySource(TransformProps);
	TransformProps->RestoreProperties(this);
	TransformProps->WatchProperty(TransformProps->TransformMode, [this](ETransformMeshesTransformMode NewMode) { UpdateTransformMode(NewMode); });
	TransformProps->WatchProperty(TransformProps->bApplyToInstances, [this](bool bNewValue) { UpdateTransformMode(TransformProps->TransformMode); });
	TransformProps->WatchProperty(TransformProps->bSetPivotMode, [this](bool bNewValue) { UpdateSetPivotModes(bNewValue); });
	
	// determine if we have any ISMCs
	TransformProps->bHaveInstances = false;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		if (Cast<UInstancedStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx])) != nullptr)
		{
			TransformProps->bHaveInstances = true;
		}
	}

	UpdateTransformMode(TransformProps->TransformMode);

	SetToolDisplayName(LOCTEXT("ToolName", "Transform"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTransformMeshesTool", "Transform the selected objects. Middle-mouse-drag on gizmo to reposition it. Hold Ctrl while dragging to snap/align. [A] cycles through Transform modes. [S] toggles Set Pivot Mode. [D] Toggles Snap Drag Mode. [W] and [E] cycle through Snap Drag Source and Rotation types."),
		EToolMessageLevel::UserNotification);
}







void UTransformMeshesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	TransformProps->SaveProperties(this);

	DragAlignmentMechanic->Shutdown();

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
}




void UTransformMeshesTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
}



void UTransformMeshesTool::UpdateSetPivotModes(bool bEnableSetPivot)
{
	for (FTransformMeshesTarget& Target : ActiveGizmos)
	{
		Target.TransformProxy->bSetPivotMode = bEnableSetPivot;
	}
}



void UTransformMeshesTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	// Note: The CheckAndUpdateWatched() calls below were added to fix some invalid states
	// that we got into if we used hotkeys while snap dragging (where the handler might be called
	// before next tick). This should no longer be possible since we check that we're not snap dragging,
	// but we'll keep them here for safety.
	// The hotkeys are still usable while dragging the gizmos, but that doesn't cause
	// the kinds of UI issues that it does for snap dragging, and changing it for consistency
	// is tedious since we build and rebuild potentially multiple transform proxies.

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 1,
		TEXT("ToggleSetPivot"),
		LOCTEXT("TransformToggleSetPivot", "Toggle Set Pivot"),
		LOCTEXT("TransformToggleSetPivotTooltip", "Toggle Set Pivot on and off"),
		EModifierKey::None, EKeys::S,
		[this]() { 
			if (bCurrentlyDragging)
			{
				return;
			}
			TransformProps->bSetPivotMode = !TransformProps->bSetPivotMode; 
			TransformProps->CheckAndUpdateWatched();
			NotifyOfPropertyChangeByTool(TransformProps);
		});

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
		TEXT("ToggleSnapDrag"),
		LOCTEXT("TransformToggleSnapDrag", "Toggle SnapDrag"),
		LOCTEXT("TransformToggleSnapDragTooltip", "Toggle SnapDrag on and off"),
		EModifierKey::None, EKeys::D,
		[this]() { 
			if (bCurrentlyDragging)
			{
				return;
			}
			TransformProps->bEnableSnapDragging = !TransformProps->bEnableSnapDragging;
			TransformProps->CheckAndUpdateWatched();
			NotifyOfPropertyChangeByTool(TransformProps);
		});

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 3,
		TEXT("CycleTransformMode"),
		LOCTEXT("TransformCycleTransformMode", "Next Transform Mode"),
		LOCTEXT("TransformCycleTransformModeTooltip", "Cycle through available Transform Modes"),
		EModifierKey::None, EKeys::A,
		[this]() { 
			if (bCurrentlyDragging)
			{
				return;
			}
			TransformProps->TransformMode = (ETransformMeshesTransformMode)(((uint8)TransformProps->TransformMode+1) % (uint8)ETransformMeshesTransformMode::LastValue);
			TransformProps->CheckAndUpdateWatched();
			NotifyOfPropertyChangeByTool(TransformProps);
		});


	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 4,
		TEXT("CycleSourceMode"),
		LOCTEXT("TransformCycleSourceMode", "Next SnapDrag Source Mode"),
		LOCTEXT("TransformCycleSourceModeTooltip", "Cycle through available SnapDrag Source Modes"),
		EModifierKey::None, EKeys::W,
		[this]() { 
			if (bCurrentlyDragging)
			{
				return;
			}
			TransformProps->SnapDragSource = (ETransformMeshesSnapDragSource)(((uint8)TransformProps->SnapDragSource + 1) % (uint8)ETransformMeshesSnapDragSource::LastValue);
			TransformProps->CheckAndUpdateWatched();
			NotifyOfPropertyChangeByTool(TransformProps);
		});

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 5,
		TEXT("CycleRotationMode"),
		LOCTEXT("TransformCycleRotationMode", "Next SnapDrag Rotation Mode"),
		LOCTEXT("TransformCycleRotationModeTooltip", "Cycle through available SnapDrag Rotation Modes"),
		EModifierKey::None, EKeys::E,
		[this]() { 
			if (bCurrentlyDragging)
			{
				return;
			}
			TransformProps->RotationMode = (ETransformMeshesSnapDragRotationMode)(((uint8)TransformProps->RotationMode + 1) % (uint8)ETransformMeshesSnapDragRotationMode::LastValue); 
			TransformProps->CheckAndUpdateWatched();
			NotifyOfPropertyChangeByTool(TransformProps);
		});


}





void UTransformMeshesTool::UpdateTransformMode(ETransformMeshesTransformMode NewMode)
{
	ResetActiveGizmos();

	switch (NewMode)
	{
		default:
		case ETransformMeshesTransformMode::SharedGizmo:
			SetActiveGizmos_Single(false);
			break;

		case ETransformMeshesTransformMode::SharedGizmoLocal:
			SetActiveGizmos_Single(true);
			break;

		case ETransformMeshesTransformMode::PerObjectGizmo:
			SetActiveGizmos_PerObject();
			break;
	}

	CurTransformMode = NewMode;
}



namespace UE {
namespace Local {

static void AddInstancedComponentInstance(UInstancedStaticMeshComponent* ISMC, int32 Index, UTransformProxy* TransformProxy, bool bModifyOnTransform)
{
	TransformProxy->AddComponentCustom(ISMC,
		[ISMC, Index]() {
			FTransform Tmp;
			ISMC->GetInstanceTransform(Index, Tmp, true);
			return Tmp;
		},
		[ISMC, Index](const FTransform& NewTransform) {
			ISMC->UpdateInstanceTransform(Index, NewTransform, true, true, true);
		},
		Index, bModifyOnTransform
	);
}

}
}



void UTransformMeshesTool::SetActiveGizmos_Single(bool bLocalRotations)
{
	check(ActiveGizmos.Num() == 0);

	FTransformMeshesTarget Transformable;
	Transformable.TransformProxy = NewObject<UTransformProxy>(this);
	Transformable.TransformProxy->bSetPivotMode = TransformProps ? TransformProps->bSetPivotMode : false;
	Transformable.TransformProxy->bRotatePerObject = bLocalRotations;

	TArray<const UPrimitiveComponent*> ComponentsToIgnoreInAlignment;

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]);
		UInstancedStaticMeshComponent* InstancedComponent = Cast<UInstancedStaticMeshComponent>(Component);
		if (InstancedComponent != nullptr && TransformProps->bApplyToInstances)
		{
			int32 NumInstances = InstancedComponent->GetInstanceCount();
			for (int32 k = 0; k < NumInstances; ++k)
			{
				if (InstancedComponent->IsValidInstance(k) == false) return;
				UE::Local::AddInstancedComponentInstance(InstancedComponent, k, Transformable.TransformProxy, true);
			}
		}
		else
		{
			Transformable.TransformProxy->AddComponent(Component);
		}

		ComponentsToIgnoreInAlignment.Add(Component);
	}

	// leave out nonuniform scale if we have multiple objects in non-local mode
	bool bCanNonUniformScale = Targets.Num() == 1 || bLocalRotations;
	ETransformGizmoSubElements GizmoElements = (bCanNonUniformScale) ?
		ETransformGizmoSubElements::FullTranslateRotateScale : ETransformGizmoSubElements::TranslateRotateUniformScale;
	Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), GizmoElements, this);
	Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy);

	DragAlignmentMechanic->AddToGizmo(Transformable.TransformGizmo, &ComponentsToIgnoreInAlignment);

	ActiveGizmos.Add(Transformable);
}

void UTransformMeshesTool::SetActiveGizmos_PerObject()
{
	check(ActiveGizmos.Num() == 0);

	TArray<const UPrimitiveComponent*> ComponentsToIgnoreInAlignment;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UPrimitiveComponent* Component = UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]);
		UInstancedStaticMeshComponent* InstancedComponent = Cast<UInstancedStaticMeshComponent>(Component);
		if (InstancedComponent != nullptr && TransformProps->bApplyToInstances)
		{
			int32 NumInstances = InstancedComponent->GetInstanceCount();
			for (int32 k = 0; k < NumInstances; ++k)
			{
				if (InstancedComponent->IsValidInstance(k) == false) return;


				FTransformMeshesTarget Transformable;
				Transformable.TransformProxy = NewObject<UTransformProxy>(this);
				Transformable.TransformProxy->bSetPivotMode = TransformProps ? TransformProps->bSetPivotMode : false;
				UE::Local::AddInstancedComponentInstance(InstancedComponent, k, Transformable.TransformProxy, true);

				ETransformGizmoSubElements GizmoElements = ETransformGizmoSubElements::FullTranslateRotateScale;
				Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), GizmoElements, this);
				Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy);

				ComponentsToIgnoreInAlignment.Reset();
				ComponentsToIgnoreInAlignment.Add(Component);
				DragAlignmentMechanic->AddToGizmo(Transformable.TransformGizmo, &ComponentsToIgnoreInAlignment);

				ActiveGizmos.Add(Transformable);
			}
		}
		else
		{

			FTransformMeshesTarget Transformable;
			Transformable.TransformProxy = NewObject<UTransformProxy>(this);
			Transformable.TransformProxy->bSetPivotMode = TransformProps ? TransformProps->bSetPivotMode : false;
			Transformable.TransformProxy->AddComponent(Component);

			ETransformGizmoSubElements GizmoElements = ETransformGizmoSubElements::FullTranslateRotateScale;
			Transformable.TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), GizmoElements, this);
			Transformable.TransformGizmo->SetActiveTarget(Transformable.TransformProxy);

			ComponentsToIgnoreInAlignment.Reset();
			ComponentsToIgnoreInAlignment.Add(Component);
			DragAlignmentMechanic->AddToGizmo(Transformable.TransformGizmo, &ComponentsToIgnoreInAlignment);

			ActiveGizmos.Add(Transformable);

		}
	}
}

void UTransformMeshesTool::ResetActiveGizmos()
{
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	ActiveGizmos.Reset();
}



// does not make sense that CanBeginClickDragSequence() returns a RayHit? Needs to be an out-argument...
FInputRayHit UTransformMeshesTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (TransformProps->bEnableSnapDragging == false || ActiveGizmos.Num() == 0)
	{
		return FInputRayHit();
	}

	ActiveSnapDragIndex = -1;

	float MinHitDistance = TNumericLimits<float>::Max();
	FVector HitNormal;

	for ( int k = 0; k < Targets.Num(); ++k )
	{
		const IPrimitiveComponentBackedTarget* Target = Cast<IPrimitiveComponentBackedTarget>(Targets[k]);

		FHitResult WorldHit;
		if (Target->HitTestComponent(PressPos.WorldRay, WorldHit))
		{
			MinHitDistance = FMath::Min(MinHitDistance, WorldHit.Distance);
			HitNormal = WorldHit.Normal;
			ActiveSnapDragIndex = k;
		}
	}
	return (MinHitDistance < TNumericLimits<float>::Max()) ? FInputRayHit(MinHitDistance, HitNormal) : FInputRayHit();
}

void UTransformMeshesTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	FInputRayHit HitPos = CanBeginClickDragSequence(PressPos);
	check(HitPos.bHit);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("TransformToolTransformTxnName", "SnapDrag"));

	FTransformMeshesTarget& ActiveTarget =
		(TransformProps->TransformMode == ETransformMeshesTransformMode::PerObjectGizmo) ?
		ActiveGizmos[ActiveSnapDragIndex] : ActiveGizmos[0];
	USceneComponent* GizmoComponent = ActiveTarget.TransformGizmo->GetGizmoActor()->GetRootComponent();
	StartDragTransform = GizmoComponent->GetComponentToWorld();

	if (TransformProps->SnapDragSource == ETransformMeshesSnapDragSource::ClickPoint)
	{
		StartDragFrameWorld = FFrame3d((FVector3d)PressPos.WorldRay.PointAt(HitPos.HitDepth), (FVector3d)HitPos.HitNormal);
	}
	else
	{
		StartDragFrameWorld = FFrame3d(StartDragTransform);
	}

}


void UTransformMeshesTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	bCurrentlyDragging = true;
	bool bApplyToPivot = TransformProps->bSetPivotMode;

	TArray<const UPrimitiveComponent*> IgnoreComponents;
	if (bApplyToPivot == false)
	{
		int IgnoreIndex = (TransformProps->TransformMode == ETransformMeshesTransformMode::PerObjectGizmo) ?
			ActiveSnapDragIndex : -1;
		for (int k = 0; k < Targets.Num(); ++k)
		{
			if (IgnoreIndex == -1 || k == IgnoreIndex)
			{
				IgnoreComponents.Add(UE::ToolTarget::GetTargetComponent(Targets[k]));
			}
		}
	}


	bool bRotate = (TransformProps->RotationMode != ETransformMeshesSnapDragRotationMode::Ignore);
	float NormalSign = (TransformProps->RotationMode == ETransformMeshesSnapDragRotationMode::AlignFlipped) ? -1.0f : 1.0f;

	FHitResult Result;
	bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, DragPos.WorldRay, &IgnoreComponents);
	if (bWorldHit == false)
	{
		return;
	}

	if (bApplyToPivot)
	{
		FVector HitPos = Result.ImpactPoint;
		FVector TargetNormal = (-NormalSign) * Result.Normal;

		FQuaterniond AlignRotation = (bRotate) ?
			FQuaterniond(FVector3d::UnitZ(), (FVector3d)TargetNormal) : FQuaterniond::Identity();

		FTransform NewTransform = StartDragTransform;
		NewTransform.SetRotation((FQuat)AlignRotation);
		NewTransform.SetTranslation(HitPos);

		FTransformMeshesTarget& ActiveTarget =
			(TransformProps->TransformMode == ETransformMeshesTransformMode::PerObjectGizmo) ?
			ActiveGizmos[ActiveSnapDragIndex] : ActiveGizmos[0];
		ActiveTarget.TransformGizmo->SetNewGizmoTransform(NewTransform);
	}
	else
	{
		FVector HitPos = Result.ImpactPoint;
		FVector TargetNormal = NormalSign * Result.Normal;

		FFrame3d FromFrameWorld = StartDragFrameWorld;
		FFrame3d ToFrameWorld((FVector3d)HitPos, (FVector3d)TargetNormal);
		FFrame3d ObjectFrameWorld(StartDragTransform);

		FVector3d CenterShift = FromFrameWorld.Origin - ObjectFrameWorld.Origin;
		FQuaterniond AlignRotation(FromFrameWorld.Z(), ToFrameWorld.Z());
		if (bRotate == false)
		{
			AlignRotation = FQuaterniond::Identity();
		}
		FVector3d AlignTranslate = ToFrameWorld.Origin - FromFrameWorld.Origin;

		FTransform NewTransform = StartDragTransform;
		NewTransform.Accumulate( FTransform((FVector)CenterShift) );
		NewTransform.Accumulate( FTransform((FQuat)AlignRotation) );
		NewTransform.Accumulate( FTransform((FVector)AlignTranslate) );
		CenterShift = AlignRotation * CenterShift;
		NewTransform.Accumulate( FTransform((FVector)-CenterShift) );

		FTransformMeshesTarget& ActiveTarget =
			(TransformProps->TransformMode == ETransformMeshesTransformMode::PerObjectGizmo) ?
			ActiveGizmos[ActiveSnapDragIndex] : ActiveGizmos[0];
		ActiveTarget.TransformGizmo->SetNewGizmoTransform(NewTransform);
	}

}


void UTransformMeshesTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnTerminateDragSequence();
}

void UTransformMeshesTool::OnTerminateDragSequence()
{
	bCurrentlyDragging = false;
	FTransformMeshesTarget& ActiveTarget =
		(TransformProps->TransformMode == ETransformMeshesTransformMode::PerObjectGizmo) ?
		ActiveGizmos[ActiveSnapDragIndex] : ActiveGizmos[0];
	USceneComponent* GizmoComponent = ActiveTarget.TransformGizmo->GetGizmoActor()->GetRootComponent();
	FTransform EndDragtransform = GizmoComponent->GetComponentToWorld();

	TUniquePtr<FComponentWorldTransformChange> Change = MakeUnique<FComponentWorldTransformChange>(StartDragTransform, EndDragtransform);
	GetToolManager()->EmitObjectChange(GizmoComponent, MoveTemp(Change),
		LOCTEXT("TransformToolTransformTxnName", "SnapDrag"));

	GetToolManager()->EndUndoTransaction();

	ActiveSnapDragIndex = -1;
}



#undef LOCTEXT_NAMESPACE

