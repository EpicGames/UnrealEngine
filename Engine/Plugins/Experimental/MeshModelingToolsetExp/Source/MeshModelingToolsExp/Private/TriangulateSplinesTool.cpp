// Copyright Epic Games, Inc. All Rights Reserved.

#include "TriangulateSplinesTool.h"

#include "Properties/MeshMaterialProperties.h"

#include "Engine/World.h"

#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TriangulateSplinesTool)

#define LOCTEXT_NAMESPACE "UTriangulateSplinesTool"

using namespace UE::Geometry;



void UTriangulateSplinesTool::Setup()
{
	// initialize our properties

	TriangulateProperties = NewObject<UTriangulateSplinesToolProperties>(this);
	TriangulateProperties->RestoreProperties(this);
	AddToolPropertySource(TriangulateProperties);

	TriangulateProperties->WatchProperty(TriangulateProperties->ErrorTolerance, [this](double ErrorTolerance)
		{
			OnSplineUpdate();
		});

	SetToolDisplayName(LOCTEXT("TriangulateSplinesToolName", "Triangulate Splines"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("TriangulateSplinesToolToolDescription", "Triangulate the shapes of the selected splines."),
		EToolMessageLevel::UserNotification);

	Super::Setup();
}

void UTriangulateSplinesTool::Shutdown(EToolShutdownType ShutdownType)
{
	TriangulateProperties->SaveProperties(this);
	Super::Shutdown(ShutdownType);
}

TUniquePtr<FDynamicMeshOperator> UTriangulateSplinesTool::MakeNewOperator()
{
	TUniquePtr<FTriangulateCurvesOp> Op = MakeUnique<FTriangulateCurvesOp>();

	Op->Thickness = TriangulateProperties->Thickness;
	Op->bFlipResult = TriangulateProperties->bFlipResult;
	Op->CombineMethod = TriangulateProperties->CombineMethod;
	Op->FlattenMethod = TriangulateProperties->FlattenMethod;
	Op->CurveOffset = TriangulateProperties->CurveOffset;
	if (TriangulateProperties->CurveOffset == 0.0)
	{
		Op->OffsetClosedMethod = EOffsetClosedCurvesMethod::DoNotOffset;
	}
	else
	{
		Op->OffsetClosedMethod = TriangulateProperties->OffsetClosedCurves;
	}
	Op->OffsetOpenMethod = TriangulateProperties->OpenCurves;
	Op->OffsetJoinMethod = TriangulateProperties->JoinMethod;
	Op->OpenEndShape = TriangulateProperties->EndShapes;
	Op->MiterLimit = TriangulateProperties->MiterLimit;
	Op->UVScaleFactor = MaterialProperties->UVScale;
	Op->bWorldSpaceUVScale = MaterialProperties->bWorldSpaceUVScale;
	
	Op->bFlipResult = TriangulateProperties->bFlipResult;

	for (FPathCache& Path : SplinesCache)
	{
		Op->AddWorldCurve(Path.Vertices, Path.bClosed, Path.ComponentTransform);
	}

	return Op;
}

void UTriangulateSplinesTool::OnSplineUpdate()
{
	if (bLostInputSpline)
	{
		return;
	}

	SplinesCache.Reset();
	EnumerateSplines([this](USplineComponent* SplineComponent)
	{
		FPathCache& Path = SplinesCache.Emplace_GetRef();
		SplineComponent->ConvertSplineToPolyLine(ESplineCoordinateSpace::Type::World, 
			TriangulateProperties->ErrorTolerance * TriangulateProperties->ErrorTolerance, Path.Vertices);
		Path.ComponentTransform = SplineComponent->GetComponentTransform();
		Path.bClosed = SplineComponent->IsClosedLoop();
	});
}

FString UTriangulateSplinesTool::GeneratedAssetBaseName() const
{
	return TEXT("Triangulation");
}
FText UTriangulateSplinesTool::TransactionName() const
{
	return LOCTEXT("TriangulateSplinesAction", "Spline Triangulation");
}


/// Tool builder

UInteractiveTool* UTriangulateSplinesToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UTriangulateSplinesTool* NewTool = NewObject<UTriangulateSplinesTool>(SceneState.ToolManager);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}


#undef LOCTEXT_NAMESPACE

