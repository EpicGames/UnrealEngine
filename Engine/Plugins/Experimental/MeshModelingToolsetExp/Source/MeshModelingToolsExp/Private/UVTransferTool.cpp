// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVTransferTool.h"

#include "Drawing/MeshElementsVisualizer.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "ModelingOperators.h" // FDynamicMeshOperator
#include "ModelingToolTargetUtil.h"
#include "Parameterization/UVTransfer.h"
#include "Properties/MeshMaterialProperties.h"
#include "Selections/GeometrySelectionUtil.h" // EnumerateSelectionTriangles
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "ToolContextInterfaces.h" // FToolBuilderState
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVTransferTool)

#define LOCTEXT_NAMESPACE "UUVTransferTool"

namespace UVTransferToolLocals
{
	using namespace UE::Geometry;

	FString CacheIdentifier = TEXT("UVTransferTool");

	class FTransferUVsOp : public FDynamicMeshOperator
	{
	public:
		virtual ~FTransferUVsOp() {}

		// inputs
		TSharedPtr<const FDynamicMesh3> SourceMesh;
		TSharedPtr<const FDynamicMesh3> DestinationMesh;
		TOptional<TSet<int32>> SourceSelectionTids;
		TOptional<TSet<int32>> DestinationSelectionTids;
		int32 SourceUVLayerIndex = 0;
		int32 DestUVLayerIndex = 0;
		bool bTransferSeamsOnly = false;
		bool bClearExistingSeams = true;

		double VertexSearchDistance = KINDA_SMALL_NUMBER;
		double VertexSearchCellSize = VertexSearchDistance * 3.0;
		double PathSimilarityWeight = 200.0;

		void SetTransform(const FTransformSRT3d& Transform)
		{
			ResultTransform = Transform;
		}

		virtual void CalculateResult(FProgressCancel* Progress) override
		{
			if (Progress && Progress->Cancelled()) { return; }

			if (!DestinationMesh || !SourceMesh
				|| SourceUVLayerIndex < 0 || SourceUVLayerIndex > 7
				|| DestUVLayerIndex < 0 || DestUVLayerIndex > 7
				|| !SourceMesh->HasAttributes()
				|| SourceMesh->Attributes()->NumUVLayers() <= SourceUVLayerIndex
				|| !DestinationMesh->HasAttributes()
				|| DestinationMesh->Attributes()->NumUVLayers() <= DestUVLayerIndex)
			{
				ensure(false);
				return;
			}
			
			ResultMesh->Copy(*DestinationMesh);

			if (Progress && Progress->Cancelled()) { return; }

			UE::Geometry::FDynamicMeshUVTransfer Transferer(SourceMesh.Get(), ResultMesh.Get(), 
				SourceUVLayerIndex, DestUVLayerIndex);

			Transferer.VertexSearchDistance = VertexSearchDistance;
			Transferer.VertexSearchCellSize = VertexSearchCellSize;
			Transferer.PathSimilarityWeight = PathSimilarityWeight;
			Transferer.bClearExistingSeamsInDestination = bClearExistingSeams;

			// TODO SELECTIONS: Once we support multi-mesh selections, we would uncomment these
			//Transferer.SourceSelectionTids = SourceSelectionTids.GetPtrOrNull();
			//Transferer.DestinationSelectionTids = DestinationSelectionTids.GetPtrOrNull();

			if (bTransferSeamsOnly)
			{
				bTransferSucceeded = Transferer.TransferSeams(Progress);
			}
			else
			{
				bTransferSucceeded = Transferer.TransferSeamsAndUVs(Progress);
			}
			
		}

		bool bTransferSucceeded = false;
	private:

	};
}

void UUVTransferTool::OnShutdown(EToolShutdownType ShutdownType)
{
	using namespace UVTransferToolLocals;

	Settings->SaveProperties(this);
	DestinationMaterialSettings->SaveProperties(this, CacheIdentifier);

	SourceSeamVisualizer->Disconnect();
	DestinationSeamVisualizer->Disconnect();

	for (TObjectPtr<UToolTarget> Target : Targets)
	{
		UE::ToolTarget::ShowSourceObject(Target);
	}

	FDynamicMeshOpResult Result = DestinationPreview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Result);
	}

	SourcePreview->Disconnect();

	Settings = nullptr;
	DestinationMaterialSettings = nullptr;
	DestinationPreview = nullptr;
	SourcePreview = nullptr;
	SourceSeamVisualizer = nullptr;
	DestinationSeamVisualizer = nullptr;
	
	for (int i = 0; i < 2; ++i)
	{
		Meshes[i].Reset();
		SelectionTidSets[i].Reset();
	}
	
}

void UUVTransferTool::Setup()
{
	using namespace UVTransferToolLocals;

	UInteractiveTool::Setup();

	// Set up our property sets
	Settings = NewObject<UUVTransferToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	DestinationMaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	DestinationMaterialSettings->RestoreProperties(this, CacheIdentifier);
	AddToolPropertySource(DestinationMaterialSettings);

	// Hide inputs
	for (TObjectPtr<UToolTarget> Target : Targets)
	{
		UE::ToolTarget::HideSourceObject(Target);
	}

	// Extract the input meshes and selections
	for (int i = 0; i < 2; ++i)
	{
		Meshes[i] = MakeShared<UE::Geometry::FDynamicMesh3>(UE::ToolTarget::GetDynamicMeshCopy(Targets[i]));
		if (HasGeometrySelection(i))
		{
			SelectionTidSets[i].Emplace();
			EnumerateSelectionTriangles(GetGeometrySelection(0), *Meshes[i], [this, i](int32 Tid)
			{
				SelectionTidSets[i]->Add(Tid);
			});
		}
	}

	// Set up previews
	DestinationPreview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	DestinationPreview->Setup(GetTargetWorld(), this);
	DestinationPreview->OnOpCompleted.AddWeakLambda(this, [this](const UE::Geometry::FDynamicMeshOperator* UncastOp)
	{
		const FTransferUVsOp* Op = static_cast<const FTransferUVsOp*>(UncastOp);
		if (!Op->bTransferSucceeded)
		{
			GetToolManager()->DisplayMessage(LOCTEXT("TransferUnsuccessful", 
				"Transfer was not fully successful, possibly because correspondence couldn't be found. The "
				"source mesh is expected to be a version of the destination mesh simplified via \"Existing Positions\" "
				"option so that any UV layout done on the simplified mesh can be mapped back to the (original) destination "
				"mesh via vertex positions."),
				EToolMessageLevel::UserWarning);
		}
		else
		{
			GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
		}
	});
	DestinationPreview->OnMeshUpdated.AddWeakLambda(this, [this](UMeshOpPreviewWithBackgroundCompute* Compute)
	{
		if (DestinationSeamVisualizer)
		{
			DestinationSeamVisualizer->NotifyMeshChanged();
		}
	});
	
	SourcePreview = NewObject<UPreviewMesh>(this);
	SourcePreview->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	
	ReinitializePreviews();

	Settings->WatchProperty(Settings->SourceUVChannel, [this](const FString& NewValue)
	{
		InvalidatePreview();
	});
	Settings->WatchProperty(Settings->bSameDestinationChannel, [this](bool NewValue)
	{
		ReinitializeUVChannelOptions();
		InvalidatePreview();
	});
	Settings->WatchProperty(Settings->DestUVChannel, [this](const FString& NewValue)
	{
		DestinationMaterialSettings->UpdateUVChannels(Settings->GetDestUVChannel(),
			Settings->GetDestUVChannelNames());
		InvalidatePreview();
	});

	SourceSeamVisualizer = NewObject<UMeshElementsVisualizer>(this);
	SourceSeamVisualizer->CreateInWorld(GetWorld(), SourcePreview->GetTransform());
	if (ensure(SourceSeamVisualizer->Settings))
	{
		SourceSeamVisualizer->Settings->ShowAllElements(false);
	}
	SourceSeamVisualizer->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		SourcePreview->ProcessMesh(ProcessFunc);
	});

	DestinationSeamVisualizer = NewObject<UMeshElementsVisualizer>(this);
	DestinationSeamVisualizer->CreateInWorld(GetWorld(), DestinationPreview->PreviewMesh->GetTransform());
	if (ensure(DestinationSeamVisualizer->Settings))
	{
		DestinationSeamVisualizer->Settings->ShowAllElements(false);
	}
	DestinationSeamVisualizer->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		DestinationPreview->ProcessCurrentMesh(ProcessFunc);
	});
	UpdateVisualizations();
	
	Settings->WatchProperty(Settings->bReverseDirection, [this](bool) 
	{ 
		ReinitializePreviews();
		InvalidatePreview();
	});
	// TODO SELECTIONS: Once we support multi-mesh selections, we would add this bool to the settings
	//Settings->WatchProperty(Settings->bRespectSelection, [this](bool) { InvalidatePreview(); });

	Settings->WatchProperty(Settings->VertexSearchDistance, [this](double) { InvalidatePreview(); });
	Settings->WatchProperty(Settings->PathSimilarityWeight, [this](double) { InvalidatePreview(); });
	Settings->WatchProperty(Settings->bTransferSeamsOnly, [this](bool) { InvalidatePreview(); });
	Settings->WatchProperty(Settings->bClearExistingSeams, [this](bool) { InvalidatePreview(); });
	
	Settings->WatchProperty(Settings->bShowWireframes, [this](bool) { UpdateVisualizations(); });
	Settings->WatchProperty(Settings->bShowSeams, [this](bool) { UpdateVisualizations(); });

	Settings->SilentUpdateWatched();
	DestinationMaterialSettings->SilentUpdateWatched();

	SetToolDisplayName(LOCTEXT("ToolName", "UV Transfer"));
	GetToolManager()->DisplayMessage(LOCTEXT("OnStart", "Transfer UVs from a low-res mesh to a high-res mesh."),
		EToolMessageLevel::UserNotification);

	InvalidatePreview();
}

void UUVTransferTool::ReinitializePreviews()
{
	ReinitializeUVChannelOptions();

	int SourceIndex = Settings->bReverseDirection ? 1 : 0;
	int DestinationIndex = Settings->bReverseDirection ? 0 : 1;

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(DestinationPreview->PreviewMesh, Targets[DestinationIndex]);
	DestinationPreview->ConfigureMaterials(UE::ToolTarget::GetMaterialSet(Targets[DestinationIndex]).Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);
	DestinationPreview->OverrideMaterial = DestinationMaterialSettings->GetActiveOverrideMaterial();
	DestinationPreview->PreviewMesh->UpdatePreview(Meshes[DestinationIndex].Get());
	DestinationPreview->PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[DestinationIndex]));

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(SourcePreview, Targets[SourceIndex]);
	SourcePreview->SetMaterials(UE::ToolTarget::GetMaterialSet(Targets[SourceIndex]).Materials);
	SourcePreview->UpdatePreview(Meshes[SourceIndex].Get());
	SourcePreview->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[SourceIndex]));

	DestinationMaterialSettings->UpdateUVChannels(
		Settings->GetDestUVChannel(),
		Settings->GetDestUVChannelNames());

	DestinationMaterialSettings->UpdateMaterials();
	DestinationPreview->OverrideMaterial = DestinationMaterialSettings->GetActiveOverrideMaterial();

	if (SourceSeamVisualizer)
	{
		SourceSeamVisualizer->SetTransform(SourcePreview->GetTransform());
		SourceSeamVisualizer->NotifyMeshChanged();
	}
	if (DestinationSeamVisualizer)
	{
		DestinationSeamVisualizer->SetTransform(DestinationPreview->PreviewMesh->GetTransform());
		DestinationSeamVisualizer->NotifyMeshChanged();
	}
}

void UUVTransferTool::UpdateVisualizations()
{
	// The visualizers expect that we add their settings objects to our property sets, which
	//  we don't want to do, so we need to do their updates ourselves.
	if (ensure(SourceSeamVisualizer->Settings))
	{
		SourceSeamVisualizer->Settings->bShowWireframe = Settings->bShowWireframes;
		SourceSeamVisualizer->Settings->bShowUVSeams = Settings->bShowSeams;
		SourceSeamVisualizer->Settings->CheckAndUpdateWatched();
	}
	if (ensure(DestinationSeamVisualizer->Settings))
	{
		DestinationSeamVisualizer->Settings->bShowWireframe = Settings->bShowWireframes;
		DestinationSeamVisualizer->Settings->bShowUVSeams = Settings->bShowSeams;
		DestinationSeamVisualizer->Settings->CheckAndUpdateWatched();
	}
}

void UUVTransferTool::ReinitializeUVChannelOptions()
{
	if (!ensure(Settings && Meshes[0] && Meshes[1] && Meshes[0]->HasAttributes() && Meshes[1]->HasAttributes()))
	{
		return;
	}

	if (Settings->bSameDestinationChannel)
	{
		int32 NumChannels = FMath::Min(Meshes[0]->Attributes()->NumUVLayers(), Meshes[1]->Attributes()->NumUVLayers());
		Settings->ReinitializeChannelOptions(NumChannels, NumChannels);
	}
	else
	{
		int SourceIndex = Settings->bReverseDirection ? 1 : 0;
		int DestinationIndex = Settings->bReverseDirection ? 0 : 1;
		Settings->ReinitializeChannelOptions(Meshes[SourceIndex]->Attributes()->NumUVLayers(), Meshes[DestinationIndex]->Attributes()->NumUVLayers());
	}
}

void UUVTransferTool::OnTick(float DeltaTime)
{
	if (DestinationPreview)
	{
		DestinationPreview->Tick(DeltaTime);
	}
	if (DestinationSeamVisualizer)
	{
		DestinationSeamVisualizer->OnTick(DeltaTime);
	}
	if (SourceSeamVisualizer)
	{
		SourceSeamVisualizer->OnTick(DeltaTime);
	}
}

void UUVTransferTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

bool UUVTransferTool::CanAccept() const
{
	return DestinationPreview && DestinationPreview->HaveValidResult();
}

void UUVTransferTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == DestinationMaterialSettings)
	{
		DestinationMaterialSettings->UpdateMaterials();
		DestinationPreview->OverrideMaterial = DestinationMaterialSettings->GetActiveOverrideMaterial();
	}
}

TUniquePtr<UE::Geometry::FDynamicMeshOperator> UUVTransferTool::MakeNewOperator()
{
	using namespace UVTransferToolLocals;
	TUniquePtr<FTransferUVsOp> Op = MakeUnique<FTransferUVsOp>();

	int SourceIndex = Settings->bReverseDirection ? 1 : 0;
	int DestinationIndex = Settings->bReverseDirection ? 0 : 1;

	Op->SourceMesh = Meshes[SourceIndex];
	Op->DestinationMesh = Meshes[DestinationIndex];
	Op->SetTransform(UE::ToolTarget::GetLocalToWorldTransform(Targets[DestinationIndex]));
	
	// TODO SELECTIONS: Once we support multi-mesh selections, we would uncomment these
	//if (Settings->bRespectSelection)
	//{
	//	Op->SourceSelectionTids = SelectionTidSets[SourceIndex];
	//	Op->DestinationSelectionTids = SelectionTidSets[DestinationIndex];
	//}

	Op->SourceUVLayerIndex = Settings->GetSourceUVChannel();
	Op->DestUVLayerIndex = Settings->GetDestUVChannel();
	Op->bTransferSeamsOnly = Settings->bTransferSeamsOnly;
	Op->bClearExistingSeams = Settings->bClearExistingSeams;

	Op->VertexSearchDistance = Settings->VertexSearchDistance;
	Op->VertexSearchCellSize = Settings->VertexSearchDistance;
	Op->PathSimilarityWeight = Settings->PathSimilarityWeight;

	return Op;
}

void UUVTransferTool::InvalidatePreview()
{
	if (DestinationPreview)
	{
		DestinationPreview->InvalidateResult();
	}
}

void UUVTransferTool::GenerateAsset(const FDynamicMeshOpResult& Result)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("UVLayoutToolTransactionName", "UV Layout Tool"));
	UE::ToolTarget::CommitDynamicMeshUVUpdate(
		Targets[Settings->bReverseDirection ? 0 : 1], 
		Result.Mesh.Get());
	GetToolManager()->EndUndoTransaction();
}


// Builder:

UMultiTargetWithSelectionTool* UUVTransferToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UUVTransferTool>(SceneState.ToolManager);
}

bool UUVTransferToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	const int32 NumTargets = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	return NumTargets == 2;
}

const FToolTargetTypeRequirements& UUVTransferToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass()
		});
	return TypeRequirements;
}

void UUVTransferToolProperties::ReinitializeChannelOptions(int32 NumSourceChannels, int32 NumDestChannels)
{
	SourceUVChannels.Reset();
	DestUVChannels.Reset();

	auto InitializeChannelNames = [](int32 NumChannelsIn, TArray<FString>& ChannelsOut)
	{
		for (int32 i = 0; i < NumChannelsIn; ++i)
		{
			ChannelsOut.Add(FString::Printf(TEXT("UV %d"), i));
		}
	};
	InitializeChannelNames(NumSourceChannels, SourceUVChannels);
	InitializeChannelNames(NumDestChannels, DestUVChannels);

	auto GetUpdatedCurrentChannel = [](const TArray<FString>& Channels, const FString& CurrentChannelIn) -> FString
	{
		if (Channels.IsEmpty())
		{
			return TEXT("");
		}

		if (CurrentChannelIn.IsEmpty())
		{
			return Channels[0];
		}

		int32 FoundIndex = Channels.IndexOfByKey(CurrentChannelIn);
		if (FoundIndex < 0)
		{
			// Probably a UV layer past the one we have, so use last layer
			return Channels.Last();
		}

		return Channels[FoundIndex];
	};

	SourceUVChannel = GetUpdatedCurrentChannel(SourceUVChannels, SourceUVChannel);
	DestUVChannel = GetUpdatedCurrentChannel(DestUVChannels, DestUVChannel);
}

int32 UUVTransferToolProperties::GetSourceUVChannel()
{
	return SourceUVChannels.IndexOfByKey(SourceUVChannel);
}

int32 UUVTransferToolProperties::GetDestUVChannel()
{
	return bSameDestinationChannel ? GetSourceUVChannel() : DestUVChannels.IndexOfByKey(DestUVChannel);
}

#undef LOCTEXT_NAMESPACE
