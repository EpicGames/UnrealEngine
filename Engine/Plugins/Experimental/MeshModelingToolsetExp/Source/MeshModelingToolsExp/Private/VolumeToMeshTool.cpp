// Copyright Epic Games, Inc. All Rights Reserved.

#include "VolumeToMeshTool.h"
#include "InteractiveToolManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ToolBuilderUtil.h"

#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"

#include "Model.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeToMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UVolumeToMeshTool"

/*
 * ToolBuilder
 */

bool UVolumeToMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return ToolBuilderUtil::CountSelectedActorsOfType<AVolume>(SceneState) == 1;
}

UInteractiveTool* UVolumeToMeshToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UVolumeToMeshTool* NewTool = NewObject<UVolumeToMeshTool>(SceneState.ToolManager);

	NewTool->SetWorld(SceneState.World);

	AVolume* Volume = ToolBuilderUtil::FindFirstActorOfType<AVolume>(SceneState);
	check(Volume != nullptr);
	NewTool->SetSelection(Volume);

	return NewTool;
}




/*
 * Tool
 */
UVolumeToMeshTool::UVolumeToMeshTool()
{
	SetToolDisplayName(LOCTEXT("VolumeToMeshToolName", "Volume to Mesh"));
}


void UVolumeToMeshTool::SetSelection(AVolume* Volume)
{
	TargetVolume = Volume;
}


void UVolumeToMeshTool::Setup()
{
	UInteractiveTool::Setup();

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(TargetVolume->GetWorld(), FTransform::Identity);
	PreviewMesh->SetTransform(TargetVolume->GetActorTransform());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr); 

	PreviewMesh->SetMaterial( ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager()) );

	PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetSelectionMaterial(GetToolManager()));
	PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
	{
		return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
	});

	VolumeEdgesSet = NewObject<ULineSetComponent>(PreviewMesh->GetRootComponent());
	VolumeEdgesSet->SetupAttachment(PreviewMesh->GetRootComponent());
	VolumeEdgesSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	VolumeEdgesSet->RegisterComponent();

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this, TEXT("VolumeToMeshTool"));
	OutputTypeProperties->Initialize(true, false, true);
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	Settings = NewObject<UVolumeToMeshToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->bWeldEdges, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->bAutoRepair, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->bOptimizeMesh, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->bShowWireframe, [this](bool) { bResultValid = false; });

	bResultValid = false;

	GetToolManager()->DisplayMessage( 
		LOCTEXT("OnStartTool", "Convert a Volume to a Mesh"),
		EToolMessageLevel::UserNotification);
}



void UVolumeToMeshTool::Shutdown(EToolShutdownType ShutdownType)
{
	OutputTypeProperties->SaveProperties(this, TEXT("VolumeToMeshTool"));
	Settings->SaveProperties(this);

	FTransform3d Transform(PreviewMesh->GetTransform());
	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	if (ShutdownType == EToolShutdownType::Accept )
	{
		UMaterialInterface* UseMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

		FString NewName = TargetVolume.IsValid() ?
			FString::Printf(TEXT("%sMesh"), *TargetVolume->GetName()) : TEXT("Volume Mesh");

		GetToolManager()->BeginUndoTransaction(LOCTEXT("CreateMeshVolume", "Volume To Mesh"));

		FCreateMeshObjectParams NewMeshObjectParams;
		NewMeshObjectParams.TargetWorld = TargetWorld;
		NewMeshObjectParams.Transform = (FTransform)Transform;
		NewMeshObjectParams.BaseName = NewName;
		NewMeshObjectParams.Materials.Add(UseMaterial);
		NewMeshObjectParams.SetMesh(&CurrentMesh);
		OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
		FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
		if (Result.IsOK() && Result.NewActor != nullptr)
		{
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}

		GetToolManager()->EndUndoTransaction();
	}


}


void UVolumeToMeshTool::OnTick(float DeltaTime)
{
	if (bResultValid == false)
	{
		RecalculateMesh();
	}
}

void UVolumeToMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

bool UVolumeToMeshTool::CanAccept() const
{
	return bResultValid && CurrentMesh.TriangleCount() > 0;
}


void UVolumeToMeshTool::UpdateLineSet()
{
	VolumeEdgesSet->Clear();

	FColor BoundaryEdgeColor = LinearColors::VideoRed3b();
	float BoundaryEdgeThickness = 1.0;
	float BoundaryEdgeDepthBias = 2.0f;

	FColor WireEdgeColor = LinearColors::Gray3b();
	float WireEdgeThickness = 0.1f;
	float WireEdgeDepthBias = 1.0f;

	if (Settings->bShowWireframe)
	{
		VolumeEdgesSet->ReserveLines(CurrentMesh.EdgeCount());

		for (int32 eid : CurrentMesh.EdgeIndicesItr())
		{
			FVector3d A, B;
			CurrentMesh.GetEdgeV(eid, A, B);
			if (CurrentMesh.IsBoundaryEdge(eid))
			{
				VolumeEdgesSet->AddLine((FVector)A, (FVector)B,
					BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
			}
			else
			{
				VolumeEdgesSet->AddLine((FVector)A, (FVector)B,
					WireEdgeColor, WireEdgeThickness, WireEdgeDepthBias);
			}
		}
	}
}

void UVolumeToMeshTool::RecalculateMesh()
{
	if (TargetVolume.IsValid())
	{
		UE::Conversion::FVolumeToMeshOptions Options;
		Options.bMergeVertices = Settings->bWeldEdges;
		Options.bAutoRepairMesh = Settings->bAutoRepair;
		Options.bOptimizeMesh = Settings->bOptimizeMesh;

		CurrentMesh = FDynamicMesh3(EMeshComponents::FaceGroups);
		UE::Conversion::VolumeToDynamicMesh(TargetVolume.Get(), CurrentMesh, Options);

		// compute normals for current polygroup topology
		CurrentMesh.EnableAttributes();
		FDynamicMeshNormalOverlay* Normals = CurrentMesh.Attributes()->PrimaryNormals();
		FMeshNormals::InitializeOverlayTopologyFromFaceGroups(&CurrentMesh, Normals);
		FMeshNormals::QuickRecomputeOverlayNormals(CurrentMesh);

		PreviewMesh->UpdatePreview(&CurrentMesh);
	}

	UpdateLineSet();

	bResultValid = true;
}




#undef LOCTEXT_NAMESPACE

