// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoxelBlendMeshesTool.h"
#include "CompositionOps/VoxelBlendMeshesOp.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "Selection/ToolSelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "CompositionOps/VoxelBlendMeshesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoxelBlendMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UVoxelBlendMeshesTool"

/*
 * Tool
 */

void UVoxelBlendMeshesTool::SetupProperties()
{
	Super::SetupProperties();
	BlendProperties = NewObject<UVoxelBlendMeshesToolProperties>(this);
	BlendProperties->RestoreProperties(this);
	AddToolPropertySource(BlendProperties);

	SetToolDisplayName(LOCTEXT("VoxelBlendMeshesToolName", "Voxel Blend"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("VoxelBlendMeshesToolDescription", "Compute a volumetric Blend of the input meshes, controlled by the Blend Power/Falloff. UVs, sharp edges, and small/thin features will be lost. Increase Voxel Count to enhance accuracy."),
		EToolMessageLevel::UserNotification);
}


void UVoxelBlendMeshesTool::SaveProperties()
{
	Super::SaveProperties();
	BlendProperties->SaveProperties(this);
}


void UVoxelBlendMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	Super::ConvertInputsAndSetPreviewMaterials(bSetPreviewMesh);

	if (!BlendProperties->bVoxWrap && HasOpenBoundariesInMeshInputs())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("WarnOpenEdges", "Open edges found: Consider using the 'Vox Wrap Preprocess' settings to avoid artifacts."),
			EToolMessageLevel::UserWarning);
	}
}


TUniquePtr<FDynamicMeshOperator> UVoxelBlendMeshesTool::MakeNewOperator()
{
	TUniquePtr<FVoxelBlendMeshesOp> Op = MakeUnique<FVoxelBlendMeshesOp>();

	Op->Transforms.SetNum(Targets.Num());
	Op->Meshes.SetNum(Targets.Num());
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		Op->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		Op->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}

	Op->BlendFalloff = BlendProperties->BlendFalloff;
	Op->BlendPower = BlendProperties->BlendPower;
	Op->bSubtract = BlendProperties->Operation == EVoxelBlendOperation::Subtract;
	Op->bVoxWrap = BlendProperties->bVoxWrap;
	Op->bRemoveInternalsAfterVoxWrap = BlendProperties->bRemoveInternalsAfterVoxWrap;
	Op->ThickenShells = BlendProperties->ThickenShells;

	VoxProperties->SetPropertiesOnOp(*Op);

	return Op;
}


FString UVoxelBlendMeshesTool::GetCreatedAssetName() const
{
	return TEXT("Blended");
}

FText UVoxelBlendMeshesTool::GetActionName() const
{
	return LOCTEXT("VoxelBlendMeshes", "Voxel Blend");
}







#undef LOCTEXT_NAMESPACE

