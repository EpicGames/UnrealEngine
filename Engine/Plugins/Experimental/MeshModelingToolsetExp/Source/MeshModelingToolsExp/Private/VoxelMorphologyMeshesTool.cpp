// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoxelMorphologyMeshesTool.h"

#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "Selection/ToolSelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "CompositionOps/VoxelMorphologyMeshesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoxelMorphologyMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UVoxelMorphologyMeshesTool"


void UVoxelMorphologyMeshesTool::SetupProperties()
{
	Super::SetupProperties();

	MorphologyProperties = NewObject<UVoxelMorphologyMeshesToolProperties>(this);
	MorphologyProperties->RestoreProperties(this);
	AddToolPropertySource(MorphologyProperties);

	SetToolDisplayName(LOCTEXT("VoxelMorphologyMeshesToolName", "Voxel Offset"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Apply Offset (aka Morphology) operations to the input meshes to create a new Mesh, using voxelization techniques. UVs, sharp edges, and small/thin features will be lost. Increase Voxel Count to enhance accuracy."),
		EToolMessageLevel::UserNotification);

}


void UVoxelMorphologyMeshesTool::SaveProperties()
{
	Super::SaveProperties();
	MorphologyProperties->SaveProperties(this);
}


void UVoxelMorphologyMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	Super::ConvertInputsAndSetPreviewMaterials(bSetPreviewMesh);

	if (!MorphologyProperties->bVoxWrap && HasOpenBoundariesInMeshInputs())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("WarnOpenEdges", "Open edges found: Consider using the 'Vox Wrap Preprocess' settings to avoid artifacts."),
			EToolMessageLevel::UserWarning);
	}
}


TUniquePtr<FDynamicMeshOperator> UVoxelMorphologyMeshesTool::MakeNewOperator()
{
	TUniquePtr<FVoxelMorphologyMeshesOp> Op = MakeUnique<FVoxelMorphologyMeshesOp>();

	Op->Transforms.SetNum(Targets.Num());
	Op->Meshes.SetNum(Targets.Num());
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		Op->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		Op->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}

	VoxProperties->SetPropertiesOnOp(*Op);
	
	Op->bVoxWrapInput = MorphologyProperties->bVoxWrap;
	Op->ThickenShells = MorphologyProperties->ThickenShells;
	Op->bRemoveInternalsAfterVoxWrap = MorphologyProperties->bRemoveInternalsAfterVoxWrap;
	Op->Distance = MorphologyProperties->Distance;
	Op->Operation = MorphologyProperties->Operation;

	return Op;
}


FString UVoxelMorphologyMeshesTool::GetCreatedAssetName() const
{
	return TEXT("Offset");
}

FText UVoxelMorphologyMeshesTool::GetActionName() const
{
	return LOCTEXT("VoxelMorphologyMeshes", "Voxel Offset");
}


#undef LOCTEXT_NAMESPACE

