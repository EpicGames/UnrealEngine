// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoxelSolidifyMeshesTool.h"
#include "CompositionOps/VoxelSolidifyMeshesOp.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"

#include "Selection/ToolSelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "CompositionOps/VoxelSolidifyMeshesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoxelSolidifyMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UVoxelSolidifyMeshesTool"


void UVoxelSolidifyMeshesTool::SetupProperties()
{
	Super::SetupProperties();
	SolidifyProperties = NewObject<UVoxelSolidifyMeshesToolProperties>(this);
	SolidifyProperties->RestoreProperties(this);
	AddToolPropertySource(SolidifyProperties);

	SetToolDisplayName(LOCTEXT("VoxelSolidifyMeshesToolName", "Voxel Wrap"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("VoxelSolidifyMeshesToolDescription", "Create a new closed/solid shell mesh that wraps the input meshes. Holes will automatically be filled, controlled by the Winding Threshold. UVs, sharp edges, and small/thin features will be lost. Increase Voxel Count to enhance accuracy."),
		EToolMessageLevel::UserNotification);
}


void UVoxelSolidifyMeshesTool::SaveProperties()
{
	Super::SaveProperties();
	SolidifyProperties->SaveProperties(this);
}


TUniquePtr<FDynamicMeshOperator> UVoxelSolidifyMeshesTool::MakeNewOperator()
{
	TUniquePtr<FVoxelSolidifyMeshesOp> Op = MakeUnique<FVoxelSolidifyMeshesOp>();

	Op->Transforms.SetNum(Targets.Num());
	Op->Meshes.SetNum(Targets.Num());
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		Op->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		Op->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}

	Op->bSolidAtBoundaries = SolidifyProperties->bSolidAtBoundaries;
	Op->WindingThreshold = SolidifyProperties->WindingThreshold;
	Op->bApplyThickenShells = SolidifyProperties->bApplyThickenShells;
	Op->ThickenShells = SolidifyProperties->ThickenShells;
	Op->SurfaceSearchSteps = SolidifyProperties->SurfaceSearchSteps;
	Op->ExtendBounds = SolidifyProperties->ExtendBounds;
	VoxProperties->SetPropertiesOnOp(*Op);
	
	return Op;
}



FString UVoxelSolidifyMeshesTool::GetCreatedAssetName() const
{
	return TEXT("Solid");
}

FText UVoxelSolidifyMeshesTool::GetActionName() const
{
	return LOCTEXT("VoxelSolidifyMeshes", "Voxel Shell");
}









#undef LOCTEXT_NAMESPACE

