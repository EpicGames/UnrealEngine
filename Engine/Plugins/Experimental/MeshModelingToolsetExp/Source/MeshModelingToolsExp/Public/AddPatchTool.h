// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialShared.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveToolBuilder.h"
#include "BaseTools/SingleClickTool.h"
#include "PreviewMesh.h"
#include "Properties/MeshMaterialProperties.h"
#include "AddPatchTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 *
 */
UCLASS(MinimalAPI)
class UAddPatchToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



UCLASS(MinimalAPI)
class UAddPatchToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API UAddPatchToolProperties();

	/** Width of Shape */
	UPROPERTY(EditAnywhere, Category = PatchSettings, meta = (DisplayName = "Width", UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0"))
	float Width;

	/** Rotation around up axis */
	UPROPERTY(EditAnywhere, Category = PatchSettings, meta = (DisplayName = "Rotation", UIMin = "0.0", UIMax = "360.0"))
	float Rotation;

	/** Subdivisions */
	UPROPERTY(EditAnywhere, Category = PatchSettings, meta = (DisplayName = "Subdivisions", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "4000"))
	int Subdivisions;

	/** Rotation around up axis */
	UPROPERTY(EditAnywhere, Category = PatchSettings, meta = (DisplayName = "Shift", UIMin = "-1000", UIMax = "1000"))
	float Shift;
};







/**
 *
 */
UCLASS(MinimalAPI)
class UAddPatchTool : public USingleClickTool, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;


	// IHoverBehaviorTarget interface
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;


protected:
	UPROPERTY()
	TObjectPtr<UAddPatchToolProperties> ShapeSettings;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;



	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

protected:
	UWorld* TargetWorld;

	FBox WorldBounds;

	UE::Geometry::FFrame3f ShapeFrame;
	bool bPreviewValid = true;

	UE_API void UpdatePreviewPosition(const FInputDeviceRay& ClickPos);
	UE_API void UpdatePreviewMesh();

	TUniquePtr<UE::Geometry::FDynamicMesh3> BaseMesh;
	UE_API void GeneratePreviewBaseMesh();

	UE_API void GeneratePlane(UE::Geometry::FDynamicMesh3* OutMesh);
};

#undef UE_API
