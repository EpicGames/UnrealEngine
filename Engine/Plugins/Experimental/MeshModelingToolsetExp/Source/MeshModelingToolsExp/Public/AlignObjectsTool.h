// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Changes/TransformChange.h"
#include "FrameTypes.h"
#include "BoxTypes.h"
#include "AlignObjectsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPrimitiveComponent;

/**
 *
 */
UCLASS(MinimalAPI)
class UAlignObjectsToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class EAlignObjectsAlignTypes
{
	Pivots,
	BoundingBoxes
};

UENUM()
enum class EAlignObjectsAlignToOptions
{
	FirstSelected,
	LastSelected,
	Combined
};

UENUM()
enum class EAlignObjectsBoxPoint
{
	Center,
	Bottom,
	Top,
	Left,
	Right,
	Front,
	Back,
	Min,
	Max
};



/**
 * Standard properties of the Align Objects Operation
 */
UCLASS(MinimalAPI)
class UAlignObjectsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = Options)
	EAlignObjectsAlignTypes AlignType = EAlignObjectsAlignTypes::BoundingBoxes;

	UPROPERTY(EditAnywhere, Category = Options)
	EAlignObjectsAlignToOptions AlignTo = EAlignObjectsAlignToOptions::Combined;

	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "AlignType == EAlignObjectsAlignTypes::BoundingBoxes || AlignTo == EAlignObjectsAlignToOptions::Combined"))
	EAlignObjectsBoxPoint BoxPosition = EAlignObjectsBoxPoint::Center;

	UPROPERTY(EditAnywhere, Category = Axes)
	bool bAlignX = false;

	UPROPERTY(EditAnywhere, Category = Axes)
	bool bAlignY = false;

	UPROPERTY(EditAnywhere, Category = Axes)
	bool bAlignZ = true;
};





/**
 * UAlignObjectsTool transforms the input Components so that they are aligned in various ways, depending on the current settings.
 * The object positions move after every change in the parameters. Currently those changes are not transacted.
 * On cancel the original positions are restored, and on accept the positions are updated with a transaction.
 */
UCLASS(MinimalAPI)
class UAlignObjectsTool : public UMultiSelectionMeshEditingTool, public IClickDragBehaviorTarget
{
	GENERATED_BODY()
	using FAxisAlignedBox3d = UE::Geometry::FAxisAlignedBox3d;
public:
	UE_API UAlignObjectsTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


	// ICLickDragBehaviorTarget interface
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

public:
	UPROPERTY()
	TObjectPtr<UAlignObjectsToolProperties> AlignProps;


protected:

	struct FAlignInfo
	{
		UPrimitiveComponent* Component;
		FTransform SavedTransform;
		FTransform3d WorldTransform;
		FVector3d WorldPivot;
		FAxisAlignedBox3d WorldBounds;
	};

	TArray<FAlignInfo> ComponentInfo;

	FAxisAlignedBox3d CombinedBounds;
	FAxisAlignedBox3d PivotBounds;
	FVector3d AveragePivot;

	UE_API void Precompute();

	bool bAlignDirty = false;
	UE_API void UpdateAlignment();
	UE_API void UpdateAlignment_Pivots();
	UE_API void UpdateAlignment_BoundingBoxes();
};

#undef UE_API
