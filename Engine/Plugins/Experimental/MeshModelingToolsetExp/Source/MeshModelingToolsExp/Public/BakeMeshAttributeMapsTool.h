// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageBuilder.h"
#include "Sampling/MeshMapBaker.h"
#include "ModelingOperators.h"
#include "PreviewMesh.h"
#include "ModelingToolTargetUtil.h"
#include "Baking/BakingTypes.h"
#include "BakeMeshAttributeMapsToolBase.h"
#include "BakeMeshAttributeMapsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


// Forward declarations
class UMaterialInstanceDynamic;
class UTexture2D;
PREDECLARE_GEOMETRY(template<typename RealType> class TMeshTangents);
PREDECLARE_GEOMETRY(class FMeshImageBakingCache);
using UE::Geometry::FImageDimensions;


/**
 * Tool Builder
 */

UCLASS(MinimalAPI)
class UBakeMeshAttributeMapsToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};






UCLASS(MinimalAPI)
class UBakeMeshAttributeMapsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** The bake output types to generate */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (DisplayName = "Output Types", Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		ValidEnumValues="TangentSpaceNormal, AmbientOcclusion, BentNormal, Curvature, Texture, ObjectSpaceNormal, FaceNormal, Position, MaterialID, PolyGroupID, MultiTexture, VertexColor, UVShell, Height"))
	int32 MapTypes = static_cast<int32>(EBakeMapType::None);

	/** The baked output type used for preview in the viewport */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (DisplayName = "Preview Output Type", GetOptions = GetMapPreviewNamesFunc, TransientToolProperty,
		EditCondition = "MapTypes != 0", NoResetToDefault))
	FString MapPreview;

	/** The pixel resolution of the generated textures */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureResolution Resolution = EBakeTextureResolution::Resolution256;

	/** The bit depth for each channel of the generated textures */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureBitDepth BitDepth = EBakeTextureBitDepth::ChannelBits8;

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureSamplesPerPixel SamplesPerPixel = EBakeTextureSamplesPerPixel::Sample1;

	/** Mask texture for filtering out samples/pixels from the output texture */
	UPROPERTY(EditAnywhere, Category = Textures, AdvancedDisplay)
	TObjectPtr<UTexture2D> SampleFilterMask = nullptr;

	UFUNCTION()
	UE_API const TArray<FString>& GetMapPreviewNamesFunc();
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> MapPreviewNamesList;
	TMap<FString, FString> MapPreviewNamesMap;
};


/**
 * Detail Map Baking Tool
 */
UCLASS(MinimalAPI)
class UBakeMeshAttributeMapsTool : public UBakeMeshAttributeMapsToolBase
{
	GENERATED_BODY()

public:
	UBakeMeshAttributeMapsTool() = default;

	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	// End UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

protected:
	// need to update bResultValid if these are modified, so we don't publicly expose them. 
	// @todo setters/getters for these

	UPROPERTY()
	TObjectPtr<UBakeInputMeshProperties> InputMeshSettings;

	UPROPERTY()
	TObjectPtr<UBakeMeshAttributeMapsToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UBakeMeshAttributeMapsResultToolProperties> ResultSettings;

	UPROPERTY()
	TObjectPtr<UBakeUVShellMapToolProperties> UVShellSettings;


	// Begin UBakeMeshAttributeMapsToolBase interface
	UE_API virtual void UpdateResult() override;
	UE_API virtual void UpdateVisualization() override;

	UE_API virtual void GatherAnalytics(FBakeAnalytics::FMeshSettings& Data) override;
	// End UBakeMeshAttributeMapsToolBase interface

	friend class FMeshMapBakerOp;

	bool bIsBakeToSelf = false;

	UE_API bool ValidDetailMeshTangents();
	bool bCheckDetailMeshTangents = true;
	bool bValidDetailMeshTangents = false;
	
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> DetailMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;
	TSharedPtr<UE::Geometry::TMeshTangents<double>, ESPMode::ThreadSafe> DetailMeshTangents;
	int32 DetailMeshTimestamp = 0;
	UE_API void UpdateDetailMesh();

	UE_API void UpdateOnModeChange();

	UE_API void InvalidateResults();

	UE_API EBakeOpState UpdateResult_DetailMeshTangents(EBakeMapType BakeType);

	FDetailMeshSettings CachedDetailMeshSettings;
	TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> CachedDetailNormalMap;
	UE_API EBakeOpState UpdateResult_DetailNormalMap();

	FUVShellMapSettings CachedUVShellMapSettings;
	UE_API EBakeOpState UpdateResult_UVShellMap(const FImageDimensions& Dimensions);

	void SetSourceObjectVisible(bool bState)
	{
		if (!bIsBakeToSelf)
		{
			UE::ToolTarget::SetSourceObjectVisible(Targets[1], bState);
		}
	}
};

#undef UE_API
