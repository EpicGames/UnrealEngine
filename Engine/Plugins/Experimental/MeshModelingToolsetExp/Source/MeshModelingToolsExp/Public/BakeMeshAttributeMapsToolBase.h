// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetUtils/Texture2DBuilder.h"
#include "BakeMeshAttributeTool.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/Texture2D.h"
#include "Image/ImageDimensions.h"
#include "InteractiveToolManager.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshOpPreviewHelpers.h"
#include "ModelingOperators.h"
#include "PreviewMesh.h"
#include "Sampling/MeshMapBaker.h"
#include "Baking/BakingTypes.h"
#include "BakeMeshAttributeMapsToolBase.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


UCLASS(MinimalAPI)
class UBakeMeshAttributeMapsResultToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Bake */
	UPROPERTY(VisibleAnywhere, Category = Results, meta = (DisplayName = "Results", TransientToolProperty))
	TMap<EBakeMapType, TObjectPtr<UTexture2D>> Result;
};


/**
 * Base Bake Maps tool
 */
UCLASS(MinimalAPI)
class UBakeMeshAttributeMapsToolBase : public UBakeMeshAttributeTool, public UE::Geometry::IGenericDataOperatorFactory<UE::Geometry::FMeshMapBaker>
{
	GENERATED_BODY()

public:
	UBakeMeshAttributeMapsToolBase() = default;

	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	// End UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

	/**
	 * Process a bitfield into an EBakeMapType. This function
	 * may inject additional map types based on the enabled bits.
	 * For example, enabling AmbientOcclusion if BentNormal is
	 * active.
	 * @return An enumerated map type from a bitfield
	 */
	static UE_API EBakeMapType GetMapTypes(const int32& MapTypes);

protected:
	//
	// Tool property sets
	//
	UPROPERTY()
	TObjectPtr<UBakeVisualizationProperties> VisualizationProps;

	//
	// Preview mesh and materials
	//
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> BentNormalPreviewMaterial;

	/**
	 * Post-client setup function. Should be invoked at end of client Setup().
	 * Initialize common base tool properties (ex. visualization properties) and
	 * analytics.
	 */
	UE_API void PostSetup();
	
	/**
	 * Process dirty props and update background compute.
	 * Invoked during Render.
	 */
	UE_API virtual void UpdateResult();

	/**
	 * Updates the preview material on the preview mesh with the
	 * computed results. Invoked by OnMapsUpdated.
	 */
	UE_API virtual void UpdateVisualization();

	/**
	 * Invalidates the background compute operator.
	 */
	UE_API void InvalidateCompute();

	/**
	 * Create texture assets from our result map of Texture2D
	 * @param Textures the result map of textures to create
	 * @param SourceWorld the source world to define where the texture assets will be stored.
	 * @param SourceAsset if not null, result textures will be stored adjacent to this asset.
	 */
	UE_API void CreateTextureAssets(const TMap<EBakeMapType, TObjectPtr<UTexture2D>>& Textures, UWorld* SourceWorld, UObject* SourceAsset);

	//
	// Bake parameters
	//
	struct FBakeSettings
	{
		EBakeMapType SourceBakeMapTypes = EBakeMapType::None;
		EBakeMapType BakeMapTypes = EBakeMapType::None;
		FImageDimensions Dimensions;
		EBakeTextureBitDepth BitDepth = EBakeTextureBitDepth::ChannelBits8;
		int32 TargetUVLayer = 0;
		int32 DetailTimestamp = 0;
		float ProjectionDistance = 3.0;
		int32 SamplesPerPixel = 1;
		bool bProjectionInWorldSpace = false;

		bool operator==(const FBakeSettings& Other) const
		{
			return BakeMapTypes == Other.BakeMapTypes && Dimensions == Other.Dimensions &&
				TargetUVLayer == Other.TargetUVLayer && DetailTimestamp == Other.DetailTimestamp &&
				ProjectionDistance == Other.ProjectionDistance && SamplesPerPixel == Other.SamplesPerPixel &&
				BitDepth == Other.BitDepth && SourceBakeMapTypes == Other.SourceBakeMapTypes &&
				bProjectionInWorldSpace == Other.bProjectionInWorldSpace;
		}
	};
	FBakeSettings CachedBakeSettings;

	UE_API EBakeOpState UpdateResult_SampleFilterMask(UTexture2D* SampleFilterMask);
	TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> CachedSampleFilterMask;

	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> TargetMeshUVCharts;

	/**
	 * To be invoked by client when bake map types change.
	 * @param ResultMapTypes the requested map types to compute
	 * @param Result the output map of bake result textures
	 * @param MapPreview the map preview string property
	 * @param MapPreviewNamesList the stored list of map preview display names.
	 * @param MapPreviewNamesMap the stored map of map preview display names to enum string values.
	 */
	UE_API void OnMapTypesUpdated(
		EBakeMapType ResultMapTypes,
		TMap<EBakeMapType, TObjectPtr<UTexture2D>>& Result,
		FString& MapPreview,
		TArray<FString>& MapPreviewNamesList,
		TMap<FString, FString>& MapPreviewNamesMap);

	//
	// Background compute
	//
	TUniquePtr<TGenericDataBackgroundCompute<UE::Geometry::FMeshMapBaker>> Compute = nullptr;

	/**
	 * Internal cache of bake texture results.
	 * The tool can inject additional bake types that were not requested by the user. This
	 * can occur in cases where a particular bake type might need another bake type to preview
	 * such as BentNormal requiring AmbientOcclusion to preview. To avoid writing out assets
	 * that the user did not request, we introduce CachedMaps as a temporary texture cache
	 * for the tool preview. The Result array is then updated from CachedMaps to only hold
	 * user requested textures that are written out on Shutdown.
	 */
	UPROPERTY()
	TMap<EBakeMapType, TObjectPtr<UTexture2D>> CachedMaps;

	/**
	 * Retrieves the result of the FMeshMapBaker and generates UTexture2D into the CachedMaps.
	 * It is the responsibility of the client to ensure that CachedMaps is appropriately sized for
	 * the range of index values in MapIndex.
	 * 
	 * @param NewResult the resulting FMeshMapBaker from the background Compute
	 */
	UE_API void OnMapsUpdated(const TUniquePtr<UE::Geometry::FMeshMapBaker>& NewResult);


	/**
	 * Update the preview material parameters for a given Bake type
	 * display name.
	 * @param PreviewDisplayName Displayed UI preview name to preview
	 * @param MapPreviewNamesMap Map containing the list of displayed preview name to enum string value.
	 */
	UE_API void UpdatePreview(const FString& PreviewDisplayName, const TMap<FString, FString>& MapPreviewNamesMap);


	/**
	 * Update the preview material parameters for a given a Bake type.
	 * @param PreviewMapType EBakeMapType to preview
	 */
	UE_API void UpdatePreview(EBakeMapType PreviewMapType);


	/**
	 * Resets the preview material parameters to their default state.
	 */
	UE_API void ResetPreview();
	

	/**
	 * Updates a tool property set's MapPreviewNamesList from the list of
	 * active map types. Also updates the MapPreview property if the current
	 * preview option is no longer available.
	 * 
	 * @param MapTypes the requested map types to compute
	 * @param MapPreview the map preview string property
	 * @param MapPreviewNamesList the stored list of map preview display names.
	 * @param MapPreviewNamesMap the stored map of map preview display names to enum string values.
	 */
	UE_API void UpdatePreviewNames(
		const EBakeMapType MapTypes,
		FString& MapPreview,
		TArray<FString>& MapPreviewNamesList,
		TMap<FString, FString>& MapPreviewNamesMap);


	//
	// Analytics
	//
	struct FBakeAnalytics
	{
		double TotalBakeDuration = 0.0;
		double WriteToImageDuration = 0.0;
		double WriteToGutterDuration = 0.0;
		int64 NumSamplePixels = 0;
		int64 NumGutterPixels = 0;

		struct FMeshSettings
		{
			int32 NumTargetMeshTris = 0;
			int32 NumDetailMesh = 0;
			int64 NumDetailMeshTris = 0;
		};
		FMeshSettings MeshSettings;

		FBakeSettings BakeSettings;
		FOcclusionMapSettings OcclusionSettings;
		FCurvatureMapSettings CurvatureSettings;
	};
	FBakeAnalytics BakeAnalytics;

	/**
	 * Computes the NumTargetMeshTris, NumDetailMesh and NumDetailMeshTris analytics.
	 * @param Data the mesh analytics data to compute
	 */
	UE_API virtual void GatherAnalytics(FBakeAnalytics::FMeshSettings& Data);

	/**
	 * Records bake timing and settings data for analytics.
	 * @param Result the result of the bake.
	 * @param Settings The bake settings used for the bake.
	 * @param Data the output bake analytics struct.
	 */
	static UE_API void GatherAnalytics(const UE::Geometry::FMeshMapBaker& Result,
								const FBakeSettings& Settings,
								FBakeAnalytics& Data);

	/**
	 * Posts an analytics event using the given analytics struct.
	 * @param Data the bake analytics struct to output.
	 * @param EventName the name of the analytics event to output.
	 */
	static UE_API void RecordAnalytics(const FBakeAnalytics& Data, const FString& EventName);

	
	/**
	 * @return the analytics event name for this tool.
	 */
	virtual FString GetAnalyticsEventName() const
	{
		return TEXT("BakeTexture");
	}
	
	//
	// Utilities
	//
	
	/** @return the Texture2D type for a given map type */
	static UE_API UE::Geometry::FTexture2DBuilder::ETextureType GetTextureType(EBakeMapType MapType, EBakeTextureBitDepth MapFormat);

	/** @return the texture name given a base name and map type */
	static UE_API void GetTextureName(EBakeMapType MapType, const FString& BaseName, FString& TexName);
	

	// empty maps are shown when nothing is computed
	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyNormalMap;

	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyColorMapBlack;

	UPROPERTY()
	TObjectPtr<UTexture2D> EmptyColorMapWhite;

	UE_API void InitializeEmptyMaps();
};



#undef UE_API
