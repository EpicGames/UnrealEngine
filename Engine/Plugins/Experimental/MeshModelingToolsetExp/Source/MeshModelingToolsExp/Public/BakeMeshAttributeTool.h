// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BakeMeshAttributeToolCommon.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/Texture2D.h"
#include "InteractiveToolQueryInterfaces.h" // for UInteractiveToolExclusiveToolAPI
#include "Materials/MaterialInstanceDynamic.h"
#include "PreviewMesh.h"
#include "BakeToolUtils.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "BakeMeshAttributeTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

/**
 * Bake map enums
 */

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EBakeMapType
{
	None                   = 0,

	/* Normals in tangent space */
	TangentSpaceNormal     = 1 << 0 UMETA(DisplayName = "Tangent Normal"),
	/* Interpolated normals in object space */
	ObjectSpaceNormal      = 1 << 1 UMETA(DisplayName = "Object Normal"),
	/* Geometric face normals in object space */
	FaceNormal             = 1 << 2,
	/* Normals skewed towards the least occluded direction */
	BentNormal             = 1 << 3,
	/* Positions in object space */
	Position               = 1 << 4,
	/* Local curvature of the mesh surface */
	Curvature              = 1 << 5,
	/* Ambient occlusion sampled across the hemisphere */
	AmbientOcclusion       = 1 << 6,
	/* Transfer a given texture */
	Texture                = 1 << 7,
	/* Transfer a texture per material ID */
	MultiTexture           = 1 << 8,
	/* Interpolated vertex colors */
	VertexColor            = 1 << 9,
	/* Material IDs as unique colors */
	MaterialID             = 1 << 10 UMETA(DisplayName = "Material ID"),
	/* PolyGroup IDs as unique colors */
	PolyGroupID            = 1 << 11 UMETA(DisplayName = "PolyGroup ID"),
	/* Constant value of One */
	One                    = 1 << 12,
	/* Constant value of Zero */
	Zero                   = 1 << 13,
	/* UV shell */
	UVShell                = 1 << 14 UMETA(DisplayName = "UV Shell"),
	/* Height */
	Height                 = 1 << 15,

	All                    = 0x7FFF UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EBakeMapType);

/**
 * Array of all bake map types. This defines the order in which
 * the bake map types are iterated in the bake tools. Consequently this
 * ordering is reflected in the order of the bake Results property.
 */
static constexpr EBakeMapType ENUM_EBAKEMAPTYPE_ALL[] =
{
	EBakeMapType::TangentSpaceNormal,
	EBakeMapType::ObjectSpaceNormal,
	EBakeMapType::FaceNormal,
	EBakeMapType::BentNormal,
	EBakeMapType::Position,
	EBakeMapType::Curvature,
	EBakeMapType::Height,
	EBakeMapType::AmbientOcclusion,
	EBakeMapType::Texture,
	EBakeMapType::MultiTexture,
	EBakeMapType::VertexColor,
	EBakeMapType::MaterialID,
	EBakeMapType::PolyGroupID,
	EBakeMapType::UVShell
};


/**
 * Base Mesh Bake tool
 */
UCLASS(MinimalAPI)
class UBakeMeshAttributeTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolExclusiveToolAPI, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UBakeMeshAttributeTool() = default;

	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }
	// End UInteractiveTool interface

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:
	//
	// Bake tool property sets
	//
	UPROPERTY()
	TObjectPtr<UBakeOcclusionMapToolProperties> OcclusionSettings;

	UPROPERTY()
	TObjectPtr<UBakeCurvatureMapToolProperties> CurvatureSettings;

	UPROPERTY()
	TObjectPtr<UBakeTexture2DProperties> TextureSettings;

	UPROPERTY()
	TObjectPtr<UBakeMultiTexture2DProperties> MultiTextureSettings;

	UPROPERTY()
	TObjectPtr<UBakeHeightMapToolProperties> HeightSettings;
	
	
	//
	// Preview materials
	//
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> WorkingPreviewMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> ErrorPreviewMaterial;
	
	float SecondsBeforeWorkingMaterial = 0.75;

protected:
	//
	// Bake parameters
	//
	EBakeOpState OpState = EBakeOpState::Evaluate;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> TargetMesh;
	TSharedPtr<UE::Geometry::TMeshTangents<double>, ESPMode::ThreadSafe> TargetMeshTangents;

	/**
	 * Compute validity of the Target Mesh tangents. Only checks validity
	 * once and then caches the result for successive calls.
	 * 
	 * @return true if the TargetMesh tangents are valid
	 */
	UE_API bool ValidTargetMeshTangents();
	bool bCheckTargetMeshTangents = true;
	bool bValidTargetMeshTangents = false;

	//
	// Bake results update management
	//
	const bool bPreferPlatformData = false;

	/** Update the OpState based on the validity of the target mesh tangents */
	UE_API EBakeOpState UpdateResult_TargetMeshTangents(EBakeMapType BakeType);
	
	UE_API EBakeOpState UpdateResult_Normal(const FImageDimensions& Dimensions);
	FNormalMapSettings CachedNormalMapSettings;

	UE_API EBakeOpState UpdateResult_Occlusion(const FImageDimensions& Dimensions);
	FOcclusionMapSettings CachedOcclusionMapSettings;

	UE_API EBakeOpState UpdateResult_Curvature(const FImageDimensions& Dimensions);
	FCurvatureMapSettings CachedCurvatureMapSettings;

	UE_API EBakeOpState UpdateResult_MeshProperty(const FImageDimensions& Dimensions);
	FMeshPropertyMapSettings CachedMeshPropertyMapSettings;
	
	UE_API EBakeOpState UpdateResult_Height(const FImageDimensions& Dimensions);
	FHeightMapSettings CachedHeightMapSettings;

	UE_API EBakeOpState UpdateResult_Texture2DImage(const FImageDimensions& Dimensions, const FDynamicMesh3* DetailMesh);
	FTexture2DSettings CachedTexture2DSettings;
	TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> CachedTextureImage;

	UE_API EBakeOpState UpdateResult_MultiTexture(const FImageDimensions& Dimensions, const FDynamicMesh3* DetailMesh);
	FTexture2DSettings CachedMultiTexture2DSettings;
	TArray<TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>> CachedMultiTextures;

protected:
	//
	// Utilities
	//

	/**
	 * Given an array of textures associated with a material,
	 * use heuristics to identify the color/albedo texture.
	 * @param Textures array of textures associated with a material.
	 * @return integer index into the Textures array representing the color/albedo texture
	 */
	static UE_API int SelectColorTextureToBake(const TArray<UTexture*>& Textures);

	/**
	 * Iterate through a primitive component's textures by material ID.
	 * @param Component the component to query
	 * @param ProcessFunc enumeration function with signature: void(int NumMaterials, int MaterialID, const TArray<UTexture*>& Textures)
	 */
	template <typename ProcessFn>
	static void ProcessComponentTextures(const UPrimitiveComponent* Component, ProcessFn&& ProcessFunc);

	/**
	 * Find all source textures and material IDs for a given target.
	 * @param Target the tool target to inspect
	 * @param AllSourceTextures the output array of all textures associated with the target.
	 * @param MaterialIDTextures the output array of material IDs and best guess textures for those IDs on the target.
	 */
	static UE_API void UpdateMultiTextureMaterialIDs(
		UToolTarget* Target,
		TArray<TObjectPtr<UTexture2D>>& AllSourceTextures,
		TArray<TObjectPtr<UTexture2D>>& MaterialIDTextures);
};


template <typename ProcessFn>
void UBakeMeshAttributeTool::ProcessComponentTextures(const UPrimitiveComponent* Component, ProcessFn&& ProcessFunc)
{
	if (!Component)
	{
		return;
	}

	TArray<UMaterialInterface*> Materials;
	Component->GetUsedMaterials(Materials);

	const int32 NumMaterials = Materials.Num();
	for (int32 MaterialID = 0; MaterialID < NumMaterials; ++MaterialID)	// TODO: This won't match MaterialIDs on the FDynamicMesh3 in general, will it?
	{
		UMaterialInterface* MaterialInterface = Materials[MaterialID];
		TArray<UTexture*> Textures;
		if (MaterialInterface)
		{
			MaterialInterface->GetUsedTextures(Textures);
		}
		ProcessFunc(NumMaterials, MaterialID, Textures);
	}
}


#undef UE_API
