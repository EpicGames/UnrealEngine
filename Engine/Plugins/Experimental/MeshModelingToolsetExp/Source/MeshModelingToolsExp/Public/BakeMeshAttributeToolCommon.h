// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "MultiSelectionTool.h"
#include "Image/ImageBuilder.h"
#include "Image/ImageDimensions.h"
#include "BakeMeshAttributeToolCommon.generated.h"

// Pre-declarations
using UE::Geometry::FImageDimensions;
using UE::Geometry::TImageBuilder;
class UStaticMesh;
class USkeletalMesh;


/**
 * Bake tool property sets
 */


UENUM()
enum class EBakeNormalSpace
{
	/** Tangent space */
	Tangent,
	/** Object space */
	Object
};


UCLASS(MinimalAPI)
class UBakeInputMeshProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetStaticMesh != nullptr", EditConditionHides))
	TObjectPtr<UStaticMesh> TargetStaticMesh = nullptr;

	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetSkeletalMesh != nullptr", EditConditionHides))
	TObjectPtr<USkeletalMesh> TargetSkeletalMesh = nullptr;

	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetDynamicMesh != nullptr", EditConditionHides))
	TObjectPtr<AActor> TargetDynamicMesh = nullptr;

	/** UV channel to use for the target mesh */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (DisplayName = "Target Mesh UV Channel",
		GetOptions = GetTargetUVLayerNamesFunc, TransientToolProperty, NoResetToDefault,
		EditCondition = "bHasTargetUVLayer == true", EditConditionHides, HideEditConditionToggle))
	FString TargetUVLayer;

	/** If true, expose the TargetUVLayer property */ 
	UPROPERTY()
	bool bHasTargetUVLayer = false;

	/** Source mesh to sample from */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Source Mesh", meta = (TransientToolProperty,
		EditCondition = "SourceStaticMesh != nullptr", EditConditionHides))
	TObjectPtr<UStaticMesh> SourceStaticMesh = nullptr;

	/** Source mesh to sample from */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Source Mesh", meta = (TransientToolProperty,
		EditCondition = "SourceSkeletalMesh != nullptr", EditConditionHides))
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh = nullptr;

	/** Source mesh to sample from */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Source Mesh", meta = (TransientToolProperty,
		EditCondition = "SourceDynamicMesh != nullptr", EditConditionHides))
	TObjectPtr<AActor> SourceDynamicMesh = nullptr;

	/** If true, hide the source mesh */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (EditConditionHides, HideEditConditionToggle,
		EditCondition = "SourceStaticMesh != nullptr || SourceSkeletalMesh != nullptr || SourceDynamicMesh != nullptr"))
	bool bHideSourceMesh = false;

	/** Source mesh normal map; if empty, the geometric normals will be used */
	UPROPERTY(EditAnywhere, Category = BakeInput, AdvancedDisplay, meta = (TransientToolProperty,
		EditCondition = "bHasSourceNormalMap == true", EditConditionHides, HideEditConditionToggle))
	TObjectPtr<UTexture2D> SourceNormalMap = nullptr;

	/** UV channel to use for the source mesh normal map; only relevant if a source normal map is set */
	UPROPERTY(EditAnywhere, Category = BakeInput, AdvancedDisplay, meta = (DisplayName = "Source Normal UV Channel",
		GetOptions = GetSourceUVLayerNamesFunc, TransientToolProperty, NoResetToDefault,
		EditCondition = "bHasSourceNormalMap == true", EditConditionHides, HideEditConditionToggle))
	FString SourceNormalMapUVLayer;

	/** Normal space of the source mesh normal map. */
	UPROPERTY(EditAnywhere, Category = BakeInput, AdvancedDisplay, meta = (TransientToolProperty,
		EditCondition = "bHasSourceNormalMap == true", EditConditionHides, HideEditConditionToggle))
	EBakeNormalSpace SourceNormalSpace = EBakeNormalSpace::Tangent;

	/** If true, expose the SourceNormalMap and SourceNormalMapUVLayer properties */
	UPROPERTY()
	bool bHasSourceNormalMap = false;

	/** Maximum allowed distance for the projection from target mesh to source mesh for the sample to be considered valid.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (ClampMin = "0.001",
		EditCondition = "SourceStaticMesh != nullptr || SourceSkeletalMesh != nullptr || SourceDynamicMesh != nullptr", HideEditConditionToggle))
	float ProjectionDistance = 3.0;

	/** If true, uses the world space positions for the projection from target mesh to source mesh, otherwise it uses their object space positions.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (
		EditCondition = "SourceStaticMesh != nullptr || SourceSkeletalMesh != nullptr || SourceDynamicMesh != nullptr", HideEditConditionToggle))
	bool bProjectionInWorldSpace = false;

	UFUNCTION()
	const TArray<FString>& GetTargetUVLayerNamesFunc() const
	{
		return TargetUVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> TargetUVLayerNamesList;

	UFUNCTION()
	const TArray<FString>& GetSourceUVLayerNamesFunc() const
	{
		return SourceUVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> SourceUVLayerNamesList;
};


UCLASS(MinimalAPI)
class UBakeNormalMapToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

};


UCLASS(MinimalAPI)
class UBakeOcclusionMapToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Number of occlusion rays per sample */
	UPROPERTY(EditAnywhere, Category = OcclusionOutput, meta = (UIMin = "1", UIMax = "1024", ClampMin = "1", ClampMax = "65536"))
	int32 OcclusionRays = 16;

	/** Maximum distance for occlusion rays to test for intersections; a value of 0 means infinity */
	UPROPERTY(EditAnywhere, Category = OcclusionOutput, meta = (UIMin = "0.0", UIMax = "1000.0", ClampMin = "0.0", ClampMax = "99999999.0"))
	float MaxDistance = 0.0f;

	/** Maximum spread angle in degrees for occlusion rays; for example, 180 degrees will cover the entire hemisphere whereas 90 degrees will only cover the center of the hemisphere down to 45 degrees from the horizon. */
	UPROPERTY(EditAnywhere, Category = OcclusionOutput, meta = (UIMin = "0", UIMax = "180.0", ClampMin = "0", ClampMax = "180.0"))
	float SpreadAngle = 180.0f;

	/** Angle in degrees from the horizon for occlusion rays for which the contribution is attenuated to reduce faceting artifacts. */
	UPROPERTY(EditAnywhere, Category = OcclusionOutput, meta = (UIMin = "0", UIMax = "45.0", ClampMin = "0", ClampMax = "89.9"))
	float BiasAngle = 15.0f;

	/** Normal space for Bent Normal bakes. */
	UPROPERTY(EditAnywhere, Category = OcclusionOutput)
	EBakeNormalSpace NormalSpace = EBakeNormalSpace::Tangent;
};


UENUM()
enum class EBakeCurvatureTypeMode
{
	/** Average of the minimum and maximum principal curvatures */
	MeanAverage UMETA(DisplayName = "Mean"),
	/** Maximum principal curvature */
	Max,
	/** Minimum principal curvature */
	Min,
	/** Product of the minimum and maximum principal curvatures */
	Gaussian
};


UENUM()
enum class EBakeCurvatureColorMode
{
	/** Map curvature values to grayscale such that black is negative, grey is zero, and white is positive */
	Grayscale,
	/** Map curvature values to red and blue such that red is negative, black is zero, and blue is positive */
	RedBlue,
	/** Map curvature values to red, green, blue such that red is negative, green is zero, and blue is positive */
	RedGreenBlue
};


UENUM()
enum class EBakeCurvatureClampMode
{
	/** Include both negative and positive curvatures */
	None,
	/** Clamp negative curvatures to zero */
	OnlyPositive,
	/** Clamp positive curvatures to zero */
	OnlyNegative
};


UCLASS(MinimalAPI)
class UBakeCurvatureMapToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Type of curvature */
	UPROPERTY(EditAnywhere, Category = CurvatureOutput)
	EBakeCurvatureTypeMode CurvatureType = EBakeCurvatureTypeMode::MeanAverage;

	/** How to map calculated curvature values to colors */
	UPROPERTY(EditAnywhere, Category = CurvatureOutput)
	EBakeCurvatureColorMode ColorMapping = EBakeCurvatureColorMode::Grayscale;

	/** Multiplier for how the curvature values fill the available range in the selected Color Mapping; a larger value means that higher curvature is required to achieve the maximum color value. */
	UPROPERTY(EditAnywhere, Category = CurvatureOutput, meta = (UIMin = "0.1", UIMax = "2.0", ClampMin = "0.001", ClampMax = "100.0"))
	float ColorRangeMultiplier = 1.0;

	/** Minimum for the curvature values to not be clamped to zero relative to the curvature for the maximum color value; a larger value means that higher curvature is required to not be considered as no curvature. */
	UPROPERTY(EditAnywhere, Category = CurvatureOutput, AdvancedDisplay, meta = (DisplayName = "Color Range Minimum", UIMin = "0.0", UIMax = "1.0"))
	float MinRangeMultiplier = 0.0;

	/** Clamping applied to curvature values before color mapping */
	UPROPERTY(EditAnywhere, Category = CurvatureOutput)
	EBakeCurvatureClampMode Clamping = EBakeCurvatureClampMode::None;
};


UCLASS(MinimalAPI)
class UBakeUVShellMapToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The source mesh UV Layer to sample. */
	UPROPERTY(EditAnywhere, Category = "UV Shell Output")
	int UVLayer = 0;

	/** The thickness of the wireframe in pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Shell Output")
	float WireframeThickness = 1.0f;

	/** The color of wireframe pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Shell Output")
	FLinearColor WireframeColor = FLinearColor::Blue;

	/** The color of the UV shell interior pixels. */
	UPROPERTY(EditAnywhere, Category = "UV Shell Output")
	FLinearColor ShellColor = FLinearColor::Gray;

	/** The color of pixels external to UV shells. */
	UPROPERTY(EditAnywhere, Category = "UV Shell Output")
	FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
};


UENUM()
enum class EBakeHeightRangeMode
{
	/** Height range defined in absolute units in object space */
	Absolute,
	/** Height range defined as a ratio of the maximal bounding box axis */
	RelativeBounds
};


UCLASS(MinimalAPI)
class UBakeHeightMapToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The coordinate space to compute height */
	UPROPERTY(EditAnywhere, Category = HeightOutput)
	EBakeHeightRangeMode HeightRangeMode = EBakeHeightRangeMode::Absolute;

	/** The inner height map distance from the low poly mesh, in object space units. This value maps to black. */
	UPROPERTY(EditAnywhere, Category = HeightOutput, meta=(EditConditionHides,EditCondition="HeightRangeMode == EBakeHeightRangeMode::Absolute", UIMin=-100.0f, UIMax=100.0f))
	float InnerDistance = -1.0f;

	/** The outer height map distance from the low poly mesh, in object space units. This value maps to white. */
	UPROPERTY(EditAnywhere, Category = HeightOutput, meta=(EditConditionHides,EditCondition="HeightRangeMode == EBakeHeightRangeMode::Absolute", UIMin=-100.0f, UIMax=100.0f))
	float OuterDistance = 1.0f;

	/** The inner height map distance from the low poly mesh, as a ratio of the maximum bounding box axis. This value maps to black. */
	UPROPERTY(EditAnywhere, Category = HeightOutput, meta=(EditConditionHides,EditCondition="HeightRangeMode == EBakeHeightRangeMode::RelativeBounds", ClampMin=-1.0f, ClampMax=1.0f))
	float InnerBoundsDistance = -0.1f;

	/** The outer height map distance from the low poly mesh, as a ratio of the maximum bounding box axis. This value maps to white. */
	UPROPERTY(EditAnywhere, Category = HeightOutput, meta=(EditConditionHides,EditCondition="HeightRangeMode == EBakeHeightRangeMode::RelativeBounds", ClampMin=-1.0f, ClampMax=1.0f))
	float OuterBoundsDistance = 0.1f;
};


UCLASS(MinimalAPI)
class UBakeTexture2DProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Source mesh texture that is to be resampled into a new texture */
	UPROPERTY(EditAnywhere, Category = TextureOutput, meta = (TransientToolProperty))
	TObjectPtr<UTexture2D> SourceTexture;

	/** UV channel to use for the source mesh texture */
	UPROPERTY(EditAnywhere, Category = TextureOutput, meta = (DisplayName = "Source Texture UV Channel",
		GetOptions = GetUVLayerNamesFunc, NoResetToDefault))
	FString UVLayer;

	UFUNCTION()
	const TArray<FString>& GetUVLayerNamesFunc() const
	{
		return UVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVLayerNamesList;
};


UCLASS(MinimalAPI)
class UBakeMultiTexture2DProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** For each material ID, the source texture that will be resampled in that material's region*/
	UPROPERTY(EditAnywhere, EditFixedSize, Category = MultiTextureOutput, meta = (DisplayName = "Material IDs",
		TransientToolProperty, EditFixedOrder))
	TArray<TObjectPtr<UTexture2D>> MaterialIDSourceTextures;

	/** UV channel to use for the source mesh texture */
	UPROPERTY(EditAnywhere, Category = MultiTextureOutput, meta = (DisplayName = "Source Texture UV Channel",
		GetOptions = GetUVLayerNamesFunc, NoResetToDefault))
	FString UVLayer;

	UFUNCTION()
	const TArray<FString>& GetUVLayerNamesFunc() const
	{
		return UVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVLayerNamesList;

	/** The set of all source textures from all input materials */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = MultiTextureOutput, meta = (DisplayName = "Source Textures",
		TransientToolProperty))
	TArray<TObjectPtr<UTexture2D>> AllSourceTextures;

};


UCLASS(MinimalAPI)
class UBakeVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Preview the texture as an input to the respective material parameter (ex. Normal, Ambient Occlusion) */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bPreviewAsMaterial = false;
	
	/** Adjust the brightness of the preview material; does not affect results stored in textures */
	UPROPERTY(EditAnywhere, Category = Preview, meta = (DisplayName = "Brightness", UIMin = "0.0", UIMax = "1.0"))
	float Brightness = 1.0f;

	/** Ambient Occlusion multiplier in the viewport; does not affect results stored in textures */
	UPROPERTY(EditAnywhere, Category = Preview, meta = (DisplayName = "AO Multiplier", UIMin = "0.0", UIMax = "1.0",
		ClampMin = "0.0", ClampMax = "1.0"))
	float AOMultiplier = 1.0f;
};


/**
 * Bake tool property settings structs
 */

struct FDetailMeshSettings
{
	int32 UVLayer = 0;
	EBakeNormalSpace NormalSpace = EBakeNormalSpace::Tangent;
	
	bool operator==(const FDetailMeshSettings& Other) const
	{
		return UVLayer == Other.UVLayer && NormalSpace == Other.NormalSpace;
	}
};

struct FNormalMapSettings
{
	FImageDimensions Dimensions;

	bool operator==(const FNormalMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions;
	}
};

struct FOcclusionMapSettings
{
	FImageDimensions Dimensions;
	int32 OcclusionRays;
	float MaxDistance;
	float SpreadAngle;
	float BiasAngle;
	EBakeNormalSpace NormalSpace = EBakeNormalSpace::Tangent;

	bool operator==(const FOcclusionMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions &&
			OcclusionRays == Other.OcclusionRays &&
			MaxDistance == Other.MaxDistance &&
			SpreadAngle == Other.SpreadAngle &&
			BiasAngle == Other.BiasAngle &&
			NormalSpace == Other.NormalSpace;
	}
};

struct FCurvatureMapSettings
{
	FImageDimensions Dimensions;
	int32 CurvatureType = 0;
	float RangeMultiplier = 1.0f;
	float MinRangeMultiplier = 0.0f;
	int32 ColorMode = 0;
	int32 ClampMode = 0;

	bool operator==(const FCurvatureMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions && CurvatureType == Other.CurvatureType && RangeMultiplier == Other.RangeMultiplier && MinRangeMultiplier == Other.MinRangeMultiplier && ColorMode == Other.ColorMode && ClampMode == Other.ClampMode;
	}
};

struct FMeshPropertyMapSettings
{
	FImageDimensions Dimensions;

	bool operator==(const FMeshPropertyMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions;
	}
};

struct FHeightMapSettings
{
	FImageDimensions Dimensions;
	int32 HeightRangeMode = 0;
	float InnerDistance = -1.0f;
	float OuterDistance = 1.0f;
	float InnerBoundsDistance = -0.1f;
	float OuterBoundsDistance = 0.1f;

	bool operator==(const FHeightMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions
		&& HeightRangeMode == Other.HeightRangeMode
		&& InnerDistance == Other.InnerDistance
		&& OuterDistance == Other.OuterDistance
		&& InnerBoundsDistance == Other.InnerBoundsDistance
		&& OuterBoundsDistance == Other.OuterBoundsDistance;
	}
};

struct FUVShellMapSettings
{
	FImageDimensions Dimensions;
	int UVLayer = 0;
	float WireframeThickness = 1.0f;
	FLinearColor WireframeColor = FLinearColor::Blue;
	FLinearColor ShellColor = FLinearColor::Gray;
	FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	bool operator==(const FUVShellMapSettings& Other) const
	{
		return Dimensions == Other.Dimensions &&
			UVLayer == Other.UVLayer &&
			WireframeThickness == Other.WireframeThickness &&
			WireframeColor == Other.WireframeColor &&
			ShellColor == Other.ShellColor &&
			BackgroundColor == Other.BackgroundColor;
	}
};

struct FTexture2DSettings
{
	FImageDimensions Dimensions;
	int32 UVLayer = 0;

	bool operator==(const FTexture2DSettings& Other) const
	{
		return Dimensions == Other.Dimensions && UVLayer == Other.UVLayer;
	}
};


/**
 * Bake compute state
 */
enum class EBakeOpState
{
	Clean              = 0,      // No-op - evaluation already launched/complete.
	Evaluate           = 1 << 0, // Inputs are modified and valid, re-evaluate.
	EvaluateDetailMesh = 1 << 1, // Detail mesh input is modified, re-evaluate the detail mesh.
	Invalid            = 1 << 2  // Inputs are modified and invalid - retry eval until valid.
};
ENUM_CLASS_FLAGS(EBakeOpState);


