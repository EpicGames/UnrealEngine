// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MultiSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "PreviewMesh.h"
#include "ModelingToolTargetUtil.h"
#include "Sampling/MeshVertexBaker.h"
#include "BakeMeshAttributeTool.h"
#include "BakeMeshAttributeVertexTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

// predeclarations
class UMaterialInstanceDynamic;


/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UBakeMeshAttributeVertexToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


UENUM()
enum class EBakeVertexTopology
{
	/* Generate new vertex color topology, optionally using normal and UV seams to determine splits */
	CreateNew,
	/* Use existing vertex color topology on the Target Mesh */
	UseExisting
};


UENUM()
enum class EBakeVertexOutput
{
	/* Bake vertex data to RGBA */
	RGBA,
	/* Bake vertex data to individual color channels */
	PerChannel
};


UENUM()
enum class EBakeVertexChannel
{
	R,
	G,
	B,
	A,
	RGBA
};


UCLASS(MinimalAPI)
class UBakeMeshAttributeVertexToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** The bake output mode */
	UPROPERTY(EditAnywhere, Category = BakeOutput)
	EBakeVertexOutput OutputMode = EBakeVertexOutput::RGBA;

	/** The source data to bake to vertex colors */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		DisplayName="Source Type",
		ValidEnumValues="TangentSpaceNormal, AmbientOcclusion, BentNormal, Curvature, Texture, ObjectSpaceNormal, FaceNormal, Position, MaterialID, PolyGroupID, MultiTexture, VertexColor, Height",
		EditCondition="OutputMode == EBakeVertexOutput::RGBA", EditConditionHides))
	int32 OutputType = static_cast<int32>(EBakeMapType::TangentSpaceNormal);

	/** The source data to bake to the Red channel */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		DisplayName="Source Type R",
		ValidEnumValues="None, AmbientOcclusion, Curvature, Height, One, Zero",
		EditCondition="OutputMode == EBakeVertexOutput::PerChannel", EditConditionHides))
	int32 OutputTypeR = static_cast<int32>(EBakeMapType::None);

	/** The source data to bake to the Green channel */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		DisplayName="Source Type G",
		ValidEnumValues="None, AmbientOcclusion, Curvature, Height, One, Zero",
		EditCondition="OutputMode == EBakeVertexOutput::PerChannel", EditConditionHides))
	int32 OutputTypeG = static_cast<int32>(EBakeMapType::None);

	/** The source data to bake to the Blue channel */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		DisplayName="Source Type B",
		ValidEnumValues="None, AmbientOcclusion, Curvature, Height, One, Zero",
		EditCondition="OutputMode == EBakeVertexOutput::PerChannel", EditConditionHides))
	int32 OutputTypeB = static_cast<int32>(EBakeMapType::None);

	/** The source data to bake to the Alpha channel */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(Bitmask, BitmaskEnum = "/Script/MeshModelingToolsExp.EBakeMapType",
		DisplayName="Source Type A",
		ValidEnumValues="None, AmbientOcclusion, Curvature, Height, One, Zero",
		EditCondition="OutputMode == EBakeVertexOutput::PerChannel", EditConditionHides))
	int32 OutputTypeA = static_cast<int32>(EBakeMapType::None);

	/** The vertex color channel to preview */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (TransientToolProperty))
	EBakeVertexChannel PreviewMode = EBakeVertexChannel::RGBA;

	/** The vertex color topology to use for the bake */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta = (TransientToolProperty))
	EBakeVertexTopology TopologyMode = EBakeVertexTopology::CreateNew;

	/** If true, compute a separate vertex color for each unique normal on a vertex */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = BakeOutput, meta=(EditCondition="TopologyMode == EBakeVertexTopology::CreateNew"))
	bool bSplitAtNormalSeams = false;

	/** If true, Compute a separate vertex color for each unique UV on a vertex. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = BakeOutput, meta=(DisplayName = "Split at UV Seams", EditCondition="TopologyMode == EBakeVertexTopology::CreateNew"))
	bool bSplitAtUVSeams = false;
};


/**
 * Vertex Baking Tool
 */
UCLASS(MinimalAPI)
class UBakeMeshAttributeVertexTool : public UBakeMeshAttributeTool, public UE::Geometry::IGenericDataOperatorFactory<UE::Geometry::FMeshVertexBaker>
{
	GENERATED_BODY()

public:
	UBakeMeshAttributeVertexTool() = default;

	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	// End UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshVertexBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

protected:
	UPROPERTY()
	TObjectPtr<UBakeInputMeshProperties> InputMeshSettings;
	
	UPROPERTY()
	TObjectPtr<UBakeMeshAttributeVertexToolProperties> Settings;

protected:
	friend class FMeshVertexBakerOp;
	
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewAlphaMaterial;

	TUniquePtr<TGenericDataBackgroundCompute<UE::Geometry::FMeshVertexBaker>> Compute = nullptr;
	UE_API void OnResultUpdated(const TUniquePtr<UE::Geometry::FMeshVertexBaker>& NewResult);

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> DetailMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> DetailSpatial;
	int32 DetailMeshTimestamp = 0;
	UE_API void UpdateDetailMesh();

	int32 NumColorElements = 0;
	bool bColorTopologyValid = false;
	bool bIsBakeToSelf = false;
	UE_API void UpdateOnModeChange();
	UE_API void UpdateVisualization();
	UE_API void UpdateColorTopology();
	UE_API void UpdateSourceVertexColors();
	UE_API void UpdateResult();

	struct FBakeSettings
	{
		EBakeVertexOutput OutputMode = EBakeVertexOutput::RGBA;
		EBakeMapType OutputType = EBakeMapType::TangentSpaceNormal;
		EBakeMapType OutputTypePerChannel[4] = { EBakeMapType::None, EBakeMapType::None, EBakeMapType::None, EBakeMapType::None };
		EBakeVertexChannel PreviewMode = EBakeVertexChannel::RGBA;
		EBakeVertexTopology TopologyMode = EBakeVertexTopology::CreateNew;
		float ProjectionDistance = 3.0;
		bool bProjectionInWorldSpace = false;
		bool bSplitAtNormalSeams = false;
		bool bSplitAtUVSeams = false;

		bool operator==(const FBakeSettings& Other) const
		{
			return (OutputMode == Other.OutputMode &&
				OutputType == Other.OutputType &&
				OutputTypePerChannel[0] == Other.OutputTypePerChannel[0] &&
				OutputTypePerChannel[1] == Other.OutputTypePerChannel[1] &&
				OutputTypePerChannel[2] == Other.OutputTypePerChannel[2] &&
				OutputTypePerChannel[3] == Other.OutputTypePerChannel[3] &&
				TopologyMode == Other.TopologyMode &&
				bProjectionInWorldSpace == Other.bProjectionInWorldSpace &&
				ProjectionDistance == Other.ProjectionDistance &&
				bSplitAtNormalSeams == Other.bSplitAtNormalSeams &&
				bSplitAtUVSeams == Other.bSplitAtUVSeams);
		}
	};
	FBakeSettings CachedBakeSettings;

	void SetSourceObjectVisible(bool bState)
	{
		if (!bIsBakeToSelf)
		{
			UE::ToolTarget::SetSourceObjectVisible(Targets[1], bState);
		}
	}


	//
	// Analytics
	//
	struct FBakeAnalytics
	{
		double TotalBakeDuration = 0.0;

		struct FMeshSettings
		{
			int32 NumTargetMeshVerts = 0;
			int32 NumTargetMeshTris = 0;
			int32 NumDetailMesh = 0;
			int64 NumDetailMeshTris = 0;
		};
		FMeshSettings MeshSettings;

		FBakeSettings BakeSettings;
		FOcclusionMapSettings OcclusionSettings;
		FCurvatureMapSettings CurvatureSettings;
	};
	FBakeAnalytics BakeAnalytics;

	/**
	 * Computes the NumTargetMeshTris, NumDetailMesh and NumDetailMeshTris analytics.
	 * @param Data the mesh analytics data to compute
	 */
	UE_API virtual void GatherAnalytics(FBakeAnalytics::FMeshSettings& Data);

	/**
	 * Records bake timing and settings data for analytics.
	 * @param Result the result of the bake.
	 * @param Settings The bake settings used for the bake.
	 * @param Data the output bake analytics struct.
	 */
	static UE_API void GatherAnalytics(const UE::Geometry::FMeshVertexBaker& Result,
								const FBakeSettings& Settings,
								FBakeAnalytics& Data);

	/**
	 * Outputs an analytics event using the given analytics struct.
	 * @param Data the bake analytics struct to output.
	 * @param EventName the name of the analytics event to output.
	 */
	static UE_API void RecordAnalytics(const FBakeAnalytics& Data, const FString& EventName);
};

#undef UE_API
