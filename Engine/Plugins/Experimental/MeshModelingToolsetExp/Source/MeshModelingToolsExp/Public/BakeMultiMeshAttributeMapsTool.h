// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MultiSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageBuilder.h"
#include "Sampling/MeshMapBaker.h"
#include "Scene/MeshSceneAdapter.h"
#include "ModelingOperators.h"
#include "PreviewMesh.h"
#include "Baking/BakingTypes.h"
#include "BakeMeshAttributeMapsToolBase.h"
#include "BakeMultiMeshAttributeMapsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UBakeMultiMeshAttributeMapsToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UCLASS(MinimalAPI)
class UBakeMultiMeshAttributeMapsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** The map types to generate */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(DisplayName="Output Types", Bitmask, BitmaskEnum= "/Script/MeshModelingToolsExp.EBakeMapType",
		ValidEnumValues="TangentSpaceNormal, ObjectSpaceNormal, Position, Texture"))
	int32 MapTypes = (int32) EBakeMapType::None;

	/** The map type index to preview */
	UPROPERTY(EditAnywhere, Category = BakeOutput, meta=(DisplayName="Preview Output Type", TransientToolProperty, GetOptions = GetMapPreviewNamesFunc,
		EditCondition = "MapTypes != 0", NoResetToDefault))
	FString MapPreview;

	/** The pixel resolution of the generated map */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureResolution Resolution = EBakeTextureResolution::Resolution256;

	/** The channel bit depth of the source data for the generated textures */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureBitDepth BitDepth = EBakeTextureBitDepth::ChannelBits8;

	/** Number of samples per pixel */
	UPROPERTY(EditAnywhere, Category = Textures)
	EBakeTextureSamplesPerPixel SamplesPerPixel = EBakeTextureSamplesPerPixel::Sample1;

	/** Mask texture for filtering out samples/pixels from the output texture */
	UPROPERTY(EditAnywhere, Category = Textures, AdvancedDisplay)
	TObjectPtr<UTexture2D> SampleFilterMask = nullptr;

	UFUNCTION()
	const TArray<FString>& GetMapPreviewNamesFunc()
	{
		return MapPreviewNamesList;
	}
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> MapPreviewNamesList;
	TMap<FString, FString> MapPreviewNamesMap;
};


USTRUCT()
struct FBakeMultiMeshDetailProperties
{
	GENERATED_BODY()

	/** Source mesh to sample from */
	UPROPERTY(VisibleAnywhere, Category = BakeSources, meta = (TransientToolProperty))
	TObjectPtr<UStaticMesh> SourceMesh = nullptr;

	/** Source mesh color texture that is to be resampled into a new texture */
	UPROPERTY(EditAnywhere, Category = BakeSources, meta = (TransientToolProperty,
		EditCondition="SourceMesh != nullptr"))
	TObjectPtr<UTexture2D> SourceTexture = nullptr;

	/** UV channel to use for the source mesh color texture */
	UPROPERTY(EditAnywhere, Category = BakeSources, meta = (TransientToolProperty, DisplayName = "Source Texture UV Channel",
		EditCondition="SourceTexture != nullptr", ClampMin=0, ClampMax=7))
	int32 SourceTextureUVLayer = 0;
};


UCLASS(MinimalAPI)
class UBakeMultiMeshInputToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetStaticMesh != nullptr", EditConditionHides))
	TObjectPtr<UStaticMesh> TargetStaticMesh = nullptr;

	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetSkeletalMesh != nullptr", EditConditionHides))
	TObjectPtr<USkeletalMesh> TargetSkeletalMesh = nullptr;

	/** Target mesh to sample to */
	UPROPERTY(VisibleAnywhere, Category = BakeInput, DisplayName = "Target Mesh", meta = (TransientToolProperty,
		EditCondition = "TargetDynamicMesh != nullptr", EditConditionHides))
	TObjectPtr<AActor> TargetDynamicMesh = nullptr;

	/** UV channel to use for the target mesh */
	UPROPERTY(EditAnywhere, Category = BakeInput, meta = (DisplayName = "Target Mesh UV Channel",
		GetOptions = GetTargetUVLayerNamesFunc, TransientToolProperty, NoResetToDefault))
	FString TargetUVLayer;

	/** Source meshes and textures to sample from */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = BakeInput, meta = (TransientToolProperty, EditFixedOrder, NoResetToDefault))
	TArray<FBakeMultiMeshDetailProperties> SourceMeshes;

	/** Maximum allowed distance for the projection from target mesh to source mesh for the sample to be considered valid.
	 * This is only relevant if a separate source mesh is provided. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = BakeInput, meta = (ClampMin = "0.001"))
	float ProjectionDistance = 3.0;

	UFUNCTION()
	const TArray<FString>& GetTargetUVLayerNamesFunc() const
	{
		return TargetUVLayerNamesList;
	}

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> TargetUVLayerNamesList;
};


struct FBakeMultiMeshDetailSettings
{
	using FColorMapData = TTuple<int32, bool>;
	TArray<FColorMapData> ColorMapData;
	
	bool operator==(const FBakeMultiMeshDetailSettings& Other) const
	{
		const int NumData = ColorMapData.Num();
		bool bIsEqual = Other.ColorMapData.Num() == NumData;
		for (int Idx = 0; bIsEqual && Idx < NumData; ++Idx)
		{
			bIsEqual = bIsEqual && ColorMapData[Idx] == Other.ColorMapData[Idx];
		}
		return bIsEqual;
	}
};


// TODO: Refactor shared code into common base class.
/**
 * N-to-1 Detail Map Baking Tool
 */
UCLASS(MinimalAPI)
class UBakeMultiMeshAttributeMapsTool : public UBakeMeshAttributeMapsToolBase
{
	GENERATED_BODY()

public:
	UBakeMultiMeshAttributeMapsTool() = default;

	// Begin UInteractiveTool interface
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	// End UInteractiveTool interface

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<UE::Geometry::FMeshMapBaker>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

protected:
	// need to update bResultValid if these are modified, so we don't publicly expose them. 
	// @todo setters/getters for these

	UPROPERTY()
	TObjectPtr<UBakeMultiMeshAttributeMapsToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UBakeMultiMeshInputToolProperties> InputMeshSettings;

	UPROPERTY()
	TObjectPtr<UBakeMeshAttributeMapsResultToolProperties> ResultSettings;


protected:
	// Begin UBakeMeshAttributeMapsToolBase interface
	UE_API virtual void UpdateResult() override;
	UE_API virtual void UpdateVisualization() override;

	UE_API virtual void GatherAnalytics(FBakeAnalytics::FMeshSettings& Data) override;
	// End UBakeMeshAttributeMapsToolBase interface
	

protected:
	friend class FMultiMeshMapBakerOp;

	TSharedPtr<UE::Geometry::FMeshSceneAdapter, ESPMode::ThreadSafe> DetailMeshScene;

	UE_API void UpdateOnModeChange();

	UE_API void InvalidateResults();

	// Cached detail mesh data
	FBakeMultiMeshDetailSettings CachedDetailSettings;
	UE_API EBakeOpState UpdateResult_DetailMeshes();
	
	using FTextureImageData = TTuple<UE::Geometry::TImageBuilder<FVector4f>*, int>;
	using FTextureImageMap = TMap<void*, UE::Geometry::IMeshBakerDetailSampler::FBakeDetailTexture>; 
	TArray<TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>>> CachedColorImages;
	TArray<int> CachedColorUVLayers;
	FTextureImageMap CachedMeshToColorImagesMap;

	// Analytics
	virtual FString GetAnalyticsEventName() const override
	{
		return TEXT("BakeAll");
	}
};

#undef UE_API
