// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UPreviewMesh;
class UToolTarget;
class AActor;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Updates a tool property set's UVLayerNamesList from the list of UV layers
 * on a given mesh. Also updates the UVLayer property if the current UV layer
 * is no longer available.
 *
 * @param UVLayer Selected UV Layer.
 * @param UVLayerNamesList List of available UV layers.
 * @param Mesh the mesh to query
 */
MESHMODELINGTOOLSEXP_API void UpdateUVLayerNames(FString& UVLayer, TArray<FString>& UVLayerNamesList, const FDynamicMesh3& Mesh);

MESHMODELINGTOOLSEXP_API UPreviewMesh* CreateBakePreviewMesh(UObject* Tool, UToolTarget* ToolTarget, UWorld* World);

/** @return AActor that owns a DynamicMeshComponent from a tool target, or nullptr if there is no such Actor */
MESHMODELINGTOOLSEXP_API AActor* GetTargetActorViaIPersistentDynamicMeshSource(UToolTarget* Target);


} // namespace Geometry
} // namespace UE
