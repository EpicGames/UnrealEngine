// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseTools/SingleClickTool.h"
#include "BakeTransformTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


// predeclarations
struct FMeshDescription;
class UDynamicMeshComponent;


/**
 *
 */
UCLASS(MinimalAPI)
class UBakeTransformToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class EBakeScaleMethod : uint8
{
	// bake all scale information, so the component has scale of 1 on all axes
	BakeFullScale,
	// bake the non-uniform scale, so the component has a uniform scale
	BakeNonuniformScale,
	// do not bake any scaling
	DoNotBakeScale
};


/**
 * Standard properties
 */
UCLASS(MinimalAPI)
class UBakeTransformToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API UBakeTransformToolProperties();

	/** If checked, the baked transform will be applied to all available LODs. Has no effect on selections without LODs. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bApplyToAllLODs = true;

	/** Bake rotation */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bBakeRotation = true;

	/** Bake scale */
	UPROPERTY(EditAnywhere, Category = Options)
	EBakeScaleMethod BakeScale = EBakeScaleMethod::BakeFullScale;

	/** Recenter pivot after baking transform */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bRecenterPivot = false;

	// This variable is used to enable/disable the "DoNotBakeScale" option in the BakeScale enum
	// It is marked as visible only so that the FBakeTransformToolDetails can read it; it will be hidden by that DetailCustomization
	UPROPERTY(VisibleAnywhere, Category = Options)
	bool bAllowNoScale = true;
};



/**
 * Simple tool to bake scene transform on meshes into the mesh assets
 */
UCLASS(MinimalAPI)
class UBakeTransformTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UBakeTransformTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<UBakeTransformToolProperties> BasicProperties;

protected:
	TArray<int> MapToFirstOccurrences;

	UE_API void UpdateAssets();
};

#undef UE_API
