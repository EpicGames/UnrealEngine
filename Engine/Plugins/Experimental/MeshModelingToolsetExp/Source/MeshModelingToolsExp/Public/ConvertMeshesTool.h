// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolExclusiveToolAPI
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "ConvertMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

/**
 *
 */
UCLASS(MinimalAPI)
class UConvertMeshesToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const;
};

/**
 * Standard properties of the Transfer operation
 */
UCLASS(MinimalAPI)
class UConvertMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bShowTransferMaterials == true", EditConditionHides, HideEditConditionToggle))
	bool bTransferMaterials = true;

	// control whether the transfer materials option is displayed
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowTransferMaterials = true;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bTransferCollision = true;
};



UCLASS(MinimalAPI)
class UConvertMeshesTool : public UInteractiveTool,
	// Disallow auto-accept switch-away for the tool
	public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UPROPERTY()
	TObjectPtr<UConvertMeshesToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	void InitializeInputs(TArray<TWeakObjectPtr<UPrimitiveComponent>>&& InInputs)
	{
		Inputs = MoveTemp(InInputs);
	}

	void SetTargetLOD(EMeshLODIdentifier LOD)
	{
		TargetLOD = LOD;
	}

private:
 	UPROPERTY()
	TArray<TWeakObjectPtr<UPrimitiveComponent>> Inputs;

	EMeshLODIdentifier TargetLOD = EMeshLODIdentifier::Default;
};

#undef UE_API
