// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BoxTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "FrameTypes.h"
#include "GeometryBase.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolNestedAcceptCancelAPI, IInteractiveToolCameraFocusAPI
#include "Mechanics/CubeGrid.h"
#include "ModelingOperators.h"
#include "OrientedBoxTypes.h"
#include "Spatial/GeometrySet3.h"
#include "ToolContextInterfaces.h" // FViewCameraState
#include "ToolDataVisualizer.h"

#include "CubeGridTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(class FDynamicMeshChange);
PREDECLARE_GEOMETRY(class FCubeGrid);

class IAssetGenerationAPI;
class UClickDragInputBehavior;
class UCubeGridTool;
class UCreateMeshObjectTypeProperties;
class UDragAlignmentMechanic;
class ULocalClickDragInputBehavior;
class ULocalSingleClickInputBehavior;
class UMeshOpPreviewWithBackgroundCompute;
class UMouseHoverBehavior;
class UNewMeshMaterialProperties;
class UPreviewGeometry;
class UCombinedTransformGizmo;
class UTransformProxy;
class UToolTarget;


UCLASS(MinimalAPI)
class UCubeGridToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	IAssetGenerationAPI* AssetAPI = nullptr;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

protected:

	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class ECubeGridToolFaceSelectionMode
{
	/** Use hit normal to pick the outer face of the containing cell. */
	OutsideBasedOnNormal,
	/** Use hit normal to pierce backward through the geometry to pick an inside face of the containing cell. */
	InsideBasedOnNormal,
	/** Use view ray to pick the outer face of the containing cell. */
	OutsideBasedOnViewRay,
	/** Use view ray to pierce backward through the geometry to pick an inside face of the containing cell. */
	InsideBasedOnViewRay
};


UCLASS(MinimalAPI)
class UCubeGridToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "bAllowedToEditGrid", HideEditConditionToggle))
	FVector GridFrameOrigin = FVector(0, 0, 0);

	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "bAllowedToEditGrid", HideEditConditionToggle,
		UIMin = -180, UIMax = 180, ClampMin = -180000, ClampMax = 18000))
	FRotator GridFrameOrientation = FRotator(0, 0, 0);

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowGrid = true;

	UPROPERTY(EditAnywhere, Category = Options, meta = (TransientToolProperty))
	bool bShowGizmo = false;

	// These are here so that we can reset the appropriate settings to their defaults in code.
	const uint8 DEFAULT_GRID_POWER = 5;
	const double DEFAULT_CURRENT_BLOCK_SIZE = 100;

	//~ Unfortunately it seems that there isn't a way to make the automatic tooltip
	//~ here platform-specific, so we can't be specific about the hotkeys
	/** Determines cube grid scale. Can also be adjusted with hotkeys. This changes Current
	 Block Size to match the current Base Block Size and Grid Power (differently depending
	 on the Power of Two setting). */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "bAllowedToEditGrid", HideEditConditionToggle,
		UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "31"))
	uint8 GridPower = DEFAULT_GRID_POWER;

	/** 
	 * Sets the size of a block at the current grid power. This changes Base Block Size
	 * such that the given value is achieved at the the current value of Grid Power.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "bAllowedToEditGrid", HideEditConditionToggle,
		UIMin = "1", UIMax = "1000", ClampMin = "0.001"))
	double CurrentBlockSize = DEFAULT_CURRENT_BLOCK_SIZE;

	/** How many blocks each push/pull invocation will do at a time.*/
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		UIMin = "0", ClampMin = "0", Delta = 1, LinearDeltaSensitivity = 50))
	int32 BlocksPerStep = 1;

	/** 
	 * When true, block sizes change by powers of two as Grid Power is changed. When false, block
	 * sizes change by twos and fives, much like the default editor grid snapping options (for
	 * instance, sizes might increase from 10 to 50 to 100 to 500).
	 * Note that toggling this option will reset Grid Power and Current Block Size to default values.
	 */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	bool bPowerOfTwoBlockSizes = true;

	// Must match ClampMax in GridPower, used to make hotkeys not exceed it.
	const uint8 MaxGridPower = 31;

	/** Smallest block size to use in the grid, i.e. the block size at Grid Power 0. This
	 changes Current Block Size according to current value of Grid Power. */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (
		EditCondition = "bAllowedToEditGrid", HideEditConditionToggle,
		UIMin = "0.1", UIMax = "10", ClampMin = "0.001", ClampMax = "1000"))
	double BlockBaseSize = 3.125;

	/** When pushing/pulling in a way where the diagonal matters, setting this to true
	 makes the diagonal generally try to lie flat across the face rather than at
	 an incline. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "bInCornerMode", HideEditConditionToggle))
	bool bCrosswiseDiagonal = false;

	/** When performing multiple push/pulls with the same selection, attempt to keep the
	 same group IDs on the sides of the new geometry (ie multiple E/Q presses will not
	 result in different group topology around the sides compared to a single Ctrl+drag). */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	bool bKeepSideGroups = true;

	/** When true, displays dimensions of the given selection in the viewport. */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	bool bShowSelectionMeasurements = true;

	/** When performing selection, the tolerance to use when determining
	 whether things lie in the same plane as a cube face. */
	//~ This turned out to not be a useful setting, so it is no longer EditAnywhere. The only cases where it
	//~ seems to have a noticeable effect were ones where it was set so high that it broke the selection
	//~ behavior slightly.
	UPROPERTY()
	double PlaneTolerance = 0.01;

	/** When raycasting to find a selected grid face, this determines whether geometry
	 in the scene that is not part of the edited mesh is hit. */
	UPROPERTY(EditAnywhere, Category = BlockSelection)
	bool bHitUnrelatedGeometry = true;

	/** When the grid ground plane is above some geometry, whether we should hit that
	 plane or pass through to the other geometry. */
	UPROPERTY(EditAnywhere, Category = BlockSelection, AdvancedDisplay)
	bool bHitGridGroundPlaneIfCloser = false;
	
	/** How the selected face is determined. */
	UPROPERTY(EditAnywhere, Category = BlockSelection, AdvancedDisplay)
	ECubeGridToolFaceSelectionMode FaceSelectionMode = ECubeGridToolFaceSelectionMode::OutsideBasedOnNormal;

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString ToggleCornerMode = TEXT("Z to start/complete corner mode.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString PushPull = TEXT("E/Q to pull/push, or use Ctrl+drag.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString ResizeGrid =
#if PLATFORM_MAC
		TEXT("Option + D/A to increase/decrease grid size.");
#else
		TEXT("Ctrl + E/Q to increase/decrease grid size.");
#endif

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString SlideSelection = TEXT("Middle mouse drag to slide selection in plane. "
		"Shift + E/Q to shift selection back/forward.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString FlipSelection = TEXT("T to flip the selection.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString GridGizmo = TEXT("R to show/hide grid gizmo.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString QuickShiftGizmo = TEXT("Ctrl + middle click to quick-reposition "
		"the gizmo while keeping it on grid.");

	UPROPERTY(VisibleAnywhere, Category = ShortcutInfo)
	FString AlignGizmo = TEXT("While dragging gizmo handles, hold Ctrl to align "
		"to items in scene (constrained to the moved axes).");

	UPROPERTY(meta = (TransientToolProperty))
	bool bInCornerMode = false;

	//~ Currently unused... Used to disallow it during corner mode, might do so again.
	UPROPERTY(meta = (TransientToolProperty))
	bool bAllowedToEditGrid = true;
};

UENUM()
enum class ECubeGridToolAction
{
	NoAction,
	Push,
	Pull,
	Flip,
	SlideForward,
	SlideBack,
	DecreaseGridPower,
	IncreaseGridPower,
	CornerMode,
	// FitGrid,
	ResetFromActor,
	AcceptAndStartNew
};

UCLASS(MinimalAPI)
class UCubeGridToolActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UCubeGridTool> ParentTool;

	void Initialize(UCubeGridTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(ECubeGridToolAction Action);

	/** Can also be invoked with E. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 1))
	void Pull() { PostAction(ECubeGridToolAction::Pull); }

	/** Can also be invoked with Q. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 2))
	void Push() { PostAction(ECubeGridToolAction::Push); }

	/** Can also be invoked with Shift + E. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 3))
	void SlideBack() { PostAction(ECubeGridToolAction::SlideBack); }

	/** Can also be invoked with Shift + Q. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 4))
	void SlideForward() { PostAction(ECubeGridToolAction::SlideForward); }

	/** Engages a mode where specific corners can be selected to push/pull only
	 those corners. Press Apply to commit the result afterward. Can also be toggled
	 with Z. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 5))
	void CornerMode() { PostAction(ECubeGridToolAction::CornerMode); }

	/** Can also be invoked with T. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 6))
	void Flip() { PostAction(ECubeGridToolAction::Flip); }

	/** Actor whose transform to use when doing Reset Grid From Actor. */
	//~ For some reason we can't seem to use TWeakObjectPtr here- it becomes unsettable in the tool.
	UPROPERTY(EditAnywhere, Category = GridReinitialization, meta = (TransientToolProperty))
	TObjectPtr<AActor> GridSourceActor = nullptr;

	/** 
	 * Resets the grid position and orientation based on the actor in Grid Source Actor. This allows 
	 * grid positions/orientations to be saved by pasting them into the transform of some actor that
	 * is later used, or by relying on the fact that the tool initializes transforms of newly created
	 * meshes based on the grid used.
	 */
	UFUNCTION(CallInEditor, Category = GridReinitialization)
	void ResetGridFromActor () { PostAction(ECubeGridToolAction::ResetFromActor); }

	/**
	 * Accepts the output of the current tool and restarts it in "create new asset" mode.
	 * Used to avoid creating one huge mesh when working on different parts of a level.
	 * Note that undoing object creation will not remove any created assets on disc (i.e.
	 * undoing the creation of a static mesh will remove the actor from the world but not
	 * remove the static mesh asset from disc).
	 */
	UFUNCTION(CallInEditor, Category = AssetActions)
	void AcceptAndStartNew() { PostAction(ECubeGridToolAction::AcceptAndStartNew); }
};

/** Tool that allows for blocky boolean operations on an orientable power-of-two grid. */
UCLASS(MinimalAPI)
class UCubeGridTool : public UInteractiveTool,
	public IClickDragBehaviorTarget, public IHoverBehaviorTarget,
	public UE::Geometry::IDynamicMeshOperatorFactory,
	public IInteractiveToolNestedAcceptCancelAPI,
	public IInteractiveToolCameraFocusAPI
{
	GENERATED_BODY()
protected:
	enum class EMouseState
	{
		NotDragging,
		DraggingExtrudeDistance,
		DraggingCornerSelection,
		DraggingRegularSelection,
	};

	enum class EMode
	{
		PushPull,
		Corner,

		// This is currently not supported, but some of the code was written with space
		// for a "fit grid" mode that would have allowed the dimensions of the grid to
		// be fit using a sequence of (snapped) mouse clicks. It seems useful to leave 
		// those code stubs for now in case we add the mode in, so it's easier to track
		// down the affected portions of code.
		FitGrid
	};

	using FCubeGrid = UE::Geometry::FCubeGrid;

public:

	struct FSelection
	{
		// Both of these boxes are in the coordinate space of the (unscaled) grid frame.
		UE::Geometry::FAxisAlignedBox3d Box;
		UE::Geometry::FAxisAlignedBox3d StartBox; // Box delineating original selected face

		// Direction must be initialized to a valid enum value (0 is not one).
		FCubeGrid::EFaceDirection Direction = FCubeGrid::EFaceDirection::PositiveX;

		bool operator==(const FSelection& Other) const
		{
			return Box == Other.Box
				&& StartBox == Other.StartBox
				&& Direction == Other.Direction;
		}

		bool operator!=(const FSelection& Other) const
		{
			return !(*this == Other);
		}
	};

	virtual bool HasAccept() const override { return true; };
	virtual bool CanAccept() const override { return true; };
	virtual bool HasCancel() const override { return true; };

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual void SetTarget(TObjectPtr<UToolTarget> TargetIn) { Target = TargetIn; }
	virtual void SetWorld(UWorld* World) { TargetWorld = World; }

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void RequestAction(ECubeGridToolAction ActionType);

	UE_API virtual void SetSelection(const FSelection& Selection, bool bEmitChange);
	UE_API virtual void ClearSelection(bool bEmitChange);

	// Used by undo/redo
	UE_API virtual void UpdateUsingMeshChange(const UE::Geometry::FDynamicMeshChange& MeshChange, bool bRevert);
	UE_API virtual bool IsInDefaultMode() const;
	UE_API virtual bool IsInCornerMode() const;
	UE_API virtual void RevertToDefaultMode();
	UE_API virtual void SetChangesMade(bool bChangesMadeIn);
	UE_API virtual void SetCurrentMeshTransform(const FTransform& TransformIn);
	UE_API virtual void SetCurrentExtrudeAmount(int32 ExtrudeAmount);
	UE_API virtual void SetCornerSelection(bool CornerSelectedFlags[4]);

	// IClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

	// IHoverBehaviorTarget
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// IInteractiveToolNestedAcceptCancelAPI
	virtual bool SupportsNestedCancelCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;
	virtual bool SupportsNestedAcceptCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedAccept() override;
	UE_API virtual bool ExecuteNestedAcceptCommand() override;

	// IInteractiveToolCameraFocusAPI
	virtual bool SupportsWorldSpaceFocusBox() { return bHaveSelection; }
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	void SetInitialGridPivot(FVector3d PivotPos)
	{
		InitialGridPivot = PivotPos;
		bHasInitialGridPivot = true;
	}

	UE_API bool CanApplyPreview() const;

protected:

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> GridGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> GridGizmoAlignmentMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> GridGizmoTransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> LineSets = nullptr;

	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalSingleClickInputBehavior> CtrlMiddleClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<ULocalClickDragInputBehavior> MiddleClickDragBehavior = nullptr;

	// Properties, etc
	UPROPERTY()
	TObjectPtr<UCubeGridToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UCubeGridToolActions> ToolActions = nullptr;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties = nullptr;

	// Existing asset to modify, if one was selected
	UPROPERTY()
	TObjectPtr<UToolTarget> Target = nullptr;

	TSharedPtr<FCubeGrid, ESPMode::ThreadSafe> CubeGrid;

	// Where to make the preview, new mesh, etc
	TObjectPtr<UWorld> TargetWorld = nullptr;

	// Important state. Could refactor things into tool activities (like PolyEdit)
	// someday.
	EMode Mode = EMode::PushPull;
	EMouseState MouseState = EMouseState::NotDragging;

	UE_API bool GetHitGridFace(const FRay& WorldRay, UE::Geometry::FCubeGrid::FCubeFace& FaceOut);
	UE_API bool UpdateHover(const FRay& WorldRay);
	UE_API void UpdateHoverLineSet(bool bHaveHover, const UE::Geometry::FAxisAlignedBox3d& HoveredBox);
	UE_API void UpdateSelectionLineSet();
	UE_API void UpdateGridLineSet();
	UE_API void UpdateCornerModeLineSet();
	UE_API void ClearHover();

	bool bHaveSelection = false;
	FSelection Selection;

	bool bPreviousHaveSelection = false;
	FSelection PreviousSelection;

	bool bHaveHoveredSelection = false;
	UE::Geometry::FAxisAlignedBox3d HoveredSelectionBox;

	UE_API void SlideSelection(int32 ExtrudeAmount, bool bEmitChange);

	bool bSlideToggle = false;
	bool bSelectionToggle = false;
	bool bChangeSideToggle = false;
	bool bMouseDragShouldPushPull = false;
	FRay3d DragProjectionAxis;
	double DragProjectedStartParam;
	int32 DragStartExtrudeAmount = 0;

	UE_API void ApplyFlipSelection();
	UE_API void ApplySlide(int32 NumBlocks);
	UE_API void ApplyPushPull(int32 NumBlocks);

	// Parameter is signed on purpose so we can give negatives for clamping.
	UE_API void SetGridPowerClamped(int32 GridPower);

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	/**
	 * @param bUpdateCornerLineSet This can be set to false when the invalidation
	 *  is a result of a grid transform change (which is applied to the line set
	 *  via the LineSets->SetTransform), or when the corner shape otherwise doesn't
	 *  change. Usually it can be left to true.
	 */
	UE_API void InvalidatePreview(bool bUpdateCornerLineSet = true);

	UE_API void ApplyPreview();

	int32 CurrentExtrudeAmount = 0;
	bool bPreviewMayDiffer = false;
	bool bWaitingToApplyPreview = false;
	bool bBlockUntilPreviewUpdate = false;
	bool bAdjustSelectionOnPreviewUpdate = false;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> CurrentMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> MeshSpatial;
	UE::Geometry::FTransformSRT3d CurrentMeshTransform = UE::Geometry::FTransformSRT3d::Identity();
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> LastOpChangedTids;

	TArray<UMaterialInterface*> CurrentMeshMaterials;
	int32 OpMeshMaterialID = 0;
	UE_API void UpdateOpMaterials();

	// These are used to keep UV's and side groups consistent across multiple E/Q (push/pull) presses. 
	// This data should be reset whenever the selection changes in a way that is not a byproduct of
	// a push/pull.
	double OpMeshHeightUVOffset = 0;
	TArray<int32, TFixedAllocator<4>> OpMeshAddSideGroups;
	TArray<int32, TFixedAllocator<4>> OpMeshSubtractSideGroups;
	UE_API void ResetMultiStepConsistencyData();

	// Safe inputs for the background compute to use, untouched by undo/redo/other CurrentMesh updates.
	TSharedPtr<const UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> ComputeStartMesh;
	UE_API void UpdateComputeInputs();

	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	ECubeGridToolAction PendingAction = ECubeGridToolAction::NoAction;
	UE_API void ApplyAction(ECubeGridToolAction ActionType);
	UE_API void ClearViewportButtonCustomization();

	int32 GridPowerWatcherIdx;
	int32 BlockBaseSizeWatcherIdx;
	int32 CurrentBlockSizeWatcherIdx;

	int32 GridFrameOriginWatcherIdx;
	int32 GridFrameOrientationWatcherIdx;
	UE_API void GridGizmoMoved(UTransformProxy* Proxy, FTransform Transform);
	UE_API void UpdateGizmoVisibility(bool bVisible);
	bool bInGizmoDrag = false;

	/*
	 * Updates the gizmo controlling the cube grid transform. Only updates the cube grid itself if
	 * bSilentlyUpdate is false.
	 * 
	 * @param bSilentlyUpdate If true, gizmo is just repositioned without triggering any callback
	 *   or emitting an undo transaction. If false, emits transactions and triggers the GridGizmoMoved
	 *   callback (which will trigger UpdateGridTransform).
	 */
	UE_API void UpdateGridGizmo(const FTransform& NewTransform, bool bSilentlyUpdate = false);
	
	/*
	* Updates the cube grid.
	* 
	 * @param bUpdateDetailPanel Not needed if updating from the detail panel, otherwise should be true.
	 * @param bTriggerDetailPanelRebuild Should be false if updating from a drag to avoid the costly rebuild, 
	 * but otherwise should be true to update the edit conditions and "revert to default" arrows. Not relevant
	 * if bUpdateDetailPanel is false.
	 */
	UE_API void UpdateGridTransform(const FTransform& NewTransform, bool bUpdateDetailPanel = true, bool bTriggerDetailPanelRebuild = true);
	
	FVector3d MiddleClickDragStart;
	UE_API FInputRayHit RayCastSelectionPlane(const FRay3d& WorldRay, 
		FVector3d& HitPointOut);
	UE_API FInputRayHit CanBeginMiddleClickDrag(const FInputDeviceRay& ClickPos);
	UE_API void OnMiddleClickDrag(const FInputDeviceRay& DragPos);
	UE_API void OnCtrlMiddleClick(const FInputDeviceRay& ClickPos);
	UE_API void PrepForSelectionChange();
	UE_API void EndSelectionChange();

	// Used in corner push/pull mode. If you create a flat FOrientedBox3d out of 
	// the current selection, with the Z axis being along selection normal, the 0-3
	// indices here correspond to the 0-3 corner indices in the box, which are the
	// bottom corners along Z axis (see TOrientedBox3::GetCorner())
	// TODO: Might be nice to pack these into one value for ease of copying/resetting/comparing
	bool CornerSelectedFlags[4] = { false, false, false, false };
	bool PreDragCornerSelectedFlags[4] = { false, false, false, false };

	FViewCameraState CameraState;
	FToolDataVisualizer SelectedCornerRenderer;
	UE::Geometry::FGeometrySet3 CornersGeometrySet;
	UE_API void UpdateCornerGeometrySet();
	UE_API void StartCornerMode();
	UE_API void ApplyCornerMode(bool bDontWaitForTick = false);
	UE_API void CancelCornerMode();
	UE_API void AttemptToSelectCorner(const FRay3d& WorldRay);

	// Used to see if we need to update the asset that we've been modifying
	bool bChangesMade = false;

	UE_API void OutputCurrentResults(bool bSetSelection);
	UE_API void AcceptToolAndStartNew();

private:
	bool bHasInitialGridPivot = false;
	FVector3d InitialGridPivot;
};

#undef UE_API
