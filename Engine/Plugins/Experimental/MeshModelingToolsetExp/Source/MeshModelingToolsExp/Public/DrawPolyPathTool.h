// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "Mechanics/PlaneDistanceFromHitMechanic.h"
#include "Mechanics/SpatialCurveDistanceMechanic.h"
#include "Mechanics/CollectSurfacePathMechanic.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "Drawing/PolyEditPreviewMesh.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "Properties/MeshMaterialProperties.h"
#include "DrawPolyPathTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class FMeshVertexChangeBuilder;
class UTransformProxy;

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UDrawPolyPathToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


UENUM()
enum class EDrawPolyPathWidthMode
{
	/** Fixed width along the drawn path determined by the Width property */
	Fixed,

	/** Extrude drawn path to height set via additional mouse input after finishing the path */
	Interactive
};

UENUM()
enum class EDrawPolyPathRadiusMode
{
	/** Fixed radius determined by the CornerRadius property. */
	Fixed,

	/** Set the radius interactively by clicking in the viewport.  */
	Interactive
};

UENUM()
enum class EDrawPolyPathExtrudeMode
{
	/** Flat path without extrusion */
	Flat,

	/** Extrude drawn path to a fixed height determined by the Extrude Height property */
	Fixed,

	/** Extrude drawn path to height set via additional mouse input after finishing the path */
	Interactive,

	/** Extrude with increasing height along the drawn path. The height at the start and the end of the ramp is determined by the Extrude Height and Ramp Start Ratio properties .*/
	RampFixed,

	/** Extrude with increasing height along the drawn path. The height is set via additional mouse input after finishing the path. */
	RampInteractive
};


UCLASS(MinimalAPI)
class UDrawPolyPathProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** How the drawn path width gets set */
	UPROPERTY(EditAnywhere, Category = Path)
	EDrawPolyPathWidthMode WidthMode = EDrawPolyPathWidthMode::Interactive;

	/** Width of the drawn path when using Fixed width mode; also shows the width in Interactive width mode */
	UPROPERTY(EditAnywhere, Category = Path, meta = (EditCondition = "WidthMode == EDrawPolyPathWidthMode::Fixed", 
		UIMin = "0.0001", UIMax = "1000", ClampMin = "0.0001", ClampMax = "999999"))
	float Width = 10.0f;

	/** Use arc segments instead of straight lines in corners */
	UPROPERTY(EditAnywhere, Category = Path)
	bool bRoundedCorners = false;

	/** How the rounded corner radius gets set */
	UPROPERTY(EditAnywhere, Category = Path, meta = (EditCondition = "bRoundedCorners"))
	EDrawPolyPathRadiusMode RadiusMode = EDrawPolyPathRadiusMode::Interactive;

	/** Radius of the corner arcs, as a fraction of path width. This is only available if Rounded Corners is enabled. */
	UPROPERTY(EditAnywhere, Category = Path, meta = (
		EditCondition = "RadiusMode == EDrawPolyPathRadiusMode::Fixed && bRoundedCorners", 
		UIMin = "0", UIMax = "2.0", ClampMin = "0", ClampMax = "999999"))
	float CornerRadius = 0.5f;

	/** Number of radial subdivisions for rounded corners */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Path, meta = (UIMin = "3", UIMax = "100", ClampMin = "3", ClampMax = "10000",
		EditCondition = "bRoundedCorners"))
	int RadialSlices = 16;

	/** If true, all quads on the path will belong to the same polygon. If false, each quad gets its own polygon. */
	UPROPERTY(EditAnywhere, Category = Path, meta = (DisplayName = "Single PolyGroup"))
	bool bSinglePolyGroup = false;

	/** If and how the drawn path gets extruded */
	UPROPERTY(EditAnywhere, Category = Extrude)
	EDrawPolyPathExtrudeMode ExtrudeMode = EDrawPolyPathExtrudeMode::Interactive;

	/** Extrusion distance when using the Fixed extrude modes; also shows the distance in Interactive extrude modes */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (
		EditCondition = "ExtrudeMode == EDrawPolyPathExtrudeMode::Fixed || ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed",
		UIMin = "-1000", UIMax = "1000", ClampMin = "-10000", ClampMax = "10000"))
	float ExtrudeHeight = 10.0f;

	/** Height of the start of the ramp as a fraction of the Extrude Height property */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (
		EditCondition = "ExtrudeMode == EDrawPolyPathExtrudeMode::RampFixed || ExtrudeMode == EDrawPolyPathExtrudeMode::RampInteractive",
		UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "100.0"))
	float RampStartRatio = 0.05f;
};



UENUM()
enum class EDrawPolyPathExtrudeDirection
{
	SelectionNormal,
	WorldX,
	WorldY,
	WorldZ,
	LocalX,
	LocalY,
	LocalZ
};


UCLASS(MinimalAPI)
class UDrawPolyPathExtrudeProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Extrude)
	EDrawPolyPathExtrudeDirection Direction = EDrawPolyPathExtrudeDirection::SelectionNormal;


};



/**
 *
 */
UCLASS(MinimalAPI)
class UDrawPolyPathTool : public UInteractiveTool, public IClickBehaviorTarget, public IHoverBehaviorTarget, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit);

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget API
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

	// IModifierToggleBehaviorTarget
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	void SetInitialDrawFrame(UE::Geometry::FFrame3d Frame)
	{
		DrawPlaneWorld = Frame;
	}

	// IInteractiveToolManageGeometrySelectionAPI - enables restoration of previous selection when applicable
	UE_API virtual bool IsInputSelectionValidOnOutput() override;

protected:
	UWorld* TargetWorld;

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UDrawPolyPathProperties> TransformProps;

	UPROPERTY()
	TObjectPtr<UDrawPolyPathExtrudeProperties> ExtrudeProperties;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;

protected:
	enum class EState
	{
		DrawingPath,
		SettingWidth,
		SettingRadius,
		SettingHeight
	};
	EState State = EState::DrawingPath;

	// camera state at last render
	FTransform3d WorldTransform;
	FViewCameraState CameraState;

	// drawing plane and gizmo

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	UE::Geometry::FFrame3d DrawPlaneWorld;

	UE_API bool CanUpdateDrawPlane() const;

	// UV Scale factor to apply to texturing on any new geometry (e.g. new faces added by extrude)
	float UVScaleFactor = 1.0f;

	TArray<UE::Geometry::FFrame3d> CurPathPoints;
	TArray<double> OffsetScaleFactors;
	TArray<FVector3d> CurPolyLine;
	double CurPathLength;
	double CurHeight;
	bool bHasSavedWidth = false;
	float SavedWidth;
	bool bHasSavedRadius = false;
	float SavedRadius;
	bool bHasSavedExtrudeHeight = false;
	float SavedExtrudeHeight;
	bool bPathIsClosed = false;		// If true, CurPathPoints are assumed to define a closed path

	static const int ShiftModifierID = 1;
	bool bIgnoreSnappingToggle = false;		// toggled by hotkey (shift)

	TArray<FVector3d> CurPolyLoop;
	TArray<FVector3d> SecondPolyLoop;

	UPROPERTY()
	TObjectPtr<UPolyEditPreviewMesh> EditPreview;

	UPROPERTY()
	TObjectPtr<UPlaneDistanceFromHitMechanic> ExtrudeHeightMechanic = nullptr;
	UPROPERTY()
	TObjectPtr<USpatialCurveDistanceMechanic> CurveDistMechanic = nullptr;
	UPROPERTY()
	TObjectPtr<UCollectSurfacePathMechanic> SurfacePathMechanic = nullptr;

	bool bSpecifyingRadius = false;

	UE_API void InitializeNewSurfacePath();
	UE_API void UpdateSurfacePathPlane();
	UE_API void OnCompleteSurfacePath();

	UE_API void BeginSettingWidth();
	UE_API void OnCompleteWidth();

	UE_API void BeginSettingRadius();
	UE_API void OnCompleteRadius();

	UE_API void BeginSettingHeight();
	UE_API void BeginInteractiveExtrudeHeight();
	UE_API void BeginConstantExtrudeHeight();
	UE_API void UpdateExtrudePreview();
	UE_API void OnCompleteExtrudeHeight();

	UE_API void UpdatePathPreview();
	UE_API void InitializePreviewMesh();
	UE_API void ClearPreview();

	[[nodiscard]] UE_API FVector3d GeneratePathMesh(UE::Geometry::FDynamicMesh3& Mesh);
	UE_API void GenerateExtrudeMesh(UE::Geometry::FDynamicMesh3& PathMesh);
	UE_API void EmitNewObject();

	// user feedback messages
	UE_API void ShowStartupMessage();
	UE_API void ShowExtrudeMessage();
	UE_API void ShowOffsetMessage();

	friend class FDrawPolyPathStateChange;
	int32 CurrentCurveTimestamp = 1;
	UE_API void UndoCurrentOperation(EState DestinationState);
	bool CheckInCurve(int32 Timestamp) const { return CurrentCurveTimestamp == Timestamp; }
private:
	// if the drawn path is incomplete upon 'Accept' of the tool, no mesh should be created and the
	// previous mesh element selection from before entering the tool should be restored
	bool bRestoreInputSelection = true;
};




class FDrawPolyPathStateChange : public FToolCommandChange
{
public:
	bool bHaveDoneUndo = false;
	int32 CurveTimestamp = 0;
	UDrawPolyPathTool::EState PreviousState = UDrawPolyPathTool::EState::DrawingPath;
	FDrawPolyPathStateChange(int32 CurveTimestampIn, UDrawPolyPathTool::EState PreviousStateIn)
		: CurveTimestamp(CurveTimestampIn)
		, PreviousState(PreviousStateIn)
	{
	}
	virtual void Apply(UObject* Object) override {}
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual bool HasExpired(UObject* Object) const override;
	UE_API virtual FString ToString() const override;
};

#undef UE_API
