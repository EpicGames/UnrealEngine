// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "CleaningOps/EditNormalsOp.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "Selections/GeometrySelection.h"
#include "EditNormalsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


// predeclarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UEditNormalsTool;
class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;

/**
 *
 */
UCLASS(MinimalAPI)
class UEditNormalsToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};





/**
 * Standard properties
 */
UCLASS(MinimalAPI)
class UEditNormalsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API UEditNormalsToolProperties();

	bool WillTopologyChange()
	{
		return bFixInconsistentNormals || bInvertNormals || SplitNormalMethod != ESplitNormalMethod::UseExistingTopology;
	}

	/** Recompute all mesh normals */
	UPROPERTY(EditAnywhere, Category = NormalsCalculation,
		meta = (EditCondition = "SplitNormalMethod == ESplitNormalMethod::UseExistingTopology && !bToolHasSelection", HideEditConditionToggle))
	bool bRecomputeNormals;

	/** Choose the method for computing vertex normals */
	UPROPERTY(EditAnywhere, Category = NormalsCalculation)
	ENormalCalculationMethod NormalCalculationMethod;

	/** For meshes with inconsistent triangle orientations/normals, flip as needed to make the normals consistent */
	UPROPERTY(EditAnywhere, Category = NormalsCalculation,
		meta = (EditCondition = "!bToolHasSelection", HideEditConditionToggle))
	bool bFixInconsistentNormals;

	/** Invert (flip) all mesh normals and associated triangle orientations */
	UPROPERTY(EditAnywhere, Category = NormalsCalculation)
	bool bInvertNormals;

	/** Control whether and how the topology of the normals is recomputed, e.g. to create sharp edges where face normals change by a large amount or where face group IDs change.  Normals will always be recomputed unless SplitNormal Method is UseExistingTopology. */
	UPROPERTY(EditAnywhere, Category = NormalsTopology)
	ESplitNormalMethod SplitNormalMethod;

	/** Threshold on angle of change in face normals across an edge, above which we create a sharp edge if bSplitNormals is true */
	UPROPERTY(EditAnywhere, Category = NormalsTopology, meta = (UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", EditCondition = "SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold"))
	float SharpEdgeAngleThreshold;

	/** Assign separate normals at 'sharp' vertices, for example, at the tip of a cone */
	UPROPERTY(EditAnywhere, Category = NormalsTopology, meta = (EditCondition = "SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold"))
	bool bAllowSharpVertices;

	//
	// The following are not user visible
	//

	UPROPERTY(meta = (TransientToolProperty))
	bool bToolHasSelection;
};






/**
 * Factory with enough info to spawn the background-thread Operator to do a chunk of work for the tool
 *  stores a pointer to the tool and enough info to know which specific operator it should spawn
 */
UCLASS(MinimalAPI)
class UEditNormalsOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UEditNormalsTool> Tool;

	int ComponentIndex;

};

/**
 * Simple Mesh Normal Updating Tool
 */
UCLASS(MinimalAPI)
class UEditNormalsTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:

	friend UEditNormalsOperatorFactory;

	UE_API UEditNormalsTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	UE_API virtual bool HasAccept() const override;
	UE_API virtual bool CanAccept() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent) override;
#endif

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// input selection support
	UE_API void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<UEditNormalsToolProperties> BasicProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;


protected:
	TArray<TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe>> OriginalDynamicMeshes;

	FViewCameraState CameraState;

	UE_API void UpdateNumPreviews();

	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);


	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();

	//
	// Selection. Only used when the tool is run with one target
	//

	UPROPERTY()
	TObjectPtr<UGeometrySelectionVisualizationProperties> GeometrySelectionVizProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> GeometrySelectionViz = nullptr;

	// The geometry selection that the user started the tool with. If the selection is empty we operate on the whole
	// mesh, if its not empty we only edit the overlay elements implied by the selection.
	UE::Geometry::FGeometrySelection InputGeometrySelection;

	// If the user starts the tool with an edge selection we convert it to a vertex selection with triangle topology
	// and store it here, we do this since we expect users to want vertex and edge selections to behave similarly.
	UE::Geometry::FGeometrySelection TriangleVertexGeometrySelection;

	// These are indices into the tool target mesh.
	// If both are non-empty we edit the corresponding elements in the overlay, otherwise operate on the whole overlay
	TSet<int> EditTriangles;
	TSet<int> EditVertices;

	// Cache the input polygroup set which was used to start the tool. We do this because users can change the
	// polygroup referenced by the operator while using the tool.
	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> InputGeometrySelectionPolygroupSet;
};

#undef UE_API
