// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Changes/TransformChange.h"
#include "FrameTypes.h"
#include "BoxTypes.h"
#include "EditPivotTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UDragAlignmentMechanic;
class UBaseAxisTranslationGizmo;
class UAxisAngleGizmo;
class UCombinedTransformGizmo;
class UTransformProxy;
class UEditPivotTool;

/**
 *
 */
UCLASS(MinimalAPI)
class UEditPivotToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};




/** Snap-Drag Rotation Mode */
UENUM()
enum class EEditPivotSnapDragRotationMode : uint8
{
	/** Snap-Drag aligns the pivot Z Axis and Target Normals to point in the same direction */
	Align = 1 UMETA(DisplayName = "Align"),

	/** Snap-Drag aligns the pivot Z Axis to the opposite of the Target Normal direction */
	AlignFlipped = 2 UMETA(DisplayName = "Align Flipped"),

	LastValue UMETA(Hidden)
};



/**
 * Standard properties of the Edit Pivot operation
 */
UCLASS(MinimalAPI)
class UEditPivotToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** If checked, the baked transform will be applied to all available LODs. Has no effect on selections without LODs. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bApplyToAllLODs = true;

	/** When enabled, click-drag to reposition the Pivot */
	UPROPERTY(EditAnywhere, Category = SnapDragging)
	bool bSnapDragPosition = false;
	
	/** When enabled, click-drag to reorient the Pivot */
	UPROPERTY(EditAnywhere, Category = SnapDragging)
	bool bSnapDragRotation = false;

	/** When snap-dragging rotation, how to align source and target normals */
	UPROPERTY(EditAnywhere, Category = SnapDragging, meta = (EditCondition = "bSnapDragRotation"))
	EEditPivotSnapDragRotationMode RotationMode = EEditPivotSnapDragRotationMode::Align;
};


USTRUCT()
struct FEditPivotTarget
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;
};




UENUM()
enum class EEditPivotToolActions
{
	NoAction,

	Center,
	Bottom,
	Top,
	Left,
	Right,
	Front,
	Back,
	WorldOrigin
};



UCLASS(MinimalAPI)
class UEditPivotToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UEditPivotTool> ParentTool;

	void Initialize(UEditPivotTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EEditPivotToolActions Action);

	/** Use the World-Space Bounding Box of the target object, instead of the Object-space Bounding Box */
	UPROPERTY(EditAnywhere, Category = BoxPositions)
	bool bUseWorldBox = false;

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 1))
	void Center() { PostAction(EEditPivotToolActions::Center); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 2))
	void Bottom() { PostAction(EEditPivotToolActions::Bottom ); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 2))
	void Top() { PostAction(EEditPivotToolActions::Top); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 3))
	void Left() { PostAction(EEditPivotToolActions::Left); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 3))
	void Right() { PostAction(EEditPivotToolActions::Right); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 4))
	void Front() { PostAction(EEditPivotToolActions::Front); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 4))
	void Back() { PostAction(EEditPivotToolActions::Back); }

	UFUNCTION(CallInEditor, Category = BoxPositions, meta = (DisplayPriority = 5))
	void WorldOrigin() { PostAction(EEditPivotToolActions::WorldOrigin); }
};



/**
 *
 */
UCLASS(MinimalAPI)
class UEditPivotTool : public UMultiSelectionMeshEditingTool, public IClickDragBehaviorTarget, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UEditPivotTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


	// ICLickDragBehaviorTarget interface
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

public:
	UPROPERTY()
	TObjectPtr<UEditPivotToolProperties> TransformProps;

	UPROPERTY()
	TObjectPtr<UEditPivotToolActionPropertySet> EditPivotActions;

	UE_API virtual void RequestAction(EEditPivotToolActions ActionType);

	void SetInitialPivot(FTransform3d InInitialPivot) 
	{
		bHasCustomInitialPivot = true;
		InitialPivot = InInitialPivot;
	}

protected:
	TArray<int> MapToFirstOccurrences;

	FTransform3d InitialPivot = FTransform3d::Identity;
	bool bHasCustomInitialPivot = false;

	FTransform3d Transform;
	UE::Geometry::FAxisAlignedBox3d ObjectBounds;
	UE::Geometry::FAxisAlignedBox3d WorldBounds;
	UE_API void Precompute();

	UPROPERTY()
	TArray<FEditPivotTarget> ActiveGizmos;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UE_API void UpdateSetPivotModes(bool bEnableSetPivot);
	UE_API void SetActiveGizmos_Single(bool bLocalRotations);
	UE_API void ResetActiveGizmos();

	FTransform StartDragTransform;

	EEditPivotToolActions PendingAction;
	UE_API virtual void ApplyAction(EEditPivotToolActions ActionType);
	UE_API virtual void SetPivotToBoxPoint(EEditPivotToolActions ActionPoint);
	UE_API virtual void SetPivotToWorldOrigin();

	UE_API void UpdateAssets(const UE::Geometry::FFrame3d& NewPivotWorldFrame);
};

#undef UE_API
