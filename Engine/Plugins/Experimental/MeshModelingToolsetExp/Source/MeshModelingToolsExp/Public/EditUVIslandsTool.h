// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Changes/MeshVertexChange.h"
#include "Components/DynamicMeshComponent.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "GroupTopology.h"
#include "ModelingTaskTypes.h"
#include "Properties/MeshMaterialProperties.h"
#include "Selection/GroupTopologySelector.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolDataVisualizer.h"
#include "EditUVIslandsTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class FMeshVertexChangeBuilder;
class UCombinedTransformGizmo;
class UTransformProxy;
using UE::Geometry::FDynamicMeshUVOverlay;

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UEditUVIslandsToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};




class FUVGroupTopology : public FGroupTopology
{
public:
	TArray<int32> TriIslandGroups;
	const FDynamicMeshUVOverlay* UVOverlay;

	FUVGroupTopology() {}
	UE_API FUVGroupTopology(const UE::Geometry::FDynamicMesh3* Mesh, uint32 UVLayerIndex, bool bAutoBuild = false);

	UE_API void CalculateIslandGroups();

	virtual int GetGroupID(int32 TriangleID) const override
	{
		return TriIslandGroups[TriangleID];
	}

	UE_API UE::Geometry::FFrame3d GetIslandFrame(int32 GroupID, UE::Geometry::FDynamicMeshAABBTree3& AABBTree);
};


/**
 *
 */
UCLASS(MinimalAPI)
class UEditUVIslandsTool : public UMeshSurfacePointTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UEditUVIslandsTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// UMeshSurfacePointTool API
	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual void OnCancelDrag() override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IClickDragBehaviorTarget API
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

public:
	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> MaterialSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> CheckerMaterial = nullptr;

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic;

	bool bSelectionStateDirty = false;
	UE_API void OnSelectionModifiedEvent();

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UE_API void OnGizmoTransformBegin(UTransformProxy* TransformProxy);
	UE_API void OnGizmoTransformUpdate(UTransformProxy* TransformProxy, FTransform Transform);
	UE_API void OnGizmoTransformEnd(UTransformProxy* TransformProxy);

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged();
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	// camera state at last render
	FTransform3d WorldTransform;
	FViewCameraState CameraState;

	// True for the duration of UI click+drag
	bool bInDrag;

	double UVTranslateScale;
	UE::Geometry::FFrame3d InitialGizmoFrame;
	FVector3d InitialGizmoScale;
	UE_API void ComputeUpdate_Gizmo();

	FUVGroupTopology Topology;
	UE_API void PrecomputeTopology();

	FDynamicMeshAABBTree3 MeshSpatial;
	UE_API FDynamicMeshAABBTree3& GetSpatial();
	bool bSpatialDirty;


	//
	// data for current drag
	//
	struct FEditIsland
	{
		UE::Geometry::FFrame3d LocalFrame;
		TArray<int32> Triangles;
		TArray<int32> UVs;
		UE::Geometry::FAxisAlignedBox2d UVBounds;
		FVector2d UVOrigin;
		TArray<FVector2f> InitialPositions;
	};
	TArray<FEditIsland> ActiveIslands;
	UE_API void UpdateUVTransformFromSelection(const FGroupTopologySelection& Selection);

	FMeshVertexChangeBuilder* ActiveVertexChange;
	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void UpdateChangeFromROI(bool bFinal);

	UE_API void OnMaterialSettingsChanged();
};

#undef UE_API
