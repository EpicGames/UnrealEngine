// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolChange.h"
#include "ToolContextInterfaces.h" // FViewCameraState
#include "TransactionUtil.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "MeshOpPreviewHelpers.h"

#include "BaseTools/SingleSelectionMeshEditingTool.h"

#include "ExtractSplineTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UGenerateCrossSectionOpFactory;
class UPreviewGeometry;
class UPolygonSelectionMechanic;

namespace UE {
	namespace Geometry {
		class FGroupTopology;
	}
}

UENUM()
enum class EExtractSplineMode : uint8
{
	// Extract splines as cross sections where a specified plane intersects the target mesh
	PlaneCut = 1,

	// Extract open boundary edges on target mesh as splines
	OpenBoundary = 2,

	// Extract splines from selected polygroup loop boundaries
	PolygroupLoops = 3
};

UCLASS(MinimalAPI)
class UExtractSplineToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Extract Splines")
	EExtractSplineMode ExtractionMode = EExtractSplineMode::PlaneCut;
};


/*
 * A tool to create splines from mesh cross sections or open mesh boundary edges.
 */
UCLASS(MinimalAPI)
class UExtractSplineTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


private:

	UPROPERTY()
	TObjectPtr<UExtractSplineToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic = nullptr;

	TSharedPtr<UE::Geometry::FGroupTopology> Topology;
	UE_API UE::Geometry::FDynamicMeshAABBTree3& GetSpatial();
	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	TSharedPtr<UE::Geometry::FDynamicMesh3> OriginalMesh;
	
	UPROPERTY()
	TObjectPtr<UGenerateCrossSectionOpFactory> Factory;

	// Cutting plane
	UE::Geometry::FFrame3d CutPlaneWorld;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> CutlineGeometry;

	typedef TArray<FVector3d> FPolygonVertices;
	TArray<FPolygonVertices> CutLoops;
	TArray<FPolygonVertices> CutSpans;

	UE_API void SetupPreviews();
	UE_API void InvalidatePreviews();
	UE_API void UpdateVisibility();
	UE_API void GetCutPlane(FVector& Origin, FVector& Normal);
	UE_API void GenerateAsset();
	UE_API void RegeneratePreviewSplines();
	UE_API void GatherSplineDataFromMeshBoundaries();
	UE_API void GatherSplineDataFromPolygroupSelection();
	UE_API void SetInteractionMode();

};

UCLASS(MinimalAPI, Transient)
class UExtractSplineToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UExtractSplineTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

#undef UE_API
