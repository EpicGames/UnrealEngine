// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "GroupTopology.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"

#include "MeshBoundaryToolBase.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class UPolygonSelectionMechanic;
class USingleClickInputBehavior;

/**
  * Base class for tools that do things with a mesh boundary. Provides ability to select mesh boundaries
  * and some other boilerplate code.
  *	TODO: We can refactor to make the HoleFiller tool inherit from this.
  */
UCLASS(MinimalAPI)
class UMeshBoundaryToolBase : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	// CanAccept() needs to be provided by child.

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

protected:

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	// Used for hit querying
	UE::Geometry::FDynamicMeshAABBTree3 MeshSpatial;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic = nullptr;


	// A variant of group topology that considers all triangles one group, so that group edges are boundary
	// edges in the mesh.
	class FBasicTopology : public UE::Geometry::FGroupTopology
	{
	public:
		FBasicTopology(const FDynamicMesh3* Mesh, bool bAutoBuild) :
			FGroupTopology(Mesh, false)
		{
			if (bAutoBuild)
			{
				// Virtual func resolution doesn't work in constructors. Though we're not currently
				// overriding RebuildTopology, let's do the proper thing in case we get copy-pasted
				// somewhere where we do.
				RebuildTopology();
			}
		}

		int GetGroupID(int TriangleID) const override
		{
			return Mesh->IsTriangle(TriangleID) ? 1 : 0;
		}
	};
	TUniquePtr<FBasicTopology> Topology;

	// Override in the child to respond to new loop selections.
	virtual void OnSelectionChanged() {};
};

#undef UE_API
