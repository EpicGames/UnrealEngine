// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "Drawing/LineSetComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Properties/MeshMaterialProperties.h"
#include "PreviewMesh.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "FaceGroupUtil.h"
#include "MeshInspectorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


// predeclarations
struct FMeshDescription;
class UMaterialInstanceDynamic;

/**
 *
 */
UCLASS(MinimalAPI)
class UMeshInspectorToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class EMeshInspectorToolDrawIndexMode : uint8
{
	None,
	VertexID,
	TriangleID,
	GroupID,
	EdgeID
};


UCLASS(MinimalAPI)
class UMeshInspectorProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Toggle visibility of all mesh edges */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWireframe = true;

	/** Toggle visibility of open boundary edges */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bBoundaryEdges = true;

	/** Toggle visibility of bowtie vertices */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bBowtieVertices = true;

	/** Toggle visibility of polygon borders */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bPolygonBorders = false;

	/** Toggle visibility of UV seam edges */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bUVSeams = false;

	/** Toggle visibility of UV bowtie vertices */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bUVBowties = false;

	/** Toggle visibility of triangles with missing UVs */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bMissingUVs = false;

	/** Toggle visibility of Normal seam edges */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bNormalSeams = false;

	/** Toggle visibility of Tangent seam edges */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTangentSeams = false;

	/** Toggle visibility of normal vectors */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bNormalVectors = false;

	/** Toggle visibility of tangent vectors */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTangentVectors = false;

	/** Toggle visibility of occluded boundary edges and seams */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bDrawHiddenEdgesAndSeams = true;

	/** Length of line segments representing normal vectors */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bNormalVectors",
		UIMin="0", UIMax="400", ClampMin = "0", ClampMax = "1000000000.0"))
	float NormalLength = 5.0f;

	/** Length of line segments representing tangent vectors */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bTangentVectors",
		UIMin = "0", UIMax = "400", ClampMin = "0", ClampMax = "1000000000.0"))
	float TangentLength = 5.0f;

	/** Draw the mesh indices of the selected type. A maximum of 500 visible indices will be rendered. */
	UPROPERTY(EditAnywhere, Category=Options)
	EMeshInspectorToolDrawIndexMode ShowIndices = EMeshInspectorToolDrawIndexMode::None;
};




/** Material Modes for Mesh Inspector Tool */
UENUM()
enum class EMeshInspectorMaterialMode : uint8
{
	/** Input material */
	Original,
	/** Flat Shaded Material, ie with per-triangle normals */
	FlatShaded,
	/** Grey material */
	Grey,
	/** Transparent material, with opacity/color controls */
	Transparent,
	/** Tangent/Normal material */
	TangentNormal,
	/** Vertex Color material */
	VertexColor,
	/** Polygroup Color material */
	GroupColor,
	/** Checkerboard material */
	Checkerboard,
	/** Override material */
	Override
};


// material settings for mesh inspector tool
UCLASS(MinimalAPI)
class UMeshInspectorMaterialProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Material that will be used to render the mesh */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial)
	EMeshInspectorMaterialMode MaterialMode = EMeshInspectorMaterialMode::Original;

	/** Number of checkerboard tiles within the 0 to 1 range; only available when Checkerboard is selected as material mode */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial,
		meta = (UIMin = "10.0", UIMax = "100.0", ClampMin = "0.01", ClampMax = "10000.0", EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Checkerboard"))
	float CheckerDensity = 50.0f;

	/** Material to use instead of the original material; only available when Override is selected as material mode */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Override"))
	TObjectPtr<UMaterialInterface> OverrideMaterial = nullptr;

	/** Which UV channel to use for visualizing the checkerboard material on the mesh; note that this does not affect the preview layout */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial,
		meta = (DisplayName = "Preview UV Channel", GetOptions = GetUVChannelNamesFunc, EditConditionHides, EditCondition =
			"MaterialMode == EMeshInspectorMaterialMode::Checkerboard", NoResetToDefault))
	FString UVChannel;

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVChannelNamesList;

	UFUNCTION()
	UE_API const TArray<FString>& GetUVChannelNamesFunc() const;


	/** Toggle flat shading on/off */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::VertexColor || MaterialMode == EMeshInspectorMaterialMode::GroupColor") )
	bool bFlatShading = false;

	/** Main Color of Material */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Diffuse"))
	FLinearColor Color = FLinearColor(0.4f, 0.4f, 0.4f);

	/** Opacity of transparent material */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Transparent", ClampMin = "0", ClampMax = "1.0"))
	double Opacity = 0.65;

	//~ Could have used the same property as Color, above, but the user may want different saved values for the two
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Transparent", DisplayName = "Color"))
	FLinearColor TransparentMaterialColor = FLinearColor(0.0606f, 0.309f, 0.842f);

	/** Although a two-sided transparent material causes rendering issues with overlapping faces, it is still frequently useful to see the shape when sculpting around other objects. */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshInspectorMaterialMode::Transparent"))
	bool bTwoSided = true;

	UPROPERTY(meta = (TransientToolProperty))
	TObjectPtr<UMaterialInstanceDynamic> CheckerMaterial = nullptr;

	UPROPERTY(meta = (TransientToolProperty))
	TObjectPtr<UMaterialInstanceDynamic> ActiveCustomMaterial = nullptr;

	// Needs custom restore in order to call setup
	UE_API virtual void RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier = TEXT("")) override;

	UE_API void Setup();

	UE_API void UpdateMaterials();
	UE_API UMaterialInterface* GetActiveOverrideMaterial() const;
	
	UE_API void UpdateUVChannels(int32 UVChannelIndex, const TArray<FString>& UVChannelNames, bool bUpdateSelection = true);
};








/**
 * Mesh Inspector Tool for visualizing mesh information
 */
UCLASS(MinimalAPI)
class UMeshInspectorTool : public USingleSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UMeshInspectorTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return false; }
	UE_API virtual bool HasAccept() const override;
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

public:

	UE_API virtual void IncreaseLineWidthAction();
	UE_API virtual void DecreaseLineWidthAction();

protected:

	UPROPERTY()
	TObjectPtr<UMeshInspectorProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshInspectorMaterialProperties> MaterialSettings = nullptr;

	float LineWidthMultiplier = 1.0f;

protected:
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLineSet;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> DefaultMaterial = nullptr;

	TArray<int> BoundaryEdges;
	TArray<int> BoundaryBowties;
	TArray<int> UVSeamEdges;
	TArray<int> UVBowties;
	TArray<int> NormalSeamEdges;
	TArray<int> TangentSeamEdges;
	TArray<int> GroupBoundaryEdges;
	TArray<int> MissingUVTriangleEdges;

	UE_API void UpdateVisualization();
	UE_API void Precompute();

	FViewCameraState CameraState;
	FTransform LocalToWorldTransform;

	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> MeshAABBTree;
	UE_API UE::Geometry::FDynamicMeshAABBTree3* GetSpatial();

	TUniquePtr<UE::Geometry::FPolygroupSet> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();

	bool bDrawGroupsDataValid = false;
	UE::Geometry::FGroupVisualizationCache GroupVisualizationCache;
};

#undef UE_API
