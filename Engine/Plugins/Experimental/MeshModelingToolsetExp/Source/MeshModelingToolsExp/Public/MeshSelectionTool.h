// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMeshBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "InteractiveToolQueryInterfaces.h"
#include "SelectionSet.h"
#include "Changes/MeshSelectionChange.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "Polygroups/PolygroupSet.h"
#include "Selections/GeometrySelection.h"
#include "MeshSelectionTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UMeshStatisticsProperties;
class UMeshElementsVisualizer;
class UMeshUVChannelProperties;
class UPolygroupLayersProperties;
class UMeshSelectionTool;

/**
 *
 */
UCLASS(MinimalAPI)
class UMeshSelectionToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMeshSurfacePointTool* Tool, const FToolBuilderState& SceneState) const override;
};




UENUM()
enum class EMeshSelectionToolActions
{
	NoAction,

	SelectAll,
	SelectAllByMaterial,
	ClearSelection,
	InvertSelection,
	GrowSelection,
	ShrinkSelection,
	ExpandToConnected,

	SelectLargestComponentByTriCount,
	SelectLargestComponentByArea,
	OptimizeSelection,

	DeleteSelected,
	DisconnectSelected,
	SeparateSelected,
	DuplicateSelected,
	FlipSelected,
	CreateGroup,
	SmoothBoundary,

	CycleSelectionMode,
	CycleViewMode
};



UCLASS(MinimalAPI)
class UMeshSelectionToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMeshSelectionTool> ParentTool;

	void Initialize(UMeshSelectionTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EMeshSelectionToolActions Action);
};




UCLASS(MinimalAPI)
class UMeshSelectionEditActions : public UMeshSelectionToolActionPropertySet
{
	GENERATED_BODY()

public:
	/** Clear the active triangle selection */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 0))
	void Clear()
	{
		PostAction(EMeshSelectionToolActions::ClearSelection);
	}

	/** Select all triangles in the mesh */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 1))
	void SelectAll()
	{
		PostAction(EMeshSelectionToolActions::SelectAll);
	}


	/** Invert the active triangle selection */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 2))
	void Invert()
	{
		PostAction(EMeshSelectionToolActions::InvertSelection);
	}

	/** Grow the active triangle selection to include any triangles touching a vertex on the selection boundary */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 3))
	void Grow()
	{
		PostAction(EMeshSelectionToolActions::GrowSelection);
	}

	/** Shrink the active triangle selection by removing any triangles touching a vertex on the selection boundary */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = ( DisplayPriority = 4))
	void Shrink()
	{
		PostAction(EMeshSelectionToolActions::ShrinkSelection);
	}

	/** Grow the active selection to include any triangle connected via shared edges (ie flood-fill) */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 5))
	void FloodFill()
	{
		PostAction(EMeshSelectionToolActions::ExpandToConnected);
	}

	/** Select the largest connected mesh component by triangle count */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 6))
	void LargestTriCountPart()
	{
		PostAction(EMeshSelectionToolActions::SelectLargestComponentByTriCount);
	}

	/** Select the largest connected mesh component by mesh area */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 7))
	void LargestAreaPart()
	{
		PostAction(EMeshSelectionToolActions::SelectLargestComponentByArea);
	}

	/** Optimize the selection border by removing "fin" triangles and including "ear" triangles */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 8))
	void OptimizeBorder()
	{
		PostAction(EMeshSelectionToolActions::OptimizeSelection);
	}

	/** Expand the selection to include all triangles with Materials matching the Materials on the currently selected triangles */
	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayPriority = 9))
	void ExpandToMaterials()
	{
		PostAction(EMeshSelectionToolActions::SelectAllByMaterial);
	}
};




UCLASS(MinimalAPI)
class UMeshSelectionMeshEditActions : public UMeshSelectionToolActionPropertySet
{
	GENERATED_BODY()

public:
	/** Delete the selected triangles */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Delete", DisplayPriority = 1))
	void Delete()
	{
		PostAction(EMeshSelectionToolActions::DeleteSelected);
	}

	/** Disconnected the selected triangles from their neighbours, to create mesh boundaries along the selection borders */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Disconnect", DisplayPriority = 3))
	void Disconnect() 
	{
		PostAction(EMeshSelectionToolActions::DisconnectSelected);
	}

	/** Flip the normals of the selected triangles. This will create hard normals at selection borders. */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Flip Normals", DisplayPriority = 4))
	void FlipNormals() 
	{
		PostAction(EMeshSelectionToolActions::FlipSelected);
	}

	/** Assign a new unique Polygroup index to the selected triangles */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Create Polygroup", DisplayPriority = 5))
	void CreatePolygroup()
	{
		PostAction(EMeshSelectionToolActions::CreateGroup);
	}

	/** Delete the selected triangles from the active Mesh Object and create a new Mesh Object containing those triangles */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Separate", DisplayPriority = 10))
	void Separate() 
	{
		PostAction(EMeshSelectionToolActions::SeparateSelected);
	}

	/** Create a new Mesh Object containing the selected triangles */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayName = "Duplicate", DisplayPriority = 11))
	void Duplicate() 
	{
		PostAction(EMeshSelectionToolActions::DuplicateSelected);
	}

	/** Smooth the selection border */
	UFUNCTION(CallInEditor, Category = MeshEdits, meta = (DisplayPriority = 12))
	void SmoothBorder()
	{
		PostAction(EMeshSelectionToolActions::SmoothBoundary);
	}

};






UENUM()
enum class EMeshSelectionToolPrimaryMode
{
	/** Select all triangles inside the brush area */
	Brush,

	/** Select all triangles inside the brush volume */
	VolumetricBrush,

	/** Select all triangles inside brush with normal within angular tolerance of hit triangle */
	AngleFiltered,

	/** Select all triangles inside brush that are visible from current view */
	Visible,

	/** Select all triangles connected to any triangle inside the brush */
	AllConnected,

	/** Select all triangles in groups connected to any triangle inside the brush */
	AllInGroup,

	/** Select the connected group of triangles with same material as hit triangle */
	ByMaterial UMETA(DisplayName = "By Material (Connected)"),

	/** Select all triangles with same material as hit triangle */
	ByMaterialAll UMETA(DisplayName = "By Material (All)"),

	/** Select all triangles in same UV island as hit triangle */
	ByUVIsland,

	/** Select all triangles with normal within angular tolerance of hit triangle */
	AllWithinAngle
};



UENUM()
enum class EMeshFacesColorMode
{
	/** Display original mesh materials */
	None,
	/** Color mesh triangles by PolyGroup Color */
	ByGroup,
	/** Color mesh triangles by Material ID */
	ByMaterialID,
	/** Color mesh triangles by UV Island */
	ByUVIsland
};


UCLASS(MinimalAPI)
class UMeshSelectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** The Selection Mode defines the behavior of the selection brush */
	UPROPERTY(EditAnywhere, Category = Selection)
	EMeshSelectionToolPrimaryMode SelectionMode = EMeshSelectionToolPrimaryMode::Brush;

	/** Angle in Degrees used for Angle-based Selection Modes */
	UPROPERTY(EditAnywhere, Category = Selection, meta = (UIMin = "0.0", UIMax = "90.0") )
	float AngleTolerance = 1.0;

	/** Allow the brush to hit back-facing parts of the surface  */
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bHitBackFaces = true;

	/** Toggle drawing of highlight points on/off */
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bShowPoints = false;

	/** Color each triangle based on the selected mesh attribute */
	UPROPERTY(EditAnywhere, Category = Selection)
	EMeshFacesColorMode FaceColorMode = EMeshFacesColorMode::None;
};



/**
 *
 */
UCLASS(MinimalAPI)
class UMeshSelectionTool : public UDynamicMeshBrushTool, public IInteractiveToolNestedAcceptCancelAPI
{
	GENERATED_BODY()

public:
	UE_API UMeshSelectionTool();

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	//virtual bool CanAccept() const override { return Super::CanAccept() && bHaveModifiedMesh; }
	virtual bool CanAccept() const override { return Super::CanAccept(); }		// allow selection w/o modified mesh to allow for use as just a selection tool

	// UBaseBrushTool overrides
	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	// IInteractiveToolCancelAPI
	UE_API virtual bool SupportsNestedCancelCommand() override;
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;

	// input selection support
	UE_API virtual void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn);


public:

	UE_API virtual void RequestAction(EMeshSelectionToolActions ActionType);

	UPROPERTY()
	TObjectPtr<UMeshSelectionToolProperties> SelectionProps;

	UPROPERTY()
	TObjectPtr<UMeshSelectionEditActions> SelectionActions;

	UPROPERTY()
	TObjectPtr<UMeshSelectionToolActionPropertySet> EditActions;

	UPROPERTY()
	TObjectPtr<UMeshStatisticsProperties> MeshStatisticsProperties;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

protected:
	UE_API virtual UMeshSelectionToolActionPropertySet* CreateEditActions();
	virtual void AddSubclassPropertySets() {}

protected:

	UE_API virtual void ApplyStamp(const FBrushStampData& Stamp);

	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

protected:

	UE::Geometry::FGeometrySelection InputGeometrySelection;

	UPROPERTY()
	TObjectPtr<UMeshSelectionSet> Selection;

	UPROPERTY()
	TArray<TObjectPtr<AActor>> SpawnedActors;

	UWorld* TargetWorld;

	// note: ideally this octree would be part of PreviewMesh!
	TUniquePtr<UE::Geometry::FDynamicMeshOctree3> Octree;
	bool bOctreeValid = false;
	UE_API TUniquePtr<UE::Geometry::FDynamicMeshOctree3>& GetOctree();

	EMeshSelectionElementType SelectionType = EMeshSelectionElementType::Face;

	bool bInRemoveStroke = false;
	FBrushStampData StartStamp;
	FBrushStampData LastStamp;
	bool bStampPending;

	UE_API void UpdateFaceSelection(const FBrushStampData& Stamp, const TArray<int>& BrushROI);


	// temp
	TArray<int> IndexBuf;
	TArray<int32> TemporaryBuffer;
	TSet<int32> TemporarySet;
	UE_API void CalculateVertexROI(const FBrushStampData& Stamp, TArray<int>& VertexROI);
	UE_API void CalculateTriangleROI(const FBrushStampData& Stamp, TArray<int>& TriangleROI);
	TArray<int> PreviewBrushROI;
	TBitArray<> SelectedVertices;
	TBitArray<> SelectedTriangles;
	UE_API void OnExternalSelectionChange();

	UE_API void OnRegionHighlightUpdated(const TArray<int32>& Triangles);
	UE_API void OnRegionHighlightUpdated(const TSet<int32>& Triangles);
	UE_API void OnRegionHighlightUpdated();
	UE_API void UpdateVisualization(bool bSelectionModified);
	bool bFullMeshInvalidationPending = false;
	bool bColorsUpdatePending = false;
	UE_API FColor GetCurrentFaceColor(const FDynamicMesh3* Mesh, int TriangleID);
	UE_API void CacheUVIslandIDs();
	TArray<int> TriangleToUVIsland;

	// selection change
	FMeshSelectionChangeBuilder* ActiveSelectionChange = nullptr;
	UE_API void BeginChange(bool bAdding);
	UE_API TUniquePtr<FToolCommandChange> EndChange();
	UE_API void CancelChange();


	// actions

	bool bHavePendingAction = false;
	EMeshSelectionToolActions PendingAction;
	UE_API virtual void ApplyAction(EMeshSelectionToolActions ActionType);

	UE_API void SelectAll();
	UE_API void ClearSelection();
	UE_API void InvertSelection();
	UE_API void GrowShrinkSelection(bool bGrow);
	UE_API void ExpandToConnected();
	UE_API void SelectAllByMaterial();

	UE_API void SelectLargestComponent(bool bWeightByArea);
	UE_API void OptimizeSelection();

	UE_API void DeleteSelectedTriangles();
	UE_API void DisconnectSelectedTriangles(); // disconnects edges between selected and not-selected triangles; keeps all triangles in the same mesh
	UE_API void SeparateSelectedTriangles(bool bDeleteSelected); // separates out selected triangles to a new mesh, optionally removing them from the current mesh
	UE_API void FlipSelectedTriangles();
	UE_API void AssignNewGroupToSelectedTriangles();

	UE_API void SmoothSelectionBoundary();

	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();

	// if true, mesh has been edited
	bool bHaveModifiedMesh = false;

	UE_API virtual void ApplyShutdownAction(EToolShutdownType ShutdownType);
};



#undef UE_API
