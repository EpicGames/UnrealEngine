// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "Changes/DynamicMeshChangeTarget.h"
#include "InteractiveToolBuilder.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "MeshOpPreviewHelpers.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "Selection/SelectClickedAction.h"
#include "ToolContextInterfaces.h"

#include "MirrorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UCreateMeshObjectTypeProperties;
class UOnAcceptHandleSourcesProperties;

UCLASS(MinimalAPI)
class UMirrorToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

UENUM()
enum class EMirrorSaveMode : uint8
{
	/**  Save the results in place of the original input objects. */
	InputObjects = 0,

	/** Save the results as new objects. */
	NewObjects = 1,
};

UENUM()
enum class EMirrorOperationMode : uint8
{
	/**  Append a mirrored version of the mesh to itself. */
	MirrorAndAppend = 0,

	/** Mirror the existing mesh. */
	MirrorExisting = 1,
};

UENUM()
enum class EMeshMirrorWeldNormalMode : uint8
{
	/** Normals are split and mirrored across the plane. */
	MirrorNormals = 0,

	/** Normals are averaged with their mirrored normal across the plane. */
	AverageMirrorNormals = 1
};

UCLASS(MinimalAPI)
class UMirrorToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** Mode of operation. */
	UPROPERTY(EditAnywhere, Category = Options)
	EMirrorOperationMode OperationMode = EMirrorOperationMode::MirrorAndAppend;

	/** Cut off everything on the back side of the mirror plane before mirroring. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bCropAlongMirrorPlaneFirst = true;

	/** Whether to locally simplify new edges created when cropping along the mirror plane. Will only simplify when doing so will not change the shape, UVs or PolyGroups. */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bCropAlongMirrorPlaneFirst"))
	bool bSimplifyAlongCrop = true;

	/** Weld vertices that lie on the mirror plane. Vertices will not be welded if doing so would give an edge more than two faces, or if they are part of a face in the plane. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "OperationMode == EMirrorOperationMode::MirrorAndAppend", EditConditionHides))
	bool bWeldVerticesOnMirrorPlane = true;

	/** The normal compute method for vertices welded along the mirror plane. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "OperationMode == EMirrorOperationMode::MirrorAndAppend && bWeldVerticesOnMirrorPlane", EditConditionHides))
	EMeshMirrorWeldNormalMode WeldVerticesNormalMode = EMeshMirrorWeldNormalMode::MirrorNormals;

	/** Distance (in unscaled mesh space) to allow a point to be from the plane and still consider it "on the mirror plane". */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "OperationMode == EMirrorOperationMode::MirrorAndAppend && bWeldVerticesOnMirrorPlane", EditConditionHides,
		UIMin = 0, UIMax = 0.01, ClampMin = 0, ClampMax = 10))
	double PlaneTolerance = KINDA_SMALL_NUMBER;

	/** When welding, whether to allow bowtie vertices to be created, or to duplicate the vertex. */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (
		EditCondition = "bWeldVerticesOnMirrorPlane && OperationMode == EMirrorOperationMode::MirrorAndAppend", EditConditionHides))
	bool bAllowBowtieVertexCreation = false;
	
	/** Whether to show the preview. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowPreview = true;

	/** How to save the result. */
	UPROPERTY(EditAnywhere, Category = OutputOptions)
	EMirrorSaveMode WriteTo = EMirrorSaveMode::InputObjects;
};


UCLASS(MinimalAPI)
class UMirrorOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UMirrorTool> MirrorTool;

	/** Index of the component within MirrorTool->ComponentTargets that this factory creates an operator for. */
	int ComponentIndex;
};

UENUM()
enum class EMirrorToolAction
{
	NoAction,

	ShiftToCenter,

	Left,
	Right,
	Up,
	Down,
	Forward,
	Backward
};

UCLASS(MinimalAPI)
class UMirrorToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMirrorTool> ParentTool;

	void Initialize(UMirrorTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EMirrorToolAction Action);

	/** Move the mirror plane to center of bounding box without changing its normal. */
	UFUNCTION(CallInEditor, Category = RepositionPlane)
	void ShiftToCenter() { PostAction(EMirrorToolAction::ShiftToCenter); }

	/** If true the "Preset Mirror Directions" buttons only change the plane orientation, not location. */
	UPROPERTY(EditAnywhere, Category = PresetMirrorDirections)
	bool bButtonsOnlyChangeOrientation = false;

	/** Move the mirror plane and adjust its normal to mirror entire selection leftward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 1))
	void Left() { PostAction(EMirrorToolAction::Left); }

	/** Move the mirror plane and adjust its normal to mirror entire selection rightward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 2))
	void Right() { PostAction(EMirrorToolAction::Right); }

	/** Move the mirror plane and adjust its normal to mirror entire selection upward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 3))
	void Up() { PostAction(EMirrorToolAction::Up); }

	/** Move the mirror plane and adjust its normal to mirror entire selection downward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 4))
	void Down() { PostAction(EMirrorToolAction::Down); }

	/** Move the mirror plane and adjust its normal to mirror entire selection forward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 5))
	void Forward() { PostAction(EMirrorToolAction::Forward); }

	/** Move the mirror plane and adjust its normal to mirror entire selection backward. */
	UFUNCTION(CallInEditor, Category = PresetMirrorDirections, meta = (DisplayPriority = 6))
	void Backward() { PostAction(EMirrorToolAction::Backward); }
};

/** Tool for mirroring one or more meshes across a plane. */
UCLASS(MinimalAPI)
class UMirrorTool : public UMultiSelectionMeshEditingTool, public IModifierToggleBehaviorTarget
{
	GENERATED_BODY()
public:

	friend UMirrorOperatorFactory;

	UE_API UMirrorTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API void RequestAction(EMirrorToolAction ActionType);

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IClickSequenceBehaviorTarget implementation
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
protected:

	UPROPERTY()
	TObjectPtr<UMirrorToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UOnAcceptHandleSourcesProperties> HandleSourcesProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UMirrorToolActionPropertySet> ToolActions = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UDynamicMeshReplacementChangeTarget>> MeshesToMirror;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;

	FVector3d MirrorPlaneOrigin = FVector3d::Zero();
	FVector3d MirrorPlaneNormal = FVector3d::UnitZ();

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic;

	EMirrorToolAction PendingAction;
	FBox CombinedBounds;
	UE_API void ApplyAction(EMirrorToolAction ActionType);

	UE_API void SetupPreviews();
	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);

private:
	UE_API void CheckAndDisplayWarnings();
};

#undef UE_API
