// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Changes/TransformChange.h"
#include "FrameTypes.h"
#include "BoxTypes.h"
#include "ToolDataVisualizer.h"
#include "TransformTypes.h"
#include "PatternTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UBaseAxisTranslationGizmo;
class UAxisAngleGizmo;
class UDragAlignmentMechanic;
class UCombinedTransformGizmo;
class UTransformProxy;
class UPreviewGeometry;
class UDynamicMesh;
class UDynamicMeshComponent;
class UInteractiveGizmo;
class UStaticMesh;
class UStaticMeshComponent;
class UMaterialInterface;
class UConstructionPlaneMechanic;
class UComponentBoundTransformProxy;


/**
 *
 */
UCLASS(MinimalAPI)
class UPatternToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const override;

	bool bEnableCreateISMCs = true;
	
protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};




UENUM()
enum class EPatternToolShape : uint8
{
	/** Arrange pattern elements along a Line */
	Line = 0,

	/** Arrange pattern elements in a 2D Grid */
	Grid = 1,

	/** Arrange pattern elements in a Circle */
	Circle = 2
};


UENUM()
enum class EPatternToolSingleAxis : uint8
{
	XAxis = 0,
	YAxis = 1,
	ZAxis = 2
};

UENUM()
enum class EPatternToolSinglePlane : uint8
{
	XYPlane = 0,
	XZPlane = 1,
	YZPlane = 2
};



UENUM()
enum class EPatternToolAxisSpacingMode : uint8
{
	/** Place a specific number of Pattern Elements along the pattern geometry */
	ByCount = 0,

	/** Place Pattern Elements at regular increments along the Pattern Geometry (on-center) */
	StepSize = 1,

	/** Pack in as many Pattern Elements as fits in the available space */
	Packed = 2
};


/**
 * Settings for the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternToolSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The seed used to introduce random transform variations when enabled */
	UPROPERTY(EditAnywhere, Category = General, meta = (NoResetToDefault, Delta = 1, LinearDeltaSensitivity = 50))
	int32 Seed = FMath::Rand();

	/** Whether or not the pattern items should be projected along the negative Z axis of the plane mechanic */
	UPROPERTY(EditAnywhere, Category = General)
	bool bProjectElementsDown = false;

	/** How much each pattern item should be moved along the negative Z axis of the plane mechanic if Project Elements Down is enabled */
	UPROPERTY(EditAnywhere, Category = General, meta = (EditCondition = "bProjectElementsDown == true", EditConditionHides, Delta = 0.1, LinearDeltaSensitivity = 1))
	float ProjectionOffset = 0.0f;

	/** Hide the source meshes when enabled */
	UPROPERTY(EditAnywhere, Category = General)
	bool bHideSources = true;

	/** If false, all pattern elements will be positioned at the origin of the first pattern element */
	UPROPERTY(EditAnywhere, Category = General)
	bool bUseRelativeTransforms = true;

	/** Whether to randomly pick which source mesh is scattered at each location, or to always use all source meshes */
	UPROPERTY(EditAnywhere, Category = General)
	bool bRandomlyPickElements = false;
	
	/** Shape of the underlying Pattern */
	UPROPERTY(EditAnywhere, Category = Shape)
	EPatternToolShape Shape = EPatternToolShape::Line;

	/** Axis direction used for the Pattern geometry */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (DisplayName = "Direction", EditCondition = "Shape == EPatternToolShape::Line", EditConditionHides))
	EPatternToolSingleAxis SingleAxis = EPatternToolSingleAxis::XAxis;

	/** Plane used for the Pattern geometry */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (DisplayName = "Plane", EditCondition = "Shape != EPatternToolShape::Line", EditConditionHides))
	EPatternToolSinglePlane SinglePlane = EPatternToolSinglePlane::XYPlane;
};

/**
 * Settings for Bounding Box adjustments in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_BoundingBoxSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** If true, pattern element bounding boxes are not changed to account for StartScale or StartRotation */
	UPROPERTY(EditAnywhere, Category = BoundingBox)
	bool bIgnoreTransforms = false;
	
	/** Value added to the all pattern elements' bounding boxes for adjusting the behavior of packed spacing mode manually */
	UPROPERTY(EditAnywhere, Category = BoundingBox, meta = (Delta = 0.1, LinearDeltaSensitivity = 1))
	float Adjustment = 0.0f;

	/** If true, the bounding boxes of each element are rendered in green and the combined bounding box of all source elements is rendered in red */
	UPROPERTY(EditAnywhere, Category = BoundingBox)
	bool bVisualize = false;
};
	
/**
 * Settings for Linear Patterns in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_LinearSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Spacing Technique used to distribute Pattern Elements */
	UPROPERTY(EditAnywhere, Category = LinearPattern)
	EPatternToolAxisSpacingMode SpacingMode = EPatternToolAxisSpacingMode::ByCount;

	/** Number of Pattern Elements to place */
	UPROPERTY(EditAnywhere, Category = LinearPattern, meta = (ClampMin = 1, UIMax = 25, EditCondition = "SpacingMode == EPatternToolAxisSpacingMode::ByCount", EditConditionHides))
	int32 Count = 10;

	/** Fixed Increment used to place Pattern Elements */
	UPROPERTY(EditAnywhere, Category = LinearPattern, meta = (ClampMin = 0, EditCondition = "SpacingMode == EPatternToolAxisSpacingMode::StepSize", EditConditionHides))
	double StepSize = 100.0;

	/** Length of Pattern along the Axis */
	UPROPERTY(EditAnywhere, Category = LinearPattern, meta = (ClampMin = 0))
	double Extent = 1000.0;

	/** If true, Pattern is centered at the Origin, otherwise Pattern starts at the Origin */
	UPROPERTY(EditAnywhere, Category = LinearPattern)
	bool bCentered = true;
};




/**
 * Settings for Grid Patterns in the Pattern Tool
 * TODO: maybe we can just re-use UPatternTool_LinearSettings for this??
 */
UCLASS(MinimalAPI)
class UPatternTool_GridSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Spacing Technique used to distribute Pattern Elements along the Main axis */
	UPROPERTY(EditAnywhere, Category = GridPatternX)
	EPatternToolAxisSpacingMode SpacingX = EPatternToolAxisSpacingMode::ByCount;

	/** Number of Pattern Elements to place along the Main axis */
	UPROPERTY(EditAnywhere, Category = GridPatternX, meta = (ClampMin = 1, UIMax = 25, EditCondition = "SpacingX == EPatternToolAxisSpacingMode::ByCount", EditConditionHides))
	int32 CountX = 10;

	/** Fixed Increment used to place Pattern Elements along the Main axis */
	UPROPERTY(EditAnywhere, Category = GridPatternX, meta = (ClampMin = 0, EditCondition = "SpacingX == EPatternToolAxisSpacingMode::StepSize", EditConditionHides))
	double StepSizeX = 100.0;

	/** Length/Extent of Pattern falong the Main Axis */
	UPROPERTY(EditAnywhere, Category = GridPatternX, meta = (ClampMin = 0))
	double ExtentX = 1000.0;

	/** If true, Pattern is centered at the Origin along the Main axis, otherwise Pattern starts at the Origin */
	UPROPERTY(EditAnywhere, Category = GridPatternX)
	bool bCenteredX = true;

	/** Spacing Technique used to distribute Pattern Elements along the Secondary axis*/
	UPROPERTY(EditAnywhere, Category = GridPatternY)
	EPatternToolAxisSpacingMode SpacingY = EPatternToolAxisSpacingMode::ByCount;

	/** Number of  Pattern Elements to place along the Secondary axis */
	UPROPERTY(EditAnywhere, Category = GridPatternY, meta = (ClampMin = 1, UIMax = 25, EditCondition = "SpacingY == EPatternToolAxisSpacingMode::ByCount", EditConditionHides))
	int32 CountY = 10;

	/** Fixed Increment used to place Pattern Elements along the Secondary axis */
	UPROPERTY(EditAnywhere, Category = GridPatternY, meta = (ClampMin = 0, EditCondition = "SpacingY == EPatternToolAxisSpacingMode::StepSize", EditConditionHides))
	double StepSizeY = 100.0;

	/** Length/Extent of Pattern falong the Secondary Axis */
	UPROPERTY(EditAnywhere, Category = GridPatternY, meta = (ClampMin = 0))
	double ExtentY = 1000.0;

	/** If true, Pattern is centered at the Origin along the Secondary axis, otherwise Pattern starts at the Origin */
	UPROPERTY(EditAnywhere, Category = GridPatternY)
	bool bCenteredY = true;
};





/**
 * Settings for Radial Patterns in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_RadialSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Spacing Technique used to distribute Pattern Elements around the Circle/Arc */
	UPROPERTY(EditAnywhere, Category = RadialPattern)
	EPatternToolAxisSpacingMode SpacingMode = EPatternToolAxisSpacingMode::ByCount;

	/** Number of  Pattern Elements to place */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (ClampMin = 1, UIMax = 25, EditCondition = "SpacingMode == EPatternToolAxisSpacingMode::ByCount", EditConditionHides))
	int32 Count = 10;

	/** Fixed Increment (in Degrees) used to position Pattern Elements around the Circle/Arc */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (Units = "Degrees", ClampMin = 0, EditCondition = "SpacingMode == EPatternToolAxisSpacingMode::StepSize", EditConditionHides))
	double StepSize = 100.0;

	/** Radius of the Circle/Arc */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (ClampMin = 0))
	double Radius = 250;

	/** Start angle of the Circle/Arc */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (Units = "Degrees", UIMin = -360, UIMax = 360))
	double StartAngle = 0.0;

	/** End angle of the Circle/Arc */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (Units = "Degrees", UIMin = -360, UIMax = 360))
	double EndAngle = 360.0;

	/** Fixed offset added to Start/End Angles */
	UPROPERTY(EditAnywhere, Category = RadialPattern, meta = (Units = "Degrees", ClampMin = -180, ClampMax = 180))
	double AngleShift = 0.0;

	/** If true, Pattern elements are rotated to align with the Circle tangent */
	UPROPERTY(EditAnywhere, Category = RadialPattern)
	bool bOriented = true;
};




/**
 * Settings for Per Element Rotation in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_RotationSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** If true, Rotation is linearly interpolated between StartRotation and Rotation values */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (InlineEditConditionToggle))
	bool bInterpolate = false;

	/** If true, Rotation at each Pattern Element is offset by a uniformly chosen random value in the range of [-RotationJitterRange, RotationJitterRange] */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (InlineEditConditionToggle))
	bool bJitter = false;

	/** Rotation applied to all Pattern Elements, or to first Pattern Element for Interpolated rotation */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (UIMin = -360, UIMax = 360))
	FRotator StartRotation = FRotator::ZeroRotator;

	/** Rotation applied to last Pattern Elements for Interpolated rotation */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (EditCondition = "bInterpolate", UIMin = -360, UIMax = 360))
	FRotator EndRotation = FRotator::ZeroRotator;

	/** Upper bound of the range which is sampled to randomly rotate each Pattern Element if Jitter is true */
	UPROPERTY(EditAnywhere, Category = Rotation, meta = (ClampMin = 0, EditCondition = "bJitter", UIMin = -360, UIMax = 360))
	FRotator Jitter = FRotator::ZeroRotator;
};


/**
 * Settings for Per Element Translation in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_TranslationSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** If true, Translation is linearly interpolated between StartTranslation and Translation values */
	UPROPERTY(EditAnywhere, Category = Translation, meta = (InlineEditConditionToggle))
	bool bInterpolate = false;

	/** If true, Translation at each Pattern Element is offset by a uniformly chosen random value in the range of [-TranslationJitterRange, TranslationJitterRange] */
	UPROPERTY(EditAnywhere, Category = Translation, meta = (InlineEditConditionToggle))
	bool bJitter = false;

	/** Translation applied to all Pattern Elements, or to first Pattern Element for Interpolated translation */
	UPROPERTY(EditAnywhere, Category = Translation)
	FVector StartTranslation = FVector::ZeroVector;

	/** Translation applied to last Pattern Element for Interpolated translation */
	UPROPERTY(EditAnywhere, Category = Translation, meta = (EditCondition = "bInterpolate"))
	FVector EndTranslation = FVector::ZeroVector;

	/** Upper bound of the range which is sampled to randomly translate each Pattern Element if Jitter is true */
	UPROPERTY(EditAnywhere, Category = Translation, meta = (ClampMin = 0, EditCondition = "bJitter"))
	FVector Jitter = FVector::ZeroVector;
};

/**
 * Settings for Per Element Scale in the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_ScaleSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Initial value for Jitter and referenced in FPatternGenerator when applying scale jitter. Can't be used for ClampMin. */
	static constexpr double MinScale = 0.001;
	
	/** If true, changes to Start Scale, End Scale, and Jitter are proportional along all the axes */
	UPROPERTY(EditAnywhere, Category = Scale)
	bool bProportional = true;
	
	/** If true, Scale is linearly interpolated between StartScale and Scale values */
	UPROPERTY(EditAnywhere, Category = Scale, meta = (InlineEditConditionToggle))
	bool bInterpolate = false;

	/** If true, Scale at each Pattern Element is offset by a uniformly chosen random value in the range of [-ScaleJitterRange, ScaleJitterRange] */
	UPROPERTY(EditAnywhere, Category = Scale, meta = (InlineEditConditionToggle))
	bool bJitter = false;
	
	/** Scale applied to all Pattern Elements, or to first Pattern Element for Interpolated scale */
	UPROPERTY(EditAnywhere, Category = Scale, meta = (ClampMin = 0.001, Delta = 0.01, LinearDeltaSensitivity = 1))
	FVector StartScale = FVector::OneVector;
	
	/** Scale applied to last Pattern Element for Interpolated scale */
	UPROPERTY(EditAnywhere, Category = Scale, meta = (ClampMin = 0.001, EditCondition = "bInterpolate", Delta = 0.01, LinearDeltaSensitivity = 1))
	FVector EndScale = FVector::OneVector;

	/** Upper bound of the range which is sampled to randomly scale each Pattern Element if Jitter is true */
	UPROPERTY(EditAnywhere, Category = Scale, meta = (ClampMin = 0.001, EditCondition = "bJitter", Delta = 0.01, LinearDeltaSensitivity = 1))
	FVector Jitter = FVector(MinScale);
};




/**
 * Output Settings for the Pattern Tool
 */
UCLASS(MinimalAPI)
class UPatternTool_OutputSettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Emit a separate Actor for each pattern element */
	UPROPERTY(EditAnywhere, Category = Output)
	bool bSeparateActors = false;

	/** Emit StaticMesh pattern elements as DynamicMeshes */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition = "bHaveStaticMeshes == true", HideEditConditionToggle))
	bool bConvertToDynamic = false;

	/** Create InstancedStaticMeshComponents instead multiple StaticMeshComponents, for StaticMesh pattern elements */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition = "bHaveStaticMeshes == true && bSeparateActors == false && bConvertToDynamic == false && bEnableCreateISMCs == true", HideEditConditionToggle))
	bool bCreateISMCs = false;

	/** internal, used to control state of Instance settings */
	UPROPERTY(meta = (TransientToolProperty))
	bool bHaveStaticMeshes = false;

	// internal, used to disable the creation of ISMCs
	UPROPERTY(meta = (TransientToolProperty))
	bool bEnableCreateISMCs = true;
};



/**
 * UPatternTool takes input meshes and generates 3D Patterns of those meshes, by
 * placing repeated copies along geometric paths like lines, grids, circles, etc.
 * The output can be a single Actor per pattern Element, or combined into single
 * Actors in various ways depending on the input mesh type. 
 */
UCLASS(MinimalAPI)
class UPatternTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()

public:
	UE_API UPatternTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API virtual void SetEnableCreateISMCs(bool bEnable);

public:
	UPROPERTY()
	TObjectPtr<UPatternToolSettings> Settings;

	UPROPERTY()
	TObjectPtr<UPatternTool_BoundingBoxSettings> BoundingBoxSettings;
	
	UPROPERTY()
	TObjectPtr<UPatternTool_LinearSettings> LinearSettings;

	UPROPERTY()
	TObjectPtr<UPatternTool_GridSettings> GridSettings;

	UPROPERTY()
	TObjectPtr<UPatternTool_RadialSettings> RadialSettings;

	UPROPERTY()
	TObjectPtr<UPatternTool_RotationSettings> RotationSettings;

	UPROPERTY()
	TObjectPtr<UPatternTool_TranslationSettings> TranslationSettings;

	UPROPERTY()
	TObjectPtr<UPatternTool_ScaleSettings> ScaleSettings;
	
	FVector CachedStartScale;
	FVector CachedEndScale;
	FVector CachedJitterScale;
	
	int32 StartScaleWatcherIdx;
	int32 EndScaleWatcherIdx;
	int32 JitterScaleWatcherIdx;

	UPROPERTY()
	TObjectPtr<UPatternTool_OutputSettings> OutputSettings;

protected:
	/**
	 * Pattern Gizmo:
	 */

	UPROPERTY()
	TObjectPtr<UComponentBoundTransformProxy> PatternGizmoProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UInteractiveGizmo> PatternGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> PatternGizmoComponent = nullptr;

	// If true, Settings->SingleAxis is being used. If false, Settings->SinglePlane is being used
	bool bUsingSingleAxis;
	
	int32 LinearExtentWatcherIdx;
	int32 GridExtentXWatcherIdx;
	int32 GridExtentYWatcherIdx;
	int32 RadiusWatcherIdx;

	UE_API void OnTransformGizmoUpdated(UTransformProxy* Proxy, FTransform Transform);
	UE_API void ResetTransformGizmoPosition();
	UE_API void ReconstructTransformGizmos();
	
	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	UE::Geometry::FFrame3d CurrentStartFrameWorld;
	TArray<UE::Geometry::FTransformSRT3d> CurrentPattern;

	bool bPatternNeedsUpdating = false;
	FDateTime LastPatternUpdateTime;
	UE_API void MarkPatternDirty();

	UE_API void OnSourceVisibilityToggled(bool bVisible);
	UE_API void OnMainFrameUpdated();
	UE_API void OnShapeUpdated();
	UE_API void OnSingleAxisUpdated();
	UE_API void OnSinglePlaneUpdated();
	UE_API void OnSpacingModeUpdated();
	UE_API void OnParametersUpdated();
	UE_API void UpdatePattern();
	UE_API void ComputeWorldTransform(FTransform& OutWorldTransform, const FTransform& InElementTransform, const FTransform& InPatternTransform) const;
	UE_API void GetPatternTransforms_Linear(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut);
	UE_API void GetPatternTransforms_Grid(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut);
	UE_API void GetPatternTransforms_Radial(TArray<UE::Geometry::FTransformSRT3d>& TransformsOut);

	UE_API void RenderBoundingBoxes(IToolsContextRenderAPI* RenderAPI);
	FToolDataVisualizer BoundingBoxVisualizer;
	
	struct FPatternElement
	{
		int32 TargetIndex = 0;

		// todo: This is no longer necessary now that InitializeNewTool filters out invalid targets.
		bool bValid = true;

		UPrimitiveComponent* SourceComponent = nullptr;
		TArray<UMaterialInterface*> SourceMaterials;
		UE::Geometry::FTransformSRT3d SourceTransform = UE::Geometry::FTransformSRT3d::Identity();
		UE::Geometry::FTransformSRT3d BaseRotateScale = UE::Geometry::FTransformSRT3d::Identity();

		// We don't need to store rotation or scale relative to first
		// element because that is handled by BaseRotateScale
		FVector3d RelativePosition = FVector3d::ZeroVector;

		UDynamicMesh* SourceDynamicMesh = nullptr;
		UStaticMesh* SourceStaticMesh = nullptr;

		UE::Geometry::FAxisAlignedBox3d LocalBounds = UE::Geometry::FAxisAlignedBox3d::Empty();			// The unchanged bounding box of the source mesh. Only used indirectly through PatternBounds
		UE::Geometry::FAxisAlignedBox3d PatternBounds = UE::Geometry::FAxisAlignedBox3d::Empty();		// The bounding box used for computing CombinedPatternBounds for packed spacing. By default this box adapts to StartScale/StartRotation
	};
	TArray<FPatternElement> Elements;
	UE::Geometry::FAxisAlignedBox3d CombinedPatternBounds = UE::Geometry::FAxisAlignedBox3d::Empty();	// The bounding box that contains all of the elements' PatternBounds bounding boxes. Used for packed mode spacing

	UE_API void ComputePatternBounds(int32 ElemIdx);
	UE_API void ComputeCombinedPatternBounds();

	// Given an Element index and an FTransformSRT3d, determine the bounding box that contains the transformed underlying mesh
	// BoundingBox is made empty before growing to contain the transformed mesh.
	UE_API void ComputeBoundingBoxWithTransform(int32 ElemIdx, UE::Geometry::FAxisAlignedBox3d& BoundingBox, const UE::Geometry::FTransformSRT3d& Transform);
	
	bool bHaveNonUniformScaleElements = false;

	bool bEnableCreateISMCs = true;

	struct FComponentSet
	{
		TArray<UPrimitiveComponent*> Components;
	};
	TArray<FComponentSet> PreviewComponents;
	
	// This duplicates the data stored by PreviewComponents but it is necessary to have a simple
	// TArray<UPrimitiveComponent*> of all preview components when raycasting, otherwise the raycasts
	// will hit the preview components from the previous frame.
	TArray<const UPrimitiveComponent*> AllPreviewComponents;			// This is passed to FindNearestVisibleObjectHit as ComponentsToIgnore

protected:
	UPROPERTY()
	TSet<TObjectPtr<UPrimitiveComponent>> AllComponents;		// to keep components in FComponentSet alive

	
	TMap<int32, FComponentSet> StaticMeshPools;
	UE_API UStaticMeshComponent* GetPreviewStaticMesh(const FPatternElement& Element);
	UE_API void ReturnStaticMeshes(FPatternElement& Element, FComponentSet& ComponentSet);

	TMap<int32, FComponentSet> DynamicMeshPools;
	UE_API UDynamicMeshComponent* GetPreviewDynamicMesh(const FPatternElement& Element);
	UE_API void ReturnDynamicMeshes(FPatternElement& Element, FComponentSet& ComponentSet);

	UE_API void HideReturnedPreviewMeshes();


	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;	// parent actor for all preview components


	UE_API void InitializeElements();
	UE_API void ResetPreviews();
	UE_API void DestroyPreviews();

	UE_API void EmitResults();
};

#undef UE_API
