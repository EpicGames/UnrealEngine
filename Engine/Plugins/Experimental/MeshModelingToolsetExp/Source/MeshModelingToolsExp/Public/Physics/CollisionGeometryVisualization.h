// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h" // Required by MESHMODELINGTOOLSEXP_API

class FPhysicsDataCollection;
class UCollisionGeometryVisualizationProperties;
class UPreviewGeometry;
class UMaterialInterface;

namespace UE::PhysicsTools
{
	/**
	 * Create triangle sets (solids) and line sets (wireframes) in a UPreviewGeometry for all the elements in a Physics Data Collection.
	 * In line sets, Spheres and Capsules are drawn as 3-axis wireframes. Convexes are added as wireframes.
	 * By default, clear existing lines and triangles from the preview
	 */
	void MESHMODELINGTOOLSEXP_API InitializeCollisionGeometryVisualization(
		UPreviewGeometry* PreviewGeom,
		UCollisionGeometryVisualizationProperties* Settings,
		const FPhysicsDataCollection& PhysicsData,
		float DepthBias = 0.f,
		int32 CircleStepResolution = 16,
		bool bClearExistingLinesAndTriangles = true);

	/**
	 * Update line sets in a UPreviewGeometry.
	 */
	void MESHMODELINGTOOLSEXP_API UpdateCollisionGeometryVisualization(
		UPreviewGeometry* PreviewGeom,
		UCollisionGeometryVisualizationProperties* Settings);

	// The following functions are like InitializeCollisionGeometryVisualization/UpdateCollisionGeometryVisualization
	// but assume that the given UPreviewGeometry only represents part of the complete collision geometry visualization

	/**
	 * Like InitializeCollisionGeometryVisualization but only initializes part of the collision geometry visualization.
	 * 
	 * The ColorIndex parameter is used to implement Settings->bRandomColors since pseudorandom colors are determined
	 * using unique integers for each line/triangle set, which may be stored in separate UPreviewGeometry objects. Since the
	 * initialization is partial this function doesn't set Settings->bVisualizationDirty, the caller should do that.
	 */
	void MESHMODELINGTOOLSEXP_API PartiallyInitializeCollisionGeometryVisualization(
		UPreviewGeometry* PartialPreviewGeom,
		UCollisionGeometryVisualizationProperties* Settings,
		const FPhysicsDataCollection& PhysicsData,
		int32 ColorIndex,
		float DepthBias = 0.f,
		int32 CircleStepResolution = 16);

	/**
	 * Like UpdateCollisionGeometryVisualization but only updates part of the collision geometry visualization.
	 * 
	 * The ColorIndex parameter is used implement Settings->bRandomColors since pseudorandom colors are determined
	 * using unique integers for each line/triangle set, which may be stored in separate UPreviewGeometry objects. Since the
	 * update is partial this function doesn't check or set Settings->bVisualizationDirty, the caller should do that. 
	 */
	void MESHMODELINGTOOLSEXP_API PartiallyUpdateCollisionGeometryVisualization(
		UPreviewGeometry* PartialPreviewGeom,
		UCollisionGeometryVisualizationProperties* Settings,
		int32 ColorIndex);
}


