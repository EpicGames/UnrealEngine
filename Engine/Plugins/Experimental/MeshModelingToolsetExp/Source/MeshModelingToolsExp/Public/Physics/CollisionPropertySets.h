// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BodySetupEnums.h"
#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "CollisionPropertySets.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


class FPhysicsDataCollection;


UENUM()
enum class ECollisionGeometryMode
{
	/** Use project physics settings (DefaultShapeComplexity) */
	Default = CTF_UseDefault,
	/** Create both simple and complex shapes. Simple shapes are used for regular scene queries and collision tests. Complex shape (per poly) is used for complex scene queries.*/
	SimpleAndComplex = CTF_UseSimpleAndComplex,
	/** Create only simple shapes. Use simple shapes for all scene queries and collision tests.*/
	UseSimpleAsComplex = CTF_UseSimpleAsComplex,
	/** Create only complex shapes (per poly). Use complex shapes for all scene queries and collision tests. Can be used in simulation for static shapes only (i.e can be collided against but not moved through forces or velocity.) */
	UseComplexAsSimple = CTF_UseComplexAsSimple
};


USTRUCT()
struct FPhysicsSphereData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	float Radius = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FTransform Transform;

	/** Shape Element storing standard collision and physics properties for a shape */
	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FKShapeElem Element;
};

USTRUCT()
struct FPhysicsBoxData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FVector Dimensions = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FTransform Transform;

	/** Shape Element storing standard collision and physics properties for a shape */
	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FKShapeElem Element;
};

USTRUCT()
struct FPhysicsCapsuleData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	float Radius = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	float Length = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FTransform Transform;

	/** Shape Element storing standard collision and physics properties for a shape */
	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FKShapeElem Element;
};

USTRUCT()
struct FPhysicsConvexData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Convex)
	int32 NumVertices = 0;

	UPROPERTY(VisibleAnywhere, Category = Convex)
	int32 NumFaces = 0;

	/** Shape Element storing standard collision and physics properties for a shape */
	UPROPERTY(VisibleAnywhere, Category = Sphere)
	FKShapeElem Element;
};

USTRUCT()
struct FPhysicsLevelSetData
{
	GENERATED_BODY()

	/** Shape Element storing standard collision and physics properties for a shape */
	UPROPERTY(VisibleAnywhere, Category = LevelSet)
	FKShapeElem Element;
};

UCLASS(MinimalAPI)
class UPhysicsObjectToolPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Source Object Name */
	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	FString ObjectName;

	/** Collision Flags controlling how simple and complex collision shapes are used */
	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	ECollisionGeometryMode CollisionType;

	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	TArray<FPhysicsSphereData> Spheres;

	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	TArray<FPhysicsBoxData> Boxes;

	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	TArray<FPhysicsCapsuleData> Capsules;

	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	TArray<FPhysicsConvexData> Convexes;

	UPROPERTY(VisibleAnywhere, Category = PhysicsData)
	TArray<FPhysicsLevelSetData> LevelSets;

	UE_API void Reset();
};




UCLASS(MinimalAPI)
class UCollisionGeometryVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Sets materials and registers property watchers to set bVisualizationDirty to true when any property changes */
	UE_API void Initialize(UInteractiveTool* Tool);

	/**
	 * Returns the line set color.
	 * If bRandomColors == true  LineSetIndex is used to choose the color
	 * If bRandomColors == false LineSetIndex is ignored and this->Color is returned
	 */
	UE_API FColor GetLineSetColor(int32 LineSetIndex = 0) const;

	/** @return The color used for solid rendering. Currently uses the same colors as the lines. */
	FColor GetTriangleSetColor(int32 TriangleSetIndex = 0) const
	{
		return GetLineSetColor(TriangleSetIndex);
	}

	UMaterialInterface* GetLineMaterial() const
	{
		ensure(LineMaterial);
		ensure(LineMaterialShowingHidden);
		return bShowHidden ? LineMaterialShowingHidden : LineMaterial;
	}

	/** @return the material to use for triangle rendering, when bShowSolid is true */
	UMaterialInterface* GetSolidMaterial() const
	{
		ensure(TriangleMaterial);
		return TriangleMaterial;
	}
	
	/** Show/hide collision geometry */
	UPROPERTY(EditAnywhere, Category = "Collision Visualization",
		meta = (EditCondition = "bEnableShowCollision", EditConditionHides, HideEditConditionToggle))
	bool bShowCollision = true;

	/** Whether to show solid shapes in addition to wireframes */
	UPROPERTY(EditAnywhere, Category = "Collision Visualization", 
		meta = (EditCondition = "bEnableShowSolid", EditConditionHides, HideEditConditionToggle))
	bool bShowSolid = false;

	/** Thickness of lines used to visualize collision shapes */
	UPROPERTY(EditAnywhere, Category = "Collision Visualization")
	float LineThickness = 3.0f;

	/** Show occluded parts of the collision geometry, rendered with dashed lines */
	UPROPERTY(EditAnywhere, DisplayName = "Show Hidden Lines", Category = "Collision Visualization")
	bool bShowHidden = false;

	/** Render each collision geometry with a randomly-chosen color */
	UPROPERTY(EditAnywhere, Category = "Collision Visualization")
	bool bRandomColors = true;

	/** The color to use for all collision visualizations, if random colors are not used */
	UPROPERTY(EditAnywhere, Category = "Collision Visualization",
		meta = (EditCondition = "!bRandomColors"))
	FColor Color = FColor::Red;

	//~Used if bShowHidden is false
	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> LineMaterial = nullptr;

	//~Used if bShowHidden is true
	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> LineMaterialShowingHidden = nullptr;

	//~Used if bShowHidden is false
	UPROPERTY(Transient, meta = (TransientToolProperty))
	TObjectPtr<UMaterialInterface> TriangleMaterial = nullptr;

	//~Some tools will want showing collision geometry to be non-optional
	UPROPERTY(Transient, meta=(TransientToolProperty))
	bool bEnableShowCollision = true;

	//~Some tools will not want the 'show solid' option
	UPROPERTY(Transient, meta = (TransientToolProperty))
	bool bEnableShowSolid = true;

	bool bVisualizationDirty = false;
};




namespace UE
{
	namespace PhysicsTools
	{
		void InitializePhysicsToolObjectPropertySet(const FPhysicsDataCollection* PhysicsData, UPhysicsObjectToolPropertySet* PropSet);
	}
}

#undef UE_API
