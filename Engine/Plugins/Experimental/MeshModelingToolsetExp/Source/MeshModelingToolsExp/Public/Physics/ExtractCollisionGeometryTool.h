// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Physics/CollisionPropertySets.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "ExtractCollisionGeometryTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewGeometry;
class UPreviewMesh;

UCLASS(MinimalAPI)
class UExtractCollisionGeometryToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class EExtractCollisionOutputType : uint8
{
	/** Simple Collision shapes (Box, Sphere, Capsule, Convex) */
	Simple = 0,
	/** Complex Collision Mesh */
	Complex = 1
};


UCLASS(MinimalAPI)
class UExtractCollisionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Type of collision geometry to convert to Mesh */
	UPROPERTY(EditAnywhere, Category = Options)
	EExtractCollisionOutputType CollisionType = EExtractCollisionOutputType::Simple;

	/** Whether or not to generate a seperate Mesh Object for each Simple Collision Shape  */
	UPROPERTY(EditAnywhere, Category = Options, Meta = (EditCondition = "CollisionType == EExtractCollisionOutputType::Simple"))
	bool bOutputSeparateMeshes = true;

	/** Show/Hide a preview of the generated mesh (overlaps source mesh) */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowPreview = false;

	/** Show/Hide input mesh */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "CollisionType != EExtractCollisionOutputType::Complex || !bShowPreview"))
	bool bShowInputMesh = true;

	/** Whether or not to weld coincident border edges of the Complex Collision Mesh (if possible) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Options, Meta = (EditCondition = "CollisionType == EExtractCollisionOutputType::Complex"))
	bool bWeldEdges = true;
};



/**
 * Mesh Inspector Tool for visualizing mesh information
 */
UCLASS(MinimalAPI)
class UExtractCollisionGeometryTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

protected:

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UExtractCollisionToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UCollisionGeometryVisualizationProperties> VizSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UPhysicsObjectToolPropertySet> ObjectProps;

protected:
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewElements;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	// these are TSharedPtr because TPimplPtr cannot currently be added to a TArray?
	TSharedPtr<FPhysicsDataCollection> PhysicsInfo;

	TArray<TSharedPtr<UE::Geometry::FDynamicMesh3>> CurrentMeshParts;
	UE::Geometry::FDynamicMesh3 CurrentMesh;
	bool bResultValid = false;
	UE_API void RecalculateMesh_Simple();
	UE_API void RecalculateMesh_Complex();
};

#undef UE_API
