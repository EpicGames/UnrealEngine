// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "Physics/CollisionPropertySets.h"
#include "PhysicsInspectorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewGeometry;

UCLASS(MinimalAPI)
class UPhysicsInspectorToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UCLASS(MinimalAPI)
class UPhysicsInspectorToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Show/Hide target mesh */
	UPROPERTY(EditAnywhere, Category = TargetVisualization)
	bool bShowTargetMesh = true;
};


/**
 * Mesh Inspector Tool for visualizing mesh information
 */
UCLASS(MinimalAPI)
class UPhysicsInspectorTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

	UE_API void OnCreatePhysics(UActorComponent* Component);

protected:

	UPROPERTY()
	TObjectPtr<UCollisionGeometryVisualizationProperties> VizSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UPhysicsInspectorToolProperties> Settings = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UPhysicsObjectToolPropertySet>> ObjectData;

protected:
	UPROPERTY()
	TArray<TObjectPtr<UPreviewGeometry>> PreviewElements;

private:
	// Helper to create or re-create preview geometry
	UE_API void InitializePreviewGeometry(bool bClearExisting);
	// Delegate to track when physics data may have been updated
	FDelegateHandle OnCreatePhysicsDelegateHandle;
	// A flag to track when the preview geometry needs to be re-initialized
	bool bUnderlyingPhysicsObjectsUpdated = false;
};

#undef UE_API
