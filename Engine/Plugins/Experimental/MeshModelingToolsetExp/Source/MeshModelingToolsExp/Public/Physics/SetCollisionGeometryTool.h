// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "SphereTypes.h"
#include "OrientedBoxTypes.h"
#include "CapsuleTypes.h"
#include "Physics/CollisionPropertySets.h"
#include "Physics/PhysicsDataCollection.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "Selections/GeometrySelection.h"
#include "TransformSequence.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "SetCollisionGeometryTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;
PREDECLARE_GEOMETRY(class FMeshSimpleShapeApproximation)
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

UCLASS(MinimalAPI)
class USetCollisionGeometryToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};




UENUM()
enum class ESetCollisionGeometryInputMode
{
	// Compute collision geometry using a combined mesh of all input objects
	CombineAll = 0,

	// Compute collision geometry for each input object
	// Note: A Geometry Selection always counts as one input object
	PerInputObject = 1,

	// Compute collision geometry for each connected component of each input object
	PerMeshComponent = 2,

	// Compute collision geometry for each PolyGroup of each input object
	PerMeshGroup = 3
};


UENUM()
enum class ECollisionGeometryType
{
	//~ TODO: We should add a way to remove this option in single-input mode
	// Copy the existing collision geometry shapes from the inputs to the target. With a single-selection,
	// always does the same thing as Empty with Append To Existing set to true.
	CopyFromInputs = 0,
	// Fit axis-aligned bounding boxes to the inputs
	AlignedBoxes = 1,
	// Fit oriented bounding boxes to the inputs
	OrientedBoxes = 2,
	// Fit spheres to the inputs
	MinimalSpheres = 3,
	// Fit capsules to the inputs
	Capsules = 4,
	// Fit convex hulls to the inputs
	ConvexHulls = 5,
	// Fit multiple convex hulls to each input
	ConvexDecompositions = 8,
	// Fit convex hulls to 2D projections of the inputs, and sweep these 2D hulls along the projection dimension
	SweptHulls = 6,
	// Fit level sets to the inputs
	LevelSets = 7,
	// Note: ConvexDecomposition = 8 is set above, to be next to Convex Hulls in the UI
	// Fit the boxes, spheres, and capsules to the inputs, and keep the best fitting of these shapes based on volume
	MinVolume = 10,

	// Do not produce new collision for inputs. If Append To Existing is false, this gives a way
	// to empty the simple collision on the target. If Append To Existing is true, the existing collision
	// is kept and can be passed through the optional filters in the tool, like removing enclosed shapes.
	Empty = 11,
};



UENUM()
enum class EProjectedHullAxis
{
	// Project along the X axis
	X = 0,
	// Project along the Y axis
	Y = 1,
	// Project along the Z axis
	Z = 2,
	// Project along the bounding box's shortest axis
	SmallestBoxDimension = 3,
	// Project along each major axis, and take the result with the smallest volume
	SmallestVolume = 4
};

// Method to use to compute convex decomposition
UENUM()
enum class EConvexDecompositionMethod
{
	// Use the Navigable Space Protection's Error Tolerance and Min Radius settings to control the decomposition
	NavigationDriven,
	// Minimize volumetric differences to the input
	VolumetricError
};


UCLASS(MinimalAPI)
class USetCollisionGeometryToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** What kind of shapes to fit to the input. Note: Will be overridden by any enabled 'Auto Detect' settings, if close-fitting 'Auto Detect' shapes are found. */
	UPROPERTY(EditAnywhere, Category = Options)
	ECollisionGeometryType GeometryType = ECollisionGeometryType::AlignedBoxes;

	/** Whether to keep the existing collision shapes, and append new shapes to that set. Otherwise, existing collision will be cleared. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bAppendToExisting = false;

	/** When using multiple inputs to generate our collision, whether to use the world-space position of those input. If false, inputs will be considered as if they were all centered at the same location. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bUsingMultipleInputs", EditConditionHides, HideEditConditionToggle))
	bool bUseWorldSpace = false;

	/** What parts of the input should be separately fit with collision shapes */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "GeometryType != ECollisionGeometryType::Empty && GeometryType != ECollisionGeometryType::CopyFromInputs"))
	ESetCollisionGeometryInputMode InputMode = ESetCollisionGeometryInputMode::PerInputObject;

	/** Whether to attempt to detect and remove collision shapes that are fully contained inside other collision shapes */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bRemoveContained = true;

	/** Whether to discard all but MaxCount collision geometries with the largest volume */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bEnableMaxCount = false;

	/** The maximum number of collision shapes to generate. If necessary, the shapes with smallest volume will be discarded to meet this count. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "9999999", EditCondition = "bEnableMaxCount"))
	int32 MaxCount = 50;

	/** Generated collision shapes will be expanded if they are smaller than this in any dimension. Not supported for Level Sets or Convex Decompositions. */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (ClampMin = "0", UIMax = "10", EditCondition = "GeometryType != ECollisionGeometryType::LevelSets && GeometryType != ECollisionGeometryType::ConvexDecompositions"))
	float MinThickness = 0.01f;

	/** Whether to override the requested Geometry Type with a box whenever a box closely fits the input shape */
	UPROPERTY(EditAnywhere, Category = AutoDetectGeometryOverrides)
	bool bDetectBoxes = true;

	/** Whether to override the requested Geometry Type with a sphere whenever a sphere closely fits the input shape */
	UPROPERTY(EditAnywhere, Category = AutoDetectGeometryOverrides)
	bool bDetectSpheres = true;

	/** Whether to override the requested Geometry Type with a capsule whenever a capsule closely fits the input shape */
	UPROPERTY(EditAnywhere, Category = AutoDetectGeometryOverrides)
	bool bDetectCapsules = true;

	/** Whether to attempt to merge the generated collision shapes, when there are more than MergeAboveCount */
	UPROPERTY(EditAnywhere, Category = MergeCollisionShapes)
	bool bMergeCollisionShapes = false;

	/** Attempt to merge generated collision shapes until there are at most this many */
	UPROPERTY(EditAnywhere, Category = MergeCollisionShapes, meta = (EditConditionHides, EditCondition = "bMergeCollisionShapes", ClampMin = "1"))
	int32 MergeAboveCount = 1;

	/** Whether to protect negative space while merging the generated collision shapes, using the negative space settings */
	UPROPERTY(EditAnywhere, Category = MergeCollisionShapes, meta = (EditConditionHides, EditCondition = "bMergeCollisionShapes"))
	bool bUseNegativeSpaceInMerge = false;

	/** Whether to simplify the convex hull down to at most a target face count. */
	UPROPERTY(EditAnywhere, DisplayName = "Simplify to Face Count", Category = ConvexHulls, 
		meta = (EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexHulls || GeometryType == ECollisionGeometryType::ConvexDecompositions"))
	bool bSimplifyHulls = true;

	/** Target number of faces in the simplified hull */
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (UIMin = "4", UIMax = "100", ClampMin = "4", ClampMax = "9999999",
		EditConditionHides, EditCondition = "(GeometryType == ECollisionGeometryType::ConvexHulls  || GeometryType == ECollisionGeometryType::ConvexDecompositions) && bSimplifyHulls"))
	int32 HullTargetFaceCount = 20;

	/** Geometric tolerance to apply after simplifying to the target face count */
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (UIMin = "0", UIMax = "100", ClampMin = "0",
		EditConditionHides, EditCondition = "(GeometryType == ECollisionGeometryType::ConvexHulls  || GeometryType == ECollisionGeometryType::ConvexDecompositions) && bSimplifyHulls"))
	double HullSimplifyGeometryTolerance = 1.;

	/** Whether to simplify the input to this edge length before computing convex decomposition. Can give a decomposition result faster for large meshes. */
	UPROPERTY(EditAnywhere, DisplayName = "Simplify to Edge Length", Category = ConvexHulls,
		meta = (EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions"))
	bool bPreSimplifyToEdgeLength = false;

	/** Simplify the input to this edge length before computing convex decomposition. Can give a decomposition result faster for large meshes. */
	UPROPERTY(EditAnywhere, DisplayName = "Target Edge Length", Category = ConvexHulls, meta = (UIMin = "0", UIMax = "100", ClampMin = "0",
		EditConditionHides, EditCondition = "bPreSimplifyToEdgeLength && GeometryType == ECollisionGeometryType::ConvexDecompositions"))
	double DecompositionTargetEdgeLength = 1.0;

	// What algorithm should be used to perform convex decomposition
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions"))
	EConvexDecompositionMethod DecompositionMethod = EConvexDecompositionMethod::NavigationDriven;

	// Whether to limit the number of convex hulls use in each decomposition
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions && DecompositionMethod == EConvexDecompositionMethod::NavigationDriven"))
	bool bLimitHullsPerShape = false;

	// Maximum number of convex hulls to use in each decomposition
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (UIMin = "1", UIMax = "100", ClampMin = "1",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions && (bLimitHullsPerShape || DecompositionMethod != EConvexDecompositionMethod::NavigationDriven)"))
	int32 MaxHullsPerShape = 10;

	/** How much to search the space of possible decompositions beyond Max Hulls Per Shape; for larger values, will do additional work to try to better approximate mesh features (but resulting hulls may overlap more) */
	UPROPERTY(EditAnywhere, DisplayName = "Max Hulls Search Factor", Category = ConvexHulls, meta = (UIMin = "0", UIMax = "2", ClampMin = "0",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions && (DecompositionMethod != EConvexDecompositionMethod::NavigationDriven)"))
	float ConvexDecompositionSearchFactor = .5;

	/** Error tolerance for adding more convex hulls, in cm.  For volumetric errors, the value will be cubed (so a value of 10 indicates a 10x10x10 volume worth of error is acceptable). */
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (UIMin = "0", UIMax = "1000", ClampMin = "0",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions && DecompositionMethod != EConvexDecompositionMethod::NavigationDriven"))
	float AddHullsErrorTolerance = 0;

	/** Minimum part thickness for convex decomposition, in cm; hulls thinner than this will be merged into adjacent hulls, if possible. */
	UPROPERTY(EditAnywhere, Category = ConvexHulls, meta = (UIMin = "0", UIMax = "1", ClampMin = "0",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::ConvexDecompositions"))
	float MinPartThickness = 0.1f;

	/** Navigable space closer to the input than this tolerance distance can be filled in */
	UPROPERTY(EditAnywhere, DisplayName = "Tolerance Distance", Category = NavigableSpaceProtection, meta = (UIMin = "1", UIMax = "100", ClampMin = "0.1",
		EditConditionHides, EditCondition = "(GeometryType == ECollisionGeometryType::ConvexDecompositions && DecompositionMethod == EConvexDecompositionMethod::NavigationDriven) || (bMergeCollisionShapes && bUseNegativeSpaceInMerge)"))
	double NegativeSpaceTolerance = 10;
	/** Minimum radius of navigable space to protect; tunnels with radius smaller than this could be filled in */
	UPROPERTY(EditAnywhere, DisplayName = "Min Radius", Category = NavigableSpaceProtection, meta = (UIMin = "1", UIMax = "200", ClampMin = "0.1",
		EditConditionHides, EditCondition = "(GeometryType == ECollisionGeometryType::ConvexDecompositions && DecompositionMethod == EConvexDecompositionMethod::NavigationDriven) || (bMergeCollisionShapes && bUseNegativeSpaceInMerge)"))
	double NegativeSpaceMinRadius = 40;
	/** Whether to ignore navigable space that is not accessible by from outside (e.g., closed-off interiors / air pockets) */
	UPROPERTY(EditAnywhere, Category = NavigableSpaceProtection, meta = (
		EditConditionHides, EditCondition = "(GeometryType == ECollisionGeometryType::ConvexDecompositions && DecompositionMethod == EConvexDecompositionMethod::NavigationDriven) || (bMergeCollisionShapes && bUseNegativeSpaceInMerge)"))
	bool bIgnoreInternalNegativeSpace = true;

	/** If > 0, the polygon used to generate the swept hull will be simplified up to this distance tolerance, in cm */
	UPROPERTY(EditAnywhere, Category = SweptHulls, meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "100000",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::SweptHulls"))
	float HullTolerance = 0.1f;

	/** How to choose which direction to sweep when creating a swept hull */
	UPROPERTY(EditAnywhere, Category = SweptHulls, meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "100000",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::SweptHulls"))
	EProjectedHullAxis SweepAxis = EProjectedHullAxis::SmallestVolume;

	/** Level set grid resolution along longest grid axis */
	UPROPERTY(EditAnywhere, Category = LevelSets, meta = (UIMin = "3", UIMax = "100", ClampMin = "3", ClampMax = "1000",
		EditConditionHides, EditCondition = "GeometryType == ECollisionGeometryType::LevelSets"))
	int32 LevelSetResolution = 10;

	/** Set how the physics system should interpret collision shapes on the output mesh. Does not affect what collision shapes are generated by this tool. */
	UPROPERTY(EditAnywhere, Category = OutputOptions)
	ECollisionGeometryMode SetCollisionType = ECollisionGeometryMode::SimpleAndComplex;

	/** Show/Hide target mesh */
	UPROPERTY(EditAnywhere, Category = TargetVisualization)
	bool bShowTargetMesh = true;

	// Set by the tool to tell the settings object whether the tool is using multiple inputs.
	UPROPERTY()
	bool bUsingMultipleInputs = false;
};





/**
 * Mesh Inspector Tool for visualizing mesh information
 */
UCLASS(MinimalAPI)
class USetCollisionGeometryTool : public UMultiSelectionMeshEditingTool, public UE::Geometry::IGenericDataOperatorFactory<FPhysicsDataCollection>, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override
	{
		// allow accept when we're showing the current, valid result
		return Super::CanAccept() && bInputMeshesValid && Compute && Compute->HaveValidResult() && !VizSettings->bVisualizationDirty;
	}

	// Begin IGenericDataOperatorFactory interface
	UE_API virtual TUniquePtr<UE::Geometry::TGenericDataOperator<FPhysicsDataCollection>> MakeNewOperator() override;
	// End IGenericDataOperatorFactory interface

	UE_API void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<USetCollisionGeometryToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UCollisionGeometryVisualizationProperties> VizSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UPhysicsObjectToolPropertySet> CollisionProps = nullptr;

	//
	// Background compute
	//
	TUniquePtr<TGenericDataBackgroundCompute<FPhysicsDataCollection>> Compute = nullptr;

protected:
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeom = nullptr;

	TArray<int32> SourceObjectIndices;
	bool bSourcesHidden = false;

	TArray<FDynamicMesh3> InitialSourceMeshes;

	UE_API void OnInputModeChanged();

	/**
	 * Invalidates the background compute operator.
	 */
	UE_API void InvalidateCompute();

	enum class EDetectedCollisionGeometry
	{
		None,
		Sphere = 2,
		Box = 4,
		Capsule = 8,
		Convex = 16
	};

	struct FSourceMesh
	{
		FDynamicMesh3 Mesh;

		EDetectedCollisionGeometry DetectedType = EDetectedCollisionGeometry::None;

		UE::Geometry::FSphere3d DetectedSphere;
		UE::Geometry::FOrientedBox3d DetectedBox;
		UE::Geometry::FCapsule3d DetectedCapsule;
	};
	
	bool bInputMeshesValid = false;
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> InputMeshes;
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> CombinedInputMeshes;
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> SeparatedInputMeshes;
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> PerGroupInputMeshes;

	TSharedPtr<UE::Geometry::FMeshSimpleShapeApproximation, ESPMode::ThreadSafe> InputMeshesApproximator;
	TSharedPtr<UE::Geometry::FMeshSimpleShapeApproximation, ESPMode::ThreadSafe> CombinedInputMeshesApproximator;
	TSharedPtr<UE::Geometry::FMeshSimpleShapeApproximation, ESPMode::ThreadSafe> SeparatedMeshesApproximator;
	TSharedPtr<UE::Geometry::FMeshSimpleShapeApproximation, ESPMode::ThreadSafe> PerGroupMeshesApproximator;

	UE_API void PrecomputeInputMeshes();
	UE_API void InitializeDerivedMeshSet(
		const TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>>& FromInputMeshes,
		TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>>& ToMeshes,
		TFunctionRef<bool(const FDynamicMesh3*, int32, int32)> TrisConnectedPredicate);

	UE_API void OnSelectedGroupLayerChanged();

	FTransform OrigTargetTransform;
	UE::Geometry::FTransformSequence3d TargetInverseTransform;
	FVector TargetScale3D;

	TSharedPtr<FPhysicsDataCollection, ESPMode::ThreadSafe> InitialCollision;
	TSharedPtr<FPhysicsDataCollection, ESPMode::ThreadSafe> GeneratedCollision;
	TSharedPtr<TArray<FPhysicsDataCollection>, ESPMode::ThreadSafe> OtherInputsCollision;
	TSharedPtr<TArray<FTransform3d>, ESPMode::ThreadSafe> OtherInputsTransforms;

	//
	// Geometry Selection
	//

	UE::Geometry::FGeometrySelection InputGeometrySelection;

	UPROPERTY()
	TObjectPtr<UGeometrySelectionVisualizationProperties> GeometrySelectionVizProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> GeometrySelectionViz = nullptr;

private:
	UE_API UE::Geometry::FPolygroupSet GetActiveGroupLayer(const FDynamicMesh3* GroupLayersMesh);
};

#undef UE_API
