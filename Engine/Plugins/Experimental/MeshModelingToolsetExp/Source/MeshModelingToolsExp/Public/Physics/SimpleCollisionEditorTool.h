// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "Physics/CollisionPropertySets.h"
#include "SimpleCollisionEditorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewGeometry;
class UCollisionPrimitivesMechanic;
class USimpleCollisionEditorTool;

UCLASS(MinimalAPI)
class USimpleCollisionEditorToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

UENUM()
enum class ESimpleCollisionEditorToolAction : uint8
{
	NoAction,
	AddSphere,
	AddBox,
	AddCapsule,
	Duplicate,
	DeleteSelected,
	DeleteAll
};

UCLASS(MinimalAPI)
class USimpleCollisionEditorToolActionProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USimpleCollisionEditorTool> ParentTool;
	void Initialize(USimpleCollisionEditorTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(ESimpleCollisionEditorToolAction Action);

	/** Duplicate all selected simple collision shapes */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Duplicate"))
	void Duplicate()
	{
		PostAction(ESimpleCollisionEditorToolAction::Duplicate);
	}

	/** Remove currently selected simple collision shapes from the mesh */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Delete Selected"))
	void Delete()
	{
		PostAction(ESimpleCollisionEditorToolAction::DeleteSelected);
	}

	/** Remove all current simple collision shapes from the mesh */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Delete All"))
	void DeleteAll()
	{
		PostAction(ESimpleCollisionEditorToolAction::DeleteAll);
	}

	/** Add a new simple sphere collision shape */
	UFUNCTION(CallInEditor, Category = "Add Shapes", meta = (DisplayName = "Add Sphere"))
		void AddSphere()
	{
		PostAction(ESimpleCollisionEditorToolAction::AddSphere);
	}

	/** Add a new simple box collision shape */
	UFUNCTION(CallInEditor, Category = "Add Shapes", meta = (DisplayName = "Add Box"))
		void AddBox()
	{
		PostAction(ESimpleCollisionEditorToolAction::AddBox);
	}

	/** Add a new simple capsule collision shape */
	UFUNCTION(CallInEditor, Category = "Add Shapes", meta = (DisplayName = "Add Capsule"))
		void AddCapsule()
	{
		PostAction(ESimpleCollisionEditorToolAction::AddCapsule);
	}

};

/**
 * Simple Collision Editing tool for updating the simple collision geometry on meshes
 */
UCLASS(MinimalAPI)
class USimpleCollisionEditorTool : public USingleSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<USimpleCollisionEditorToolActionProperties> ActionProperties;
	
	ESimpleCollisionEditorToolAction PendingAction = ESimpleCollisionEditorToolAction::NoAction;
	UE_API void RequestAction(ESimpleCollisionEditorToolAction Action);
	UE_API void ApplyAction(ESimpleCollisionEditorToolAction Action);

	friend USimpleCollisionEditorToolActionProperties;

protected:
	TSharedPtr<FPhysicsDataCollection> PhysicsInfos;
	TObjectPtr<UCollisionPrimitivesMechanic> CollisionPrimitivesMechanic;
	UE_API void InitializeObjectProperties(const FPhysicsDataCollection& PhysicsData, UPhysicsObjectToolPropertySet* PropSet);
};

#undef UE_API
