// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Changes/DynamicMeshChangeTarget.h"
#include "BaseTools/SingleClickTool.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "PlaneCutTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPlaneCutTool;


/**
 *
 */
UCLASS(MinimalAPI)
class UPlaneCutToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Standard properties of the plane cut operation
 */
UCLASS(MinimalAPI)
class UPlaneCutToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPlaneCutToolProperties()
	{}

	/** If true, both halves of the cut are computed */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bKeepBothHalves = false;

	/** If keeping both halves, separate the two pieces by this amount */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bKeepBothHalves", UIMin = "0", ClampMin = "0", Delta = 0.5, LinearDeltaSensitivity = 1))
	float SpacingBetweenHalves = 0;

	/** If true, meshes cut into multiple pieces will be saved as separate assets on 'accept'. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bKeepBothHalves"))
	bool bExportSeparatedPiecesAsNewMeshAssets = true;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowPreview = true;

	/** If true, the cut surface is filled with simple planar hole fill surface(s) */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bFillCutHole = true;

	/** If true, will attempt to fill cut holes even if they're ill-formed (e.g. because they connect to pre-existing holes in the geometry) */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bFillCutHole"))
	bool bFillSpans = false;

	/** If true, will simplify triangulation along plane cut when doing so will not affect the shape, UVs or PolyGroups */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	bool bSimplifyAlongCut = true;
};



UENUM()
enum class EPlaneCutToolActions
{
	NoAction,
	Cut,
	FlipPlane
};




UCLASS(MinimalAPI)
class UPlaneCutOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UPlaneCutTool> CutTool;

	int ComponentIndex;
};

/**
 * Simple Mesh Plane Cutting Tool
 */
UCLASS(MinimalAPI)
class UPlaneCutTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	friend UPlaneCutOperatorFactory;

	UE_API UPlaneCutTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent) override;
#endif

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

protected:

	UPROPERTY()
	TObjectPtr<UPlaneCutToolProperties> BasicProperties;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;


	/// Action buttons.
	/// Note these set a flag to call the action later (in OnTick)
	/// Otherwise, the actions in undo history will end up being generically named by an outer UI handler transaction

	/** Cut with the current plane without exiting the tool (Hotkey: T)*/
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Cut"))
	void Cut()
	{
		PendingAction = EPlaneCutToolActions::Cut;
	}

	/** Flip the cutting plane (Hotkey: R) */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Flip Plane"))
	void FlipPlane()
	{
		PendingAction = EPlaneCutToolActions::FlipPlane;
	}

protected:

	UPROPERTY()
	TArray<TObjectPtr<UDynamicMeshReplacementChangeTarget>> MeshesToCut;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	// Cutting plane
	UE::Geometry::FFrame3d CutPlaneWorld;

	// UV Scale factor is cached based on the bounding box of the mesh before any cuts are performed, so you don't get inconsistent UVs if you multi-cut the object to smaller sizes
	TArray<float> MeshUVScaleFactor;

	FViewCameraState CameraState;

	EPlaneCutToolActions PendingAction = EPlaneCutToolActions::NoAction;


	UE_API void DoCut();
	UE_API void DoFlipPlane();

	UE_API void SetupPreviews();

	UE_API void InvalidatePreviews();

	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);

	UE_API void UpdateVisibility();
};

#undef UE_API
