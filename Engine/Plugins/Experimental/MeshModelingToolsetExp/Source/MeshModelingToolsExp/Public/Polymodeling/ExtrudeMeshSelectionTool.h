// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "Templates/PimplPtr.h"
#include "ExtrudeMeshSelectionTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewMesh;
class UMeshOpPreviewWithBackgroundCompute; 
class FExtrudeMeshSelectionOpFactory;
class UTransformProxy;
class UCombinedTransformGizmo;
namespace UE::Geometry { class FMeshRegionOperator; }

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UExtrudeMeshSelectionToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual bool RequiresInputSelection() const override { return true; }
};


UENUM()
enum class EExtrudeMeshSelectionInteractionMode : uint8
{
	/** Define the extrusion distance using a 3D gizmo */
	Interactive = 0,
	/** Define the extrusion distance using a slider in the Settings */
	Fixed = 1
};

UENUM()
enum class EExtrudeMeshSelectionRegionModifierMode : uint8
{
	/** Transform the original selected area */
	OriginalShape = 0,
	/** Flatten the extrusion area to the X/Y plane of the extrusion frame */
	FlattenToPlane = 1,
	/** Flatten the extrusion area by raycasting against the X/Y plane of the extrusion frame */
	RaycastToPlane = 2
};

UCLASS(MinimalAPI)
class UExtrudeMeshSelectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Control how the Extruded Area should be Transformed */
	UPROPERTY(EditAnywhere, Category = "Extrude", meta = (ModelingQuickEdit))
	EExtrudeMeshSelectionInteractionMode InputMode = EExtrudeMeshSelectionInteractionMode::Interactive;

	/** The Extrusion Distance used in Fixed Input Mode*/
	UPROPERTY(EditAnywhere, Category = "Extrude", meta = (DisplayName="Fixed Distance", UIMin = -250, UIMax = 250, EditCondition = "InputMode == EExtrudeMeshSelectionInteractionMode::Fixed", ModelingQuickEdit))
	double ExtrudeDistance = 10.0;

	/** Control how the Extruded Area should be deformed as part of the Extrusion */
	UPROPERTY(EditAnywhere, Category = "Extrude")
	EExtrudeMeshSelectionRegionModifierMode RegionMode = EExtrudeMeshSelectionRegionModifierMode::OriginalShape;

	/** Specify the number of subdivisions along the sides of the Extrusion */
	UPROPERTY(EditAnywhere, Category = "Extrude", meta = (DisplayName="Subdivisions", UIMin = 0, UIMax = 10, ClampMin = 0, ClampMax = 1000))
	int NumSubdivisions = 0;

	/** Specify the Crease Angle used to split the sides of the Extrusion into separate Groups */
	UPROPERTY(EditAnywhere, Category = "Extrude", meta = (UIMin = 0, UIMax = 180, Min = 0, Max = 180))
	double CreaseAngle = 60.0;

	/** Control the maximum distance each vertex may be moved in Raycast To Plane Mode */
	UPROPERTY(EditAnywhere, Category = "Extrude", meta = (DisplayName="Max Distance", UIMin = 0, UIMax = 1000, ClampMin = 0, EditCondition = "RegionMode == EExtrudeMeshSelectionRegionModifierMode::RaycastToPlane"))
	double RaycastMaxDistance = 1000.0;

	/** If the Extruded Area has a fully open border, this option determines if Extrusion will create a Solid mesh or leave the base "open" */
	UPROPERTY(EditAnywhere, Category = "Extrude")
	bool bShellsToSolids = true;			// todo: should only be available if input has shells that could become solid...

	/** Control whether a single Group should be generated along the sides of the Extrusion, or multiple Groups based on the adjacent Groups around the Extruded Area border */
	UPROPERTY(EditAnywhere, Category = "Groups", meta = (DisplayName="Propagate Groups"))
	bool bInferGroupsFromNbrs = true;

	/** Control whether a new Group is generated for each Subdivision */
	UPROPERTY(EditAnywhere, Category = "Groups", meta=(DisplayName="Per Subdivision", EditCondition="NumSubdivisions > 0") )
	bool bGroupPerSubdivision = true;

	/** Control whether groups in the Extruded Area are mapped to new Groups, or replaced with a single new Group */
	UPROPERTY(EditAnywhere, Category = "Groups", meta=(DisplayName="Replace Cap Groups"))
	bool bReplaceSelectionGroups = false;

	/** The automatically-generated UVs on the sides of the Extrusion are scaled by this value */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (DisplayName="UV Scale", UIMin = 0.001, UIMax = 10.0, ClampMin = 0.000001))
	double UVScale = 1.0f;

	/** Control whether a separate UV island should be generated for each output Group on the sides of the Extrusion, or a single UV island wrapping around the entire "tube" */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (DisplayName="Island Per Group"))
	bool bUVIslandPerGroup = true;

	/** Control whether SetMaterialID is assigned to all triangles along the sides of the Extrusion, or if MaterialIDs should be inferred from the Extruded Area */
	UPROPERTY(EditAnywhere, Category = "Material", meta=(DisplayName="Infer Materials"))
	bool bInferMaterialID = true;

	/** Constant Material ID used when MaterialIDs are not being inferred, or no adjacent MaterialID exists */
	UPROPERTY(EditAnywhere, Category = "Material", meta=(DisplayName="Material ID", EditCondition="bInferMaterialID == false"))
	int SetMaterialID = 0;

	/** Control whether the original Mesh Materials should be shown, or a visualization of the extruded Groups */
	UPROPERTY(EditAnywhere, Category = "Visualization", meta=(DisplayName="Show Materials"))
	bool bShowInputMaterials = false;
};



UCLASS(MinimalAPI)
class UExtrudeMeshSelectionTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	UE_API UExtrudeMeshSelectionTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	UE_API FTransform GetTransform() const;
	UE_API void SetTransform(const FTransform& NewTransform);

protected:
	UPROPERTY()
	TObjectPtr<UExtrudeMeshSelectionToolProperties> ExtrudeProperties = nullptr;


protected:
	FBox SelectionBoundsWorld;
	UE::Geometry::FTransformSRT3d WorldTransform;

	UE::Geometry::FFrame3d SelectionFrameLocal;
	UE::Geometry::FFrame3d InitialFrameLocal;
	UE::Geometry::FFrame3d InitialFrameWorld;
	UE::Geometry::FFrame3d ExtrudeFrameWorld;
	UE::Geometry::FFrame3d ExtrudeFrameLocal;
	FVector3d LocalScale;

	UE::Geometry::FDynamicMesh3 CurrentMesh;
	TArray<int32> ExtrudeROI;
	TSet<int32> ModifiedROI;
	int32 MaxMaterialID = 0;		// maximum material ID on mesh (inclusive)

	TPimplPtr<UE::Geometry::FMeshRegionOperator> RegionOperator;
	UE::Geometry::FDynamicMesh3 EditRegionMesh;
	TSharedPtr<UE::Geometry::FSharedConstDynamicMesh3> EditRegionSharedMesh;

	TSet<int32> RegionExtrudeROI;
	TSet<int32> RegionBorderTris;

	TPimplPtr<FExtrudeMeshSelectionOpFactory> OperatorFactory;
	friend class FExtrudeMeshSelectionOpFactory;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> SourcePreview = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> EditCompute = nullptr;

	UE_API void UpdateVisualizationSettings();

	UE_API void Initialize_GizmoMechanic();

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);

	UE_API void UpdateInteractionMode(EExtrudeMeshSelectionInteractionMode InteractionMode);
};

#undef UE_API
