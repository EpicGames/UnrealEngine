// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "Templates/PimplPtr.h"
#include "OffsetMeshSelectionTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UPreviewMesh;
class UMeshOpPreviewWithBackgroundCompute; 
class FOffsetMeshSelectionOpFactory;
namespace UE::Geometry { class FMeshRegionOperator; }

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UOffsetMeshSelectionToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual bool RequiresInputSelection() const override { return true; }
};


UENUM()
enum class EOffsetMeshSelectionInteractionMode : uint8
{
	/** Define the offset distance using a slider in the Settings */
	Fixed = 0
};

UENUM()
enum class EOffsetMeshSelectionDirectionMode : uint8
{
	/** */
	VertexNormals = 0,
	/** */
	FaceNormals = 1,
	/**  */
	ConstantWidth = 2
};


UCLASS(MinimalAPI)
class UOffsetMeshSelectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Control how the Offset Area should be Transformed */
	//UPROPERTY(EditAnywhere, Category = "Offset")
	//EOffsetMeshSelectionInteractionMode InputMode = EOffsetMeshSelectionInteractionMode::Interactive;

	/** The Extrusion Distance used in Fixed Input Mode*/
	UPROPERTY(EditAnywhere, Category = "Offset", meta = (DisplayName="Distance", UIMin = -250, UIMax = 250, ModelingQuickEdit))
	double OffsetDistance = 10.0;

	/** Control how the Offset Area should be displaced */
	UPROPERTY(EditAnywhere, Category = "Offset")
	EOffsetMeshSelectionDirectionMode Direction = EOffsetMeshSelectionDirectionMode::ConstantWidth;

	/** Specify the number of subdivisions along the sides of the Extrusion */
	UPROPERTY(EditAnywhere, Category = "Offset", meta = (DisplayName="Subdivisions", UIMin = 0, UIMax = 10, ClampMin = 0, ClampMax = 1000))
	int NumSubdivisions = 0;

	/** Specify the Crease Angle used to split the sides of the Extrusion into separate Groups */
	UPROPERTY(EditAnywhere, Category = "Offset", meta = (UIMin = 0, UIMax = 180, Min = 0, Max = 180))
	double CreaseAngle = 60.0;

	/** If the Offset Area has a fully open border, this option determines if Extrusion will create a Solid mesh or leave the base "open" */
	UPROPERTY(EditAnywhere, Category = "Offset")
	bool bShellsToSolids = true;			// todo: should only be available if input has shells that could become solid...

	/** Control whether a single Group should be generated along the sides of the Extrusion, or multiple Groups based on the adjacent Groups around the Offset Area border */
	UPROPERTY(EditAnywhere, Category = "Groups", meta = (DisplayName="Propagate Groups"))
	bool bInferGroupsFromNbrs = true;

	/** Control whether a new Group is generated for each Subdivision */
	UPROPERTY(EditAnywhere, Category = "Groups", meta=(DisplayName="Per Subdivision", EditCondition="NumSubdivisions > 0") )
	bool bGroupPerSubdivision = true;

	/** Control whether groups in the Offset Area are mapped to new Groups, or replaced with a single new Group */
	UPROPERTY(EditAnywhere, Category = "Groups", meta=(DisplayName="Replace Cap Groups"))
	bool bReplaceSelectionGroups = false;

	/** The automatically-generated UVs on the sides of the Extrusion are scaled by this value */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (DisplayName="UV Scale", UIMin = 0.001, UIMax = 10.0, ClampMin = 0.000001))
	double UVScale = 1.0f;

	/** Control whether a separate UV island should be generated for each output Group on the sides of the Extrusion, or a single UV island wrapping around the entire "tube" */
	UPROPERTY(EditAnywhere, Category = "UVs", meta = (DisplayName="Island Per Group"))
	bool bUVIslandPerGroup = true;

	/** Control whether SetMaterialID is assigned to all triangles along the sides of the Extrusion, or if MaterialIDs should be inferred from the Offset Area */
	UPROPERTY(EditAnywhere, Category = "Material", meta=(DisplayName="Infer Materials"))
	bool bInferMaterialID = true;

	/** Constant Material ID used when MaterialIDs are not being inferred, or no adjacent MaterialID exists */
	UPROPERTY(EditAnywhere, Category = "Material", meta=(DisplayName="Material ID", EditCondition="bInferMaterialID == false"))
	int SetMaterialID = 0;

	/** Control whether the original Mesh Materials should be shown, or a visualization of the Offset Groups */
	UPROPERTY(EditAnywhere, Category = "Visualization", meta=(DisplayName="Show Materials"))
	bool bShowInputMaterials = false;
};



UCLASS(MinimalAPI)
class UOffsetMeshSelectionTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	UE_API UOffsetMeshSelectionTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual FBox GetWorldSpaceFocusBox() override;


protected:
	UPROPERTY()
	TObjectPtr<UOffsetMeshSelectionToolProperties> OffsetProperties = nullptr;


protected:
	FBox SelectionBoundsWorld;
	UE::Geometry::FTransformSRT3d WorldTransform;

	UE::Geometry::FFrame3d SelectionFrameLocal;
	UE::Geometry::FFrame3d InitialFrameLocal;
	UE::Geometry::FFrame3d InitialFrameWorld;
	UE::Geometry::FFrame3d OffsetFrameWorld;
	UE::Geometry::FFrame3d OffsetFrameLocal;
	FVector3d LocalScale;

	UE::Geometry::FDynamicMesh3 CurrentMesh;
	TArray<int32> OffsetROI;
	TSet<int32> ModifiedROI;
	int32 MaxMaterialID = 0;		// maximum material ID on mesh (inclusive)

	TPimplPtr<UE::Geometry::FMeshRegionOperator> RegionOperator;
	UE::Geometry::FDynamicMesh3 EditRegionMesh;
	TSharedPtr<UE::Geometry::FSharedConstDynamicMesh3> EditRegionSharedMesh;

	TSet<int32> RegionOffsetROI;
	TSet<int32> RegionBorderTris;

	TPimplPtr<FOffsetMeshSelectionOpFactory> OperatorFactory;
	friend class FOffsetMeshSelectionOpFactory;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> SourcePreview = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> EditCompute = nullptr;

	UE_API void UpdateVisualizationSettings();
};

#undef UE_API
