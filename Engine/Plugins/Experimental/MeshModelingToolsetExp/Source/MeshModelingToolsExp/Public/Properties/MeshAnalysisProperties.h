// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "GeometryBase.h"
#include "MeshAnalysisProperties.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


// predeclarations
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);



UCLASS(MinimalAPI)
class UMeshAnalysisProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** In meters squared */
	UPROPERTY(VisibleAnywhere, Category = MeshAnalysis, meta = (NoResetToDefault))
	FString SurfaceArea;

	/** In cubic meters */
	UPROPERTY(VisibleAnywhere, Category = MeshAnalysis, meta = (NoResetToDefault))
	FString Volume;

	UE_API void Update(const FDynamicMesh3& Mesh, const FTransform& Transform);
};

#undef UE_API
