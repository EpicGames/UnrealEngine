// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "MeshBoundaryToolBase.h"
#include "MeshOpPreviewHelpers.h" //UMeshOpPreviewWithBackgroundCompute
#include "ModelingOperators.h" //IDynamicMeshOperatorFactory
#include "InteractiveTool.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/RevolveProperties.h"
#include "ToolContextInterfaces.h" // FToolBuilderState
#include "PropertySets/CreateMeshObjectTypeProperties.h"

#include "RevolveBoundaryTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

// Tool Builder

UCLASS(MinimalAPI)
class URevolveBoundaryToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};

UCLASS(MinimalAPI)
class URevolveBoundaryOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	UE_API TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<URevolveBoundaryTool> RevolveBoundaryTool;
};

UCLASS(MinimalAPI)
class URevolveBoundaryToolProperties : public URevolveProperties
{
	GENERATED_BODY()

public:

	/** Determines how end caps are created. This is not relevant if the end caps are not visible or if the path is not closed. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayAfter = "QuadSplitMode",
		EditCondition = "HeightOffsetPerDegree != 0 || RevolveDegrees != 360"))
	ERevolvePropertiesCapFillMode CapFillMode = ERevolvePropertiesCapFillMode::Delaunay;

	/** If true, displays the original mesh in addition to the revolved boundary. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	bool bDisplayInputMesh = false;

	/** Sets the revolution axis origin. */
	UPROPERTY(EditAnywhere, Category = RevolutionAxis, meta = (DisplayName = "Origin",
		Delta = 5, LinearDeltaSensitivity = 1))
	FVector AxisOrigin = FVector(0, 0, 0);

	/** Sets the revolution axis pitch and yaw. */
	UPROPERTY(EditAnywhere, Category = RevolutionAxis, meta = (DisplayName = "Orientation",
		UIMin = -180, UIMax = 180, ClampMin = -180000, ClampMax = 180000))
	FVector2D AxisOrientation;

protected:
	virtual ERevolvePropertiesCapFillMode GetCapFillMode() const override
	{
		return CapFillMode;
	}
};

/** 
 * Tool that revolves the boundary of a mesh around an axis to create a new mesh. Mainly useful for
 * revolving planar meshes. 
 */
UCLASS(MinimalAPI)
class URevolveBoundaryTool : public UMeshBoundaryToolBase, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanAccept() const override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
protected:

	// Support for Ctrl+(Shift+)Clicking a boundary to align the revolution axis to that segment
	bool bMoveAxisOnClick = false;
	bool bAlignAxisOnClick = true;
	int32 CtrlModifier = 2;
	int32 ShiftModifier = 3;

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<URevolveBoundaryToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	FVector3d RevolutionAxisOrigin;
	FVector3d RevolutionAxisDirection;

	UE_API void GenerateAsset(const FDynamicMeshOpResult& Result);
	UE_API void UpdateRevolutionAxis();
	UE_API void StartPreview();

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	friend class URevolveBoundaryOperatorFactory;
};

#undef UE_API
