// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Properties/RevolveProperties.h"
#include "Spline/BaseMeshFromSplinesTool.h"

#include "RevolveSplineTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UConstructionPlaneMechanic;
class URevolveSplineTool;

//~ TODO: Might want to have some shared enum for sampling splines in a util folder,
//~ but hesitant to prescribe it until other tools want it.
UENUM()
enum class ERevolveSplineSampleMode : uint8
{
	// Place points only at the spline control points
	ControlPointsOnly,

	// Place points along the spline such that the resulting polyline has no more than 
	// some maximum deviation from the curve.
	PolyLineMaxError,

	// Place points along spline that are an equal spacing apart, and so that the spacing
	// is as close as possible to some max spacing.
	UniformSpacingAlongCurve,
};



UCLASS(MinimalAPI)
class URevolveSplineToolProperties : public URevolveProperties
{
	GENERATED_BODY()

public:

	/** Determines how points to revolve are actually picked from the spline. */
	UPROPERTY(EditAnywhere, Category = Spline)
	ERevolveSplineSampleMode SampleMode = ERevolveSplineSampleMode::ControlPointsOnly;

	/** How far to allow the triangulation boundary can deviate from the spline curve before we add more vertices. */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (ClampMin = 0.001, EditConditionHides,
		EditCondition = "SampleMode == ERevolveSplineSampleMode::PolyLineMaxError"))
	double ErrorTolerance = 1.0;

	/** The maximal distance that the spacing should be allowed to be. */
	UPROPERTY(EditAnywhere, Category = Spline, meta = (ClampMin = 0.01, EditConditionHides,
		EditCondition = "SampleMode == ERevolveSplineSampleMode::UniformSpacingAlongCurve"))
	double MaxSampleDistance = 50.0;

	/** Determines how end caps are created. This is not relevant if the end caps are not visible or if the path is not closed. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayAfter = "QuadSplitMode",
		EditCondition = "HeightOffsetPerDegree != 0 || RevolveDegrees != 360"))
	ERevolvePropertiesCapFillMode CapFillMode = ERevolvePropertiesCapFillMode::Delaunay;

	/** Connect the ends of an open path to the axis to add caps to the top and bottom of the revolved result.
	  * This is not relevant for paths that are already closed. */
	UPROPERTY(EditAnywhere, Category = Revolve)
	bool bClosePathToAxis = true;
	
	/** Sets the revolution axis origin. */
	UPROPERTY(EditAnywhere, Category = RevolutionAxis, meta = (DisplayName = "Origin",
		Delta = 5, LinearDeltaSensitivity = 1))
	FVector AxisOrigin = FVector(0, 0, 0);

	/** Sets the revolution axis pitch and yaw. */
	UPROPERTY(EditAnywhere, Category = RevolutionAxis, meta = (DisplayName = "Orientation",
		UIMin = -180, UIMax = 180, ClampMin = -180000, ClampMax = 180000))
	FVector2D AxisOrientation;

	/** 
	 * If true, the revolution axis is re-fit to the input spline on each tool start. If false, the previous
	 * revolution axis is kept.
	 */
	UPROPERTY(EditAnywhere, Category = RevolutionAxis, AdvancedDisplay)
	bool bResetAxisOnStart = true;

protected:
	virtual ERevolvePropertiesCapFillMode GetCapFillMode() const override
	{
		return CapFillMode;
	}
};

enum class ERevolveSplineToolAction
{
	ResetAxis
};

UCLASS(MinimalAPI)
class URevolveSplineToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<URevolveSplineTool> ParentTool;

	void Initialize(URevolveSplineTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(ERevolveSplineToolAction Action);

	/** Fit the axis to the current curve(by aligning it to the startand end points) */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayPriority = 1))
	void ResetAxis() { PostAction(ERevolveSplineToolAction::ResetAxis); }
};

//~ We might someday decide to merge this tool as a separate path in the DrawAndRevolveTool...
/**
 * Revolves a selected spline to create a new mesh.
 */
UCLASS(MinimalAPI)
class URevolveSplineTool : public UBaseMeshFromSplinesTool
{
	GENERATED_BODY()

public:

	UE_API virtual void RequestAction(ERevolveSplineToolAction ActionType);

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UE_API virtual FString GeneratedAssetBaseName() const override;
	UE_API virtual FText TransactionName() const override;

protected:
	// Update the profile curve and fit plane from spline
	UE_API virtual void OnSplineUpdate() override;

	// Keep the result mesh in the same space as set by the operator result
	virtual FTransform3d HandleOperatorTransform(const FDynamicMeshOpResult& OpResult) override
	{
		return OpResult.Transform;
	}

private:

	UPROPERTY()
	TObjectPtr<URevolveSplineToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<URevolveSplineToolActionPropertySet> ToolActions = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	// The actual points to be resolved, sampled from the spline
	TArray<FVector3d> ProfileCurve;
	bool bProfileCurveIsClosed;

	// Axis direction in vector form (since the user modifiable values are a pitch and yaw)
	FVector3d RevolutionAxisDirection;
	// This duplicates Settings->AxisOrigin, but kept for cleanliness since we do need RevolutionAxisDirection
	FVector3d RevolutionAxisOrigin;
	UE_API void UpdateRevolutionAxis();

	UE_API void ResetAxis();

	FVector3d SplineFitPlaneOrigin;
	FVector3d SplineFitPlaneNormal;
};



UCLASS(MinimalAPI, Transient)
class URevolveSplineToolBuilder : public UBaseMeshFromSplinesToolBuilder
{
	GENERATED_BODY()

public:
	/** @return new Tool instance initialized with selected spline source(s) */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	virtual UE::Geometry::FIndex2i GetSupportedSplineCountRange() const override
	{
		return UE::Geometry::FIndex2i(1, 1);
	}
};

#undef UE_API
