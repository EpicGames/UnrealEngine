// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMeshBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "SeamSculptTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UExistingMeshMaterialProperties;
class UPreviewGeometry;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 *
 */
UCLASS(MinimalAPI)
class USeamSculptToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};





UCLASS(MinimalAPI)
class USeamSculptToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowWireframe = false;


	UPROPERTY(EditAnywhere, Category = Options)
	bool bHitBackFaces = false;

	/**
	 * Setting this above 0 will include a measure of path similarity to seam transfer, so that among
	 *  similarly short paths, we pick one that lies closer to the edge. Useful in cases where the path
	 *  is on the wrong diagonal to the triangulation, because it prefers a closely zigzagging path over
	 *  a wider "up and over" path that has similar length. If set to 0, only path length is used.
	 */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (
		ClampMin = 0, UIMax = 1000))
	double PathSimilarityWeight = 200;
};







UCLASS(MinimalAPI, Transient)
class USeamSculptTool : public UDynamicMeshBrushTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API USeamSculptTool();

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// UBaseBrushTool overrides
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual bool SupportsBrushAdjustmentInput() override { return false; }

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}


protected:
	UE_API virtual void ApplyStamp(const FBrushStampData& Stamp);

	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;


public:
	UPROPERTY()
	TObjectPtr<USeamSculptToolProperties> Settings;


protected:
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeom;

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
	FTransform3d MeshTransform;
	double NormalOffset = 0;

	UE_API void InitPreviewGeometry();
	UE_API void UpdatePreviewGeometry();
	bool bPreviewGeometryNeedsUpdate = false;

	FVector3d CurrentSnapPositionLocal = FVector3d::Zero();
	int32 CurrentSnapVertex = -1;

	FVector3d DrawPathStartPositionLocal = FVector3d::Zero();
	int32 DrawPathStartVertex = -1;

	TArray<int32> CurDrawPath;
	UE_API void UpdateCurrentDrawPath();


	UE_API void CreateSeamAlongPath();

	enum class EActiveCaptureState
	{
		NoState,
		DrawNewPath
	};
	EActiveCaptureState CaptureState = EActiveCaptureState::NoState;

	UWorld* TargetWorld;

private:
	double MeshMaxDim = 0;
};

#undef UE_API
