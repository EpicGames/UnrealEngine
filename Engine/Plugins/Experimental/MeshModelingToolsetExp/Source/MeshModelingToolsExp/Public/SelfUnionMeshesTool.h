// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "Drawing/LineSetComponent.h"
#include "BaseTools/BaseCreateFromSelectedTool.h"

#include "CompositionOps/SelfUnionMeshesOp.h"

#include "SelfUnionMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

// predeclarations
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


/**
 * Standard properties of the self-union operation
 */
UCLASS(MinimalAPI)
class USelfUnionMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** If true, remove open, visible geometry */
	UPROPERTY(EditAnywhere, Category = Merge)
	bool bTrimFlaps = false;

	/** Try to fill holes created by the merge, e.g. due to numerical errors */
	UPROPERTY(EditAnywhere, Category = Merge, AdvancedDisplay)
	bool bTryFixHoles = false;

	/** Try to collapse extra edges created by the merge */
	UPROPERTY(EditAnywhere, Category = Merge, AdvancedDisplay)
	bool bTryCollapseEdges = true;

	/** Threshold to determine whether a triangle in one mesh is inside or outside of the other */
	UPROPERTY(EditAnywhere, Category = Merge, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float WindingThreshold = 0.5;

	/** Show boundary edges created by the merge (often due to numerical error) */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowNewBoundaryEdges = true;

	/** If true, only the first mesh will keep its materials assignments; all other triangles will be assigned material 0 */
	UPROPERTY(EditAnywhere, Category = Materials)
	bool bOnlyUseFirstMeshMaterials = false;
};



/**
 * Union of meshes, resolving self intersections
 */
UCLASS(MinimalAPI)
class USelfUnionMeshesTool : public UBaseCreateFromSelectedTool
{
	GENERATED_BODY()

public:

	USelfUnionMeshesTool() {}

protected:

	UE_API void TransformChanged(UTransformProxy* Proxy, FTransform Transform) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh = true) override;

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;
	UE_API virtual void SetPreviewCallbacks() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

protected:

	UE_API void UpdateVisualization();

protected:

	UPROPERTY()
	TObjectPtr<USelfUnionMeshesToolProperties> Properties;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLineSet;

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CombinedSourceMeshes;

	// for visualization of any errors in the currently-previewed merge operation
	TArray<int> CreatedBoundaryEdges;

	FVector3d CombinedCenter;
};


UCLASS(MinimalAPI)
class USelfUnionMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:
	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<USelfUnionMeshesTool>(SceneState.ToolManager);
	}
};



#undef UE_API
