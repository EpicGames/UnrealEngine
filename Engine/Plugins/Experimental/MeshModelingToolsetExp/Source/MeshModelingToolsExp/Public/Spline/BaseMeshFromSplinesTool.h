// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

#include "Components/SplineComponent.h" // (to use with TWeakObjectPtr)
#include "Engine/World.h" // (to use with TWeakObjectPtr)
#include "IndexTypes.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "InteractiveToolQueryInterfaces.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"


#include "BaseMeshFromSplinesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class USplineComponent;
class UNewMeshMaterialProperties;
class UWorld;
class AActor;
struct FDynamicMeshOpResult;
class UCreateMeshObjectTypeProperties;
class UMeshOpPreviewWithBackgroundCompute;
class UNewMeshMaterialProperties;

/**
 * Tool to create a mesh from a set of selected Spline Components
 */
UCLASS(MinimalAPI)
class UBaseMeshFromSplinesTool : public UInteractiveTool, public IInteractiveToolEditorGizmoAPI, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	// IInteractiveToolEditorGizmoAPI -- allow editor gizmo so users can live-edit the splines
	virtual bool GetAllowStandardEditorGizmos() override
	{
		return true;
	}

	// InteractiveTool API
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	virtual void SetSplineActors(TArray<TWeakObjectPtr<AActor>> InSplineActors)
	{
		ActorsWithSplines = MoveTemp(InSplineActors);
	}

	UE_API virtual void SetWorld(UWorld* World);
	UE_API virtual UWorld* GetTargetWorld();

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator();

protected:

	// Override to respond to spline changes
	virtual void OnSplineUpdate() {};


	UE_API virtual void GenerateAsset(const FDynamicMeshOpResult& OpResult);

	//
	// API for asset generation: Override the below methods to customize common parts of spline tool asset generation:
	// 
	
	// Called by GenerateAsset to set the desired asset transform and if needed transform the result to the corresponding local space
	// @return The transform to use for the generated asset
	UE_API virtual FTransform3d HandleOperatorTransform(const FDynamicMeshOpResult& OpResult);
	// Override with an appropriate prefix for generated meshes
	virtual FString GeneratedAssetBaseName() const
	{
		return FString(TEXT("SplineMesh"));
	}
	// Override with an appropriate tool transaction name
	virtual FText TransactionName() const
	{
		return FText();
	}

	//
	// Methods to access the active splines 
	//

	template<typename Func>
	void EnumerateSplines(Func SplineComponentFunc) const
	{
		if (bLostInputSpline)
		{
			return;
		}

		for (int32 ActorIdx = 0; ActorIdx < ActorsWithSplines.Num(); ++ActorIdx)
		{
			if (AActor* Actor = ActorsWithSplines[ActorIdx].Get())
			{
				Actor->ForEachComponent<USplineComponent>(false, [&SplineComponentFunc](USplineComponent* SplineComponent)
					{
						SplineComponentFunc(SplineComponent);
					});
			}
		}
	}

	inline int32 NumSplines() const
	{
		int32 SplineCount = 0;
		EnumerateSplines([&](USplineComponent* Unused) -> void
			{
				SplineCount++;
			});
		return SplineCount;
	}

	UE_API USplineComponent* GetFirstSpline() const;
	UE_API USplineComponent* GetLastSpline() const;

	// Common spline tool properties

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;

	// Note: We track actors instead of the USplineComponents here because the USplineComponents objects are often deleted / swapped for identical but new objects
	UPROPERTY()
	TArray<TWeakObjectPtr<AActor>> ActorsWithSplines;

	// Helper to track the splines we are triangulating, so we can re-triangulate when they are moved or changed
	UE_API void PollSplineUpdates();
	// Track the spline 'Version' integer, which is incremented when splines are changed
	TArray<uint32> LastSplineVersions;
	// Track the spline component's transform (to world space)
	TArray<FTransform> LastSplineTransforms;

	// If failed to reacquire once, used to avoid trying to reaquire again.
	bool bLostInputSpline = false;
};



/**
 * Base Tool Builder for tools that operate on a selection of Spline Components
 */
UCLASS(MinimalAPI, Transient)
class UBaseMeshFromSplinesToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	/** @return true if spline component sources can be found in the active selection */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** Called by BuildTool to configure the Tool with the input spline source(s) based on the SceneState */
	UE_API virtual void InitializeNewTool(UBaseMeshFromSplinesTool* Tool, const FToolBuilderState& SceneState) const;

	// @return the min and max (inclusive) number of splines allowed in the selection for the tool to be built. A value of -1 can be used to indicate there is no maximum.
	virtual UE::Geometry::FIndex2i GetSupportedSplineCountRange() const
	{
		return UE::Geometry::FIndex2i(1, -1);
	}
};



#undef UE_API
