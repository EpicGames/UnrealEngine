// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiTargetWithSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "SplitMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UMaterialInterface;


UCLASS(MinimalAPI)
class USplitMeshesToolBuilder : public UMultiTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMultiTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	virtual bool RequiresInputSelection() const override { return false; }

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// Methods for splitting meshes
UENUM()
enum class ESplitMeshesMethod : uint8
{
	// Split meshes based on the triangle-connected regions of the mesh
	ByMeshTopology,
	// Split meshes based on triangle-connected regions, and consider vertices to be connected if they are within a tolerance distance
	ByVertexOverlap,
	// Split meshes based on material ID
	ByMaterialID,
	// Split meshes based on PolyGroup ID
	ByPolyGroup
};

UCLASS(MinimalAPI)
class USplitMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	// Method to use to split the input(s) into output meshes
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "!bIsInSelectionMode", EditConditionHides, HideEditConditionToggle))
	ESplitMeshesMethod SplitMethod = ESplitMeshesMethod::ByMeshTopology;

	// Vertices as close as this distance will be treated as overlapping, and kept in the same output mesh
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = .0001, UIMax = 1.0, EditCondition = "!bIsInSelectionMode && SplitMethod == ESplitMeshesMethod::ByVertexOverlap", EditConditionHides, HideEditConditionToggle))
	double ConnectVerticesThreshold = 0.01;

	// Whether to transfer materials to the output meshes
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTransferMaterials = true;

	// Whether to color mesh faces based on how they will be split into output meshes
	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowPreview = true;

	UPROPERTY()
	bool bIsInSelectionMode = false;
};



UCLASS(MinimalAPI)
class USplitMeshesTool : public UMultiTargetWithSelectionTool
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UPROPERTY()
	TObjectPtr<USplitMeshesToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

protected:
	struct FSourceMeshInfo
	{
		UE::Geometry::FDynamicMesh3 Mesh;
		TArray<UMaterialInterface*> Materials;
	};
	TArray<FSourceMeshInfo> SourceMeshes;


	struct FComponentsInfo
	{
		bool bNoComponents;
		TArray<UE::Geometry::FDynamicMesh3> Meshes;
		TArray<TArray<UMaterialInterface*>> Materials;
		TArray<FVector3d> Origins;
	};
	TArray<FComponentsInfo> SplitMeshes;

	int32 NoSplitCount = 0;

	UE_API void UpdateSplitMeshes();

private:
	// Preview how the meshes are to be split
	UPROPERTY()
	TArray<TObjectPtr<UPreviewGeometry>> PerTargetPreviews;
	UPROPERTY()
	TObjectPtr<UMaterialInterface> PreviewMaterial = nullptr;

	UE_API void UpdatePreviewVisibility(bool bShowPreview);
};

#undef UE_API
