// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolExclusiveToolAPI
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "TransferMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

/**
 *
 */
UCLASS(MinimalAPI)
class UTransferMeshToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
 * Standard properties of the Transfer operation
 */
UCLASS(MinimalAPI)
class UTransferMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTransferMaterials = true;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bTransferCollision = false;

	/** Specify which LOD from the Source (First) mesh to copy from */
	UPROPERTY(EditAnywhere, Category = SourceMesh, meta = (DisplayName = "Source LOD", TransientToolProperty, NoResetToDefault, GetOptions = GetSourceLODNamesFunc, HideEditConditionToggle, EditCondition="bIsStaticMeshSource", EditConditionHides))
	FString SourceLOD;

	/** Specify which LOD on the Target (Second) mesh to copy to */
	UPROPERTY(EditAnywhere, Category = TargetMesh, meta = (DisplayName = "Target LOD", TransientToolProperty, NoResetToDefault, GetOptions = GetTargetLODNamesFunc, HideEditConditionToggle, EditCondition="bIsStaticMeshTarget", EditConditionHides))
	FString TargetLOD;


	//
	// non-exposed properties used to provide custom lists to SourceLOD/TargetLOD
	//

	UPROPERTY(meta = (TransientToolProperty))
	bool bIsStaticMeshSource  = false;

	UFUNCTION()
	const TArray<FString>& GetSourceLODNamesFunc() const { return SourceLODNamesList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> SourceLODNamesList;

	TArray<EMeshLODIdentifier> SourceLODEnums;


	UFUNCTION()
	const TArray<FString>& GetTargetLODNamesFunc() const { return TargetLODNamesList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> TargetLODNamesList;

	TArray<EMeshLODIdentifier> TargetLODEnums;

	UPROPERTY(meta = (TransientToolProperty))
	bool bIsStaticMeshTarget = false;
};



UCLASS(MinimalAPI)
class UTransferMeshTool : public UMultiSelectionMeshEditingTool,
	// Disallow auto-accept switch-away for the tool
	public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UPROPERTY()
	TObjectPtr<UTransferMeshToolProperties> BasicProperties;
};

#undef UE_API
