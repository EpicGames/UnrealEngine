// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Changes/TransformChange.h"
#include "FrameTypes.h"
#include "TransformMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

class UBaseAxisTranslationGizmo;
class UAxisAngleGizmo;
class UDragAlignmentMechanic;
class UCombinedTransformGizmo;
class UTransformProxy;


/**
 *
 */
UCLASS(MinimalAPI)
class UTransformMeshesToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



/** Mesh Transform modes */
UENUM()
enum class ETransformMeshesTransformMode : uint8
{
	/** Single Gizmo for all Objects */
	SharedGizmo = 0 UMETA(DisplayName = "Shared Gizmo"),

	/** Single Gizmo for all Objects, Rotations applied per-Object */
	SharedGizmoLocal = 1 UMETA(DisplayName = "Shared Gizmo (Local)"),

	/** Separate Gizmo for each Object */
	PerObjectGizmo = 2 UMETA(DisplayName = "Multi-Gizmo"),

	LastValue UMETA(Hidden)
};



/** Snap-Drag Source Point */
UENUM()
enum class ETransformMeshesSnapDragSource : uint8
{
	/** Snap-Drag moves the Clicked Point to the Target Location */
	ClickPoint = 0 UMETA(DisplayName = "Click Point"),

	/** Snap-Drag moves the Gizmo/Pivot to the Target Location */
	Pivot = 1 UMETA(DisplayName = "Pivot"),


	LastValue UMETA(Hidden)

};



/** Snap-Drag Rotation Mode */
UENUM()
enum class ETransformMeshesSnapDragRotationMode : uint8
{
	/** Snap-Drag only translates, ignoring Normals */
	Ignore = 0 UMETA(DisplayName = "Ignore"),

	/** Snap-Drag aligns the Source and Target Normals to point in the same direction */
	Align = 1 UMETA(DisplayName = "Align"),

	/** Snap-Drag aligns the Source Normal to the opposite of the Target Normal direction */
	AlignFlipped = 2 UMETA(DisplayName = "Align Flipped"),

	LastValue UMETA(Hidden)
};



/**
 * Standard properties of the Transform Meshes operation
 */
UCLASS(MinimalAPI)
class UTransformMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Transformation Mode controls the overall behavior of the Gizmos in the Tool */
	UPROPERTY(EditAnywhere, Category = Options)
	ETransformMeshesTransformMode TransformMode = ETransformMeshesTransformMode::SharedGizmo;

	/** When true, transformations are applied to the Instances of any Instanced Components (eg InstancedStaticMeshComponent) instead of to the Components */
	UPROPERTY(EditAnywhere, Category = Options, meta = (HideEditConditionToggle, EditCondition = "bHaveInstances"))
	bool bApplyToInstances = true;


	/** When true, the Gizmo can be moved independently without affecting objects. This allows the Gizmo to be repositioned before transforming. */
	UPROPERTY(EditAnywhere, Category = Pivot, meta = (TransientToolProperty, EditCondition = "TransformMode != ETransformMeshesTransformMode::PerObjectGizmo") )
	bool bSetPivotMode = false;


	/** When Snap-Dragging is enabled, you can Click-drag starting on the target objects to reposition them relative to the rest of the scene */
	UPROPERTY(EditAnywhere, Category = SnapDragging, meta = (TransientToolProperty, DisplayName = "Enable"))
	bool bEnableSnapDragging = false;

	/** Which point on the object being Snap-Dragged to use as the "Source" point */
	UPROPERTY(EditAnywhere, Category = SnapDragging, meta = (EditCondition = "bEnableSnapDragging == true"))
	ETransformMeshesSnapDragSource SnapDragSource = ETransformMeshesSnapDragSource::ClickPoint;

	/** How the object being Snap-Dragged should be rotated relative to the Source point location and Hit Surface normal  */
	UPROPERTY(EditAnywhere, Category = SnapDragging, meta = (EditCondition = "bEnableSnapDragging == true"))
	ETransformMeshesSnapDragRotationMode RotationMode = ETransformMeshesSnapDragRotationMode::AlignFlipped;


public:

	// internal, used to control visibility of Instance settings
	UPROPERTY(meta = (TransientToolProperty))
	bool bHaveInstances = false;
};


USTRUCT()
struct FTransformMeshesTarget
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;
};


/**
 *
 */
UCLASS(MinimalAPI)
class UTransformMeshesTool : public UMultiSelectionMeshEditingTool, public IClickDragBehaviorTarget, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UTransformMeshesTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }


	// ICLickDragBehaviorTarget interface
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:
	UPROPERTY()
	TObjectPtr<UTransformMeshesToolProperties> TransformProps;

protected:
	UPROPERTY()
	TArray<FTransformMeshesTarget> ActiveGizmos;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	ETransformMeshesTransformMode CurTransformMode;
	UE_API void UpdateTransformMode(ETransformMeshesTransformMode NewMode);
	UE_API void UpdateSetPivotModes(bool bEnableSetPivot);

	UE_API void SetActiveGizmos_Single(bool bLocalRotations);
	UE_API void SetActiveGizmos_PerObject();
	UE_API void ResetActiveGizmos();


	UE::Geometry::FFrame3d StartDragFrameWorld;
	FTransform StartDragTransform;
	int ActiveSnapDragIndex = -1;
	// Can't just use ActiveSnapDragIndex to replace this bool, because the former gets set while checking for capture
	bool bCurrentlyDragging = false;

	UE_API void OnParametersUpdated();
};

#undef UE_API
