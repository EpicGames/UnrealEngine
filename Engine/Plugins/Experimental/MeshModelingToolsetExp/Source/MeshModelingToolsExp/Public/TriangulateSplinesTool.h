// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

#include "CurveOps/TriangulateCurvesOp.h"

#include "Spline/BaseMeshFromSplinesTool.h"


#include "TriangulateSplinesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


/**
 * Parameters for controlling the spline triangulation
 */
UCLASS(MinimalAPI)
class UTriangulateSplinesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	// How far to allow the triangulation boundary can deviate from the spline curve before we add more vertices
	UPROPERTY(EditAnywhere, Category = Spline, meta = (ClampMin = 0.001))
	double ErrorTolerance = 1.0;
	
	// Whether and how to flatten the curves. If curves are flattened, they can also be offset and combined
	UPROPERTY(EditAnywhere, Category = Spline)
	EFlattenCurveMethod FlattenMethod = EFlattenCurveMethod::DoNotFlatten;

	// Whether or how to combine the curves
	UPROPERTY(EditAnywhere, Category = Spline, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten", EditConditionHides))
	ECombineCurvesMethod CombineMethod = ECombineCurvesMethod::LeaveSeparate;

	// If > 0, Extrude the triangulation by this amount
	UPROPERTY(EditAnywhere, Category = Mesh, meta = (ClampMin = 0.0))
	double Thickness = 0.0;

	// Whether to flip the facing direction of the generated mesh
	UPROPERTY(EditAnywhere, Category = Mesh)
	bool bFlipResult = false;

	// How to handle open curves: Either offset them, or treat them as closed curves
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten", EditConditionHides))
	EOffsetOpenCurvesMethod OpenCurves = EOffsetOpenCurvesMethod::TreatAsClosed;

	// How much offset to apply to curves
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten", EditConditionHides))
	double CurveOffset = 0.0;

	// Whether and how to apply offset to closed curves
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten && CurveOffset != 0", EditConditionHides))
	EOffsetClosedCurvesMethod OffsetClosedCurves = EOffsetClosedCurvesMethod::OffsetOuterSide;

	// The shape of the ends of offset curves
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten && OpenCurves != EOffsetOpenCurvesMethod::TreatAsClosed && CurveOffset != 0", EditConditionHides))
	EOpenCurveEndShapes EndShapes = EOpenCurveEndShapes::Square;

	// The shape of joins between segments of an offset curve
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten && CurveOffset != 0", EditConditionHides))
	EOffsetJoinMethod JoinMethod = EOffsetJoinMethod::Square;

	// How far a miter join can extend before it is replaced by a square join
	UPROPERTY(EditAnywhere, Category = Offset, meta = (ClampMin = 0.0, EditCondition = "FlattenMethod != EFlattenCurveMethod::DoNotFlatten && CurveOffset != 0 && JoinMethod == EOffsetJoinMethod::Miter", EditConditionHides))
	double MiterLimit = 1.0;

};

/**
 * Tool to create a mesh by triangulating the shapes outlined or traced by a set of selected Spline Components, with optional offset and extrusion
 */
UCLASS(MinimalAPI)
class UTriangulateSplinesTool : public UBaseMeshFromSplinesTool
{
	GENERATED_BODY()

public:

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UE_API virtual FString GeneratedAssetBaseName() const override;
	UE_API virtual FText TransactionName() const override;

protected:
	UE_API virtual void OnSplineUpdate() override;

private:

	UPROPERTY()
	TObjectPtr<UTriangulateSplinesToolProperties> TriangulateProperties;

	// Sampled splines, computed in OnSplineUpdate
	struct FPathCache
	{
		TArray<FVector3d> Vertices;
		bool bClosed;
		FTransform ComponentTransform;
	};
	TArray<FPathCache> SplinesCache;

};


UCLASS(MinimalAPI, Transient)
class UTriangulateSplinesToolBuilder : public UBaseMeshFromSplinesToolBuilder
{
	GENERATED_BODY()

public:
	/** @return new Tool instance initialized with selected spline source(s) */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



#undef UE_API
