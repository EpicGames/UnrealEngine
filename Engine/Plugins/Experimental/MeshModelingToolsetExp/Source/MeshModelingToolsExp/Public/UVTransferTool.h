// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MultiTargetWithSelectionTool.h" 
#include "Containers/UnrealString.h"
#include "GeometryBase.h"
#include "InteractiveToolBuilder.h"
#include "Misc/Optional.h"
#include "ModelingOperators.h" // IDynamicMeshOperatorFactory

#include "UVTransferTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

struct FDynamicMeshOpResult;
class UExistingMeshMaterialProperties;
class UMeshElementsVisualizer;
class UMeshOpPreviewWithBackgroundCompute;
class UPreviewMesh;
class UUVLayoutPreview;

PREDECLARE_GEOMETRY(class FDynamicMesh3);


UCLASS(MinimalAPI)
class UUVTransferToolBuilder : public UMultiTargetWithSelectionToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual bool RequiresInputSelection() const override { return false; }

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

UCLASS(MinimalAPI)
class UUVTransferToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	// If false, the first selected mesh's UV's are applied to the second selected mesh. If true, the
	//  reverse direction is used.
	UPROPERTY(EditAnywhere, Category = Options)
	bool bReverseDirection = false;

	/**
	 * If true, we only transfer the seams without trying to transfer actual UV element values.
	 */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTransferSeamsOnly = false;

	/**
	 * If true, clears existing seams on the destination mesh before carrying over new ones.
	 */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bClearExistingSeams = true;

	/**
	 * Setting this above 0 will include a measure of path similarity to seam transfer, so that among
	 *  similarly short paths, we pick one that lies closer to the edge. Useful in cases where the path
	 *  is on the wrong diagonal to the triangulation, because it prefers a closely zigzagging path over
	 *  a wider "up and over" path that has similar length. If set to 0, only path length is used.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		ClampMin = 0, UIMax = 1000))
	double PathSimilarityWeight = 200;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowWireframes = false;

	UPROPERTY(EditAnywhere, Category = Visualization);
	bool bShowSeams = true;
	/**
	 * How far to look for a corresponding vertex on the destination. The destination is expected to
	 *  be a simplified version of source using existing vertices, so this should not need to be set high.
	 */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (
		ClampMin = 0.0001, ClampMax = 10000, UIMax = 1.0))
	double VertexSearchDistance = 0.0001;

private:
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DisplayName = "Source UV Channel", 
		GetOptions = GetSourceUVChannelNames, NoResetToDefault))
	FString SourceUVChannel;

	/** 
	 * When true, the source channel is the same index in the source and destination, and options
	 *  are limited to channels that exist on both meshes.
	 */
	UPROPERTY(EditAnywhere, Category = "Options")
	bool bSameDestinationChannel = true;
	
	/** UV Channel to use as destination, if Same Destination Channel is false. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DisplayName = "Destination UV Channel", 
		GetOptions = GetDestUVChannelNames, NoResetToDefault,
		EditCondition = "!bSameDestinationChannel", EditConditionHides))
	FString DestUVChannel;

	UFUNCTION()
	const TArray<FString>& GetSourceUVChannelNames() const { return SourceUVChannels; }

	UFUNCTION()
	const TArray<FString>& GetDestUVChannelNames() const { return DestUVChannels; }
	
	void ReinitializeChannelOptions(int32 NumSourceChannels, int32 NumDestChannels);
	int32 GetSourceUVChannel();
	int32 GetDestUVChannel();

	TArray<FString> SourceUVChannels;
	TArray<FString> DestUVChannels;

	friend class UUVTransferTool;
};


/**
 * Tool that transfers UV data from a lower res mesh to a higher one. The lower resolution mesh is typically
 *  obtained by simplifying the destination mesh with a "use existing vertices" setting so that an easy
 *  correspondence between mesh vertices can be found.
 */
UCLASS(MinimalAPI)
class UUVTransferTool : public UMultiTargetWithSelectionTool
	, public UE::Geometry::IDynamicMeshOperatorFactory
	, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:

	// UMultiSelectionMeshEditingTool
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

private:

	UPROPERTY()
	TObjectPtr<UUVTransferToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> DestinationMaterialSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> DestinationPreview = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> SourcePreview = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> SourceSeamVisualizer = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> DestinationSeamVisualizer = nullptr;

	// Normally, Mesh1 corresponds to source and Mesh2 is destination, but this is reversed if bReverseDirection is true.
	TSharedPtr<UE::Geometry::FDynamicMesh3> Meshes[2];
	TOptional<TSet<int32>> SelectionTidSets[2];
	
	void ReinitializePreviews();
	void UpdateVisualizations();
	void ReinitializeUVChannelOptions();
	
	void InvalidatePreview();
	void GenerateAsset(const FDynamicMeshOpResult& Result);
};

#undef UE_API
