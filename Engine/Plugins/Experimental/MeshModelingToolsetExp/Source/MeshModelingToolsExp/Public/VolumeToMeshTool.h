// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Volume.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PreviewMesh.h"
#include "Drawing/LineSetComponent.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "VolumeToMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API

/**
 *
 */
UCLASS(MinimalAPI)
class UVolumeToMeshToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};






UCLASS(MinimalAPI)
class UVolumeToMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Weld coincident vertices and edges together in the resulting mesh to form a closed mesh surface. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWeldEdges = true;

	/** If WeldEdges is enabled, attempt to fill any small holes or cracks in the resulting mesh to form a closed surface. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bWeldEdges") )
	bool bAutoRepair = true;

	/** If WeldEdges is enabled, and after mesh generation is complete, flip edges in planar regions to improve triangle quality. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "bWeldEdges"))
	bool bOptimizeMesh = true;

	/** Show the wireframe of the resulting converted mesh geometry.*/
	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowWireframe = true;
};



/**
 *
 */
UCLASS(MinimalAPI)
class UVolumeToMeshTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	UE_API UVolumeToMeshTool();

	virtual void SetWorld(UWorld* World) { this->TargetWorld = World; }
	UE_API virtual void SetSelection(AVolume* Volume);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;


protected:
	UPROPERTY()
	TObjectPtr<UVolumeToMeshToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TSoftObjectPtr<AVolume> TargetVolume;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> VolumeEdgesSet;

protected:
	UWorld* TargetWorld = nullptr;

	UE::Geometry::FDynamicMesh3 CurrentMesh;

	UE_API void RecalculateMesh();

	UE_API void UpdateLineSet();

	bool bResultValid = false;

};

#undef UE_API
