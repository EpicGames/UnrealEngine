// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseVoxelTool.h"

#include "VoxelBlendMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API


/** CSG-style blend operations */
UENUM()
enum class EVoxelBlendOperation : uint8
{
	/** Smoothed union of all shapes */
	Union = 0,

	/** Smoothed subtraction of all shapes from the first selected shape */
	Subtract = 1,
};


/**
 * Properties of the blend operation
 */
UCLASS(MinimalAPI)
class UVoxelBlendMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Blend power controls the shape of the blend between shapes */
	UPROPERTY(EditAnywhere, Category = Blend, meta = (UIMin = "1", UIMax = "4", ClampMin = "1", ClampMax = "10"))
	double BlendPower = 2;

	/** Blend falloff controls the size of the blend region */
	UPROPERTY(EditAnywhere, Category = Blend, meta = (UIMin = ".1", UIMax = "100", ClampMin = ".001", ClampMax = "1000"))
	double BlendFalloff = 10;

	/** How to combine the shapes */
	UPROPERTY(EditAnywhere, Category = Blend)
	EVoxelBlendOperation Operation = EVoxelBlendOperation::Union;



	/** Apply a "VoxWrap" operation to input mesh(es) before computing the blend.  Fixes results for inputs with holes and/or self-intersections. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess)
	bool bVoxWrap = false;

	/** Remove internal surfaces from the VoxWrap output, before computing the blend. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess, meta = (EditCondition = "bVoxWrap == true"))
	bool bRemoveInternalsAfterVoxWrap = false;

	/** Thicken open-boundary surfaces (extrude them inwards) before VoxWrapping them. Units are in world space. If 0 then no extrusion is applied. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess, meta = (EditCondition = "bVoxWrap == true", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	double ThickenShells = 0.0;
};



/**
 * Tool to smoothly blend meshes together
 */
UCLASS(MinimalAPI)
class UVoxelBlendMeshesTool : public UBaseVoxelTool
{
	GENERATED_BODY()

public:

	UVoxelBlendMeshesTool() {}

protected:

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh) override;

	// TODO: this tool also needs a warning for meshes w/ open boundaries

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UVoxelBlendMeshesToolProperties> BlendProperties;

	virtual bool KeepCollisionFrom(int32 TargetIdx) const override
	{
		if (BlendProperties->Operation == EVoxelBlendOperation::Subtract)
		{
			return TargetIdx == 0;
		}
		return true;
	}
};


UCLASS(MinimalAPI)
class UVoxelBlendMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:
	virtual int32 MinComponentsSupported() const override { return 2; }

	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UVoxelBlendMeshesTool>(SceneState.ToolManager);
	}
};


#undef UE_API
