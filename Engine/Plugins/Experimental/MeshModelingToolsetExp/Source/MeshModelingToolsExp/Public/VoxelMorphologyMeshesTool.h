// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseVoxelTool.h"
#include "CompositionOps/VoxelMorphologyMeshesOp.h"

#include "VoxelMorphologyMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API



/**
 * Properties of the morphology tool
 */
UCLASS(MinimalAPI)
class UVoxelMorphologyMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** Which offset (or morphology) operation to apply */
	UPROPERTY(EditAnywhere, Category = Offset)
	EMorphologyOperation Operation = EMorphologyOperation::Dilate;

	/** Distance to offset the mesh */
	UPROPERTY(EditAnywhere, Category = Offset, meta = (UIMin = ".1", UIMax = "100", ClampMin = ".001", ClampMax = "1000"))
	double Distance = 5;

	/** Apply a "VoxWrap" operation to input mesh(es) before computing the morphology.  Fixes results for inputs with holes and/or self-intersections. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess)
	bool bVoxWrap = false;

	/** Remove internal surfaces from the VoxWrap output, before computing the morphology. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess, meta = (EditCondition = "bVoxWrap == true"))
	bool bRemoveInternalsAfterVoxWrap = false;

	/** Thicken open-boundary surfaces (extrude them inwards) before VoxWrapping them. Units are in world space. If 0, no extrusion is applied. */
	UPROPERTY(EditAnywhere, Category = VoxWrapPreprocess, meta = (EditCondition = "bVoxWrap == true", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	double ThickenShells = 0.0;
};



/**
 * Morphology tool -- dilate, contract, close, open operations on the input shape
 */
UCLASS(MinimalAPI)
class UVoxelMorphologyMeshesTool : public UBaseVoxelTool
{
	GENERATED_BODY()

public:

	UVoxelMorphologyMeshesTool() {}

protected:

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UVoxelMorphologyMeshesToolProperties> MorphologyProperties;

};




UCLASS(MinimalAPI)
class UVoxelMorphologyMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:
	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UVoxelMorphologyMeshesTool>(SceneState.ToolManager);
	}
};

#undef UE_API
