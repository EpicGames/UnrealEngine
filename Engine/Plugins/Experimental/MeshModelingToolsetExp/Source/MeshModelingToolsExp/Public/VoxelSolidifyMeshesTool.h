// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

#include "BaseTools/BaseVoxelTool.h"

#include "VoxelSolidifyMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLSEXP_API




/**
 * Properties of the solidify operation
 */
UCLASS(MinimalAPI)
class UVoxelSolidifyMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Winding number threshold to determine what is consider inside the mesh */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.1", UIMax = ".9", ClampMin = "-10", ClampMax = "10"))
	double WindingThreshold = .5;

	/** How far we allow bounds of solid surface to go beyond the bounds of the original input surface before clamping / cutting the surface off */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"))
	double ExtendBounds = 1;

	/** How many binary search steps to take when placing vertices on the surface */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "6", ClampMin = "0", ClampMax = "10"))
	int SurfaceSearchSteps = 4;

	/** Whether to fill at the border of the bounding box, if the surface extends beyond the voxel boundaries */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bSolidAtBoundaries = true;

	/** If true, uses the ThickenShells setting */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bApplyThickenShells = false;

	/** Thicken open-boundary surfaces (extrude them inwards) to ensure they are captured in the VoxWrap output. Units are in world space. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = ".1", UIMax = "100", ClampMin = ".001", ClampMax = "1000", EditCondition = "bApplyThickenShells == true"))
	double ThickenShells = 5;
};



/**
 * Tool to take one or more meshes, possibly intersecting and possibly with holes, and create a single solid mesh with consistent inside/outside
 */
UCLASS(MinimalAPI)
class UVoxelSolidifyMeshesTool : public UBaseVoxelTool
{
	GENERATED_BODY()

public:

	UVoxelSolidifyMeshesTool() {}

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

protected:

	UPROPERTY()
	TObjectPtr<UVoxelSolidifyMeshesToolProperties> SolidifyProperties;

};


UCLASS(MinimalAPI)
class UVoxelSolidifyMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:
	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UVoxelSolidifyMeshesTool>(SceneState.ToolManager);
	}
};


#undef UE_API
