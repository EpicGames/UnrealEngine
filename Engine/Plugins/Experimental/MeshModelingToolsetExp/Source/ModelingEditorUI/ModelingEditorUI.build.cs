// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingEditorUI : ModuleRules
{
	public ModelingEditorUI(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "CoreUObject",
                "ApplicationCore",
                "Slate",
                "SlateCore",
                "Engine",
                "InputCore",
				"EditorFramework",
				"UnrealEd",
                "ContentBrowser",
				"ContentBrowserData",
				"InteractiveToolsFramework",
                "LevelEditor",
				"ModelingComponents", // FTransformGizmoDataBinder
				"StatusBar",
                "Projects",
				"ToolWidgets",
				"EditorWidgets",
				"DeveloperSettings",
				"Persona",
				"ModelingUI",
				"MeshModelingToolsEditorOnlyExp",
				"MeshModelingTools",
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
