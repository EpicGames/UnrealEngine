// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"

class FModelingEditorUIStyle final : public FSlateStyleSet
{
public:
	
	FModelingEditorUIStyle() : FSlateStyleSet("ModelingEditorUIStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/MeshModelingToolsetExp/Content"));
		
		Set("MeshLayers.DragSolver", new IMAGE_BRUSH("Icons/DragSolver", FVector2D(6, 15)));

		const FSlateColor LayerOutlineColor = FSlateColor(FLinearColor(0.1843f, 0.1843f, 0.1843f, 0.5f));
		const FSlateColor LayerOutlineColorSelected = FSlateColor(FLinearColor(0.1843f, 0.1843f, 0.1843f, 1.0f));
		Set("MeshLayers.LayerBorder", new FSlateRoundedBoxBrush(
			FStyleColors::Header, 4.f,
			LayerOutlineColor, 1.f));
		Set("MeshLayers.SelectedLayerBorder", new FSlateRoundedBoxBrush(
			FStyleColors::Select, 4.f,
			LayerOutlineColorSelected, 1.f));
		
		FCheckBoxStyle TransparentCheckBox = FCheckBoxStyle(FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.SectionButton"))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Transparent))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent))
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Transparent))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent));
		Set("MeshLayers.TransparentCheckBox", TransparentCheckBox);


		FButtonStyle TransparentButton = FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Input, 4.0f, FStyleColors::Input, 1.0f))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
			.SetNormalPadding(FMargin(4.f, 4.f, 4.f, 4.f))
			.SetPressedPadding(FMargin(4.f, 5.f, 4.f, 3.f));
		Set("MeshLayers.TransparentButton", TransparentButton);
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FModelingEditorUIStyle& Get()
	{
		static FModelingEditorUIStyle Inst;
		return Inst;
	}
	
	~FModelingEditorUIStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
