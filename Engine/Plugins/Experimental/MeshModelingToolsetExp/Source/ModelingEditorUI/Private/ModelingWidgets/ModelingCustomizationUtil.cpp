// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/ModelingCustomizationUtil.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SCheckBox.h"

// necessary for export of static constexpr to work (?)
constexpr int UE::ModelingUI::ModelingUIConstants::DetailRowVertPadding;
constexpr int UE::ModelingUI::ModelingUIConstants::LabelWidgetMinPadding;
constexpr int UE::ModelingUI::ModelingUIConstants::MultiWidgetRowHorzPadding;



TSharedRef<SBox> UE::ModelingUI::WrapInFixedWidthBox(TSharedRef<SWidget> SubWidget, int32 Width)
{
	return SNew(SBox)
		.WidthOverride(Width)
		[
			SubWidget
		];
}



TSharedRef<SHorizontalBox> UE::ModelingUI::MakeFixedWidthLabelSliderHBox(
	TSharedPtr<IPropertyHandle> LabelHandle,
	TSharedPtr<SDynamicNumericEntry::FDataSource> SliderDataSource,
	int32 LabelFixedWidth)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			UE::ModelingUI::WrapInFixedWidthBox( 
				LabelHandle->CreatePropertyNameWidget(), LabelFixedWidth )
		]
		+ SHorizontalBox::Slot()
		.Padding(ModelingUIConstants::LabelWidgetMinPadding, 0,0,0)
		.FillWidth(3.0f)		// seems arbitrary....
		[
			SNew(SDynamicNumericEntry)
			.Source(SliderDataSource)
		];
}




TSharedRef<SHorizontalBox> UE::ModelingUI::MakeToggleSliderHBox(
	TSharedPtr<IPropertyHandle> BoolToggleHandle,
	FText ToggleLabelText,
	TSharedPtr<SDynamicNumericEntry::FDataSource> SliderDataSource,
	int32 LabelFixedWidth)
{
	TSharedRef<SDynamicNumericEntry> SliderWidget = SNew(SDynamicNumericEntry)
		.Source(SliderDataSource);

	TFunction<void(bool)> ToggledFunc = [SliderWidget](bool bToggled)
	{
		SliderWidget->SetEnabled(bToggled);
	};

	TSharedRef<SCheckBox> ToggleWidget = MakeBoolToggleButton(BoolToggleHandle, ToggleLabelText, ToggledFunc);

	bool bInitialValue = true;
	BoolToggleHandle->GetValue(bInitialValue);
	SliderWidget->SetEnabled(bInitialValue);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			UE::ModelingUI::WrapInFixedWidthBox( ToggleWidget, LabelFixedWidth )
		]
		+ SHorizontalBox::Slot()
		.Padding(ModelingUIConstants::LabelWidgetMinPadding, 0,0,0)
		.FillWidth(3.0f)		// seems arbitrary....
		[
			SliderWidget
		];
}



TSharedRef<SHorizontalBox> UE::ModelingUI::MakeTwoWidgetDetailRowHBox(
	TSharedRef<SWidget> Widget1,
	TSharedRef<SWidget> Widget2,
	float FillWidth1, float FillWidth2)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0, ModelingUIConstants::DetailRowVertPadding))
		.FillWidth(FillWidth1)
		[
			Widget1
		]

		+ SHorizontalBox::Slot()
		.Padding(FMargin(ModelingUIConstants::MultiWidgetRowHorzPadding, ModelingUIConstants::DetailRowVertPadding, 0, ModelingUIConstants::DetailRowVertPadding))
		.FillWidth(FillWidth2)
		[
			Widget2
		];
}



TSharedRef<SCheckBox> UE::ModelingUI::MakeBoolToggleButton(	
	TSharedPtr<IPropertyHandle> BoolToggleHandle,
	FText ButtonLabelText,
	TFunction<void(bool)> OnToggledCallback,
	int HorzPadding)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.Padding(FMargin(HorzPadding, 2))
		.ToolTipText(BoolToggleHandle->GetToolTipText())
		.HAlign(HAlign_Center)
		.OnCheckStateChanged_Lambda([BoolToggleHandle, LocalToggledFunc=OnToggledCallback](ECheckBoxState NewState)
		{
			bool bSet = (NewState == ECheckBoxState::Checked) ? true : false;
			BoolToggleHandle->SetValue(bSet);
			LocalToggledFunc(bSet);
		})
		.IsChecked_Lambda([BoolToggleHandle]() -> ECheckBoxState
		{
			bool bSet;
			BoolToggleHandle->GetValue(bSet);
			return bSet ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0))
			.AutoWidth()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.Text(ButtonLabelText)
			]
		];
}


void UE::ModelingUI::ProcessChildWidgetsByType(
	const TSharedRef<SWidget>& RootWidget,
	const FString& WidgetType,
	TFunction<bool(TSharedRef<SWidget>&)> ProcessFunc)
{
	auto ProcessChildWidgets = [ProcessFunc](const TSharedRef<SWidget>& Widget, const FString& WidgetType, auto& FindRef) -> void
	{
		FChildren* Children = Widget->GetChildren();
		const int32 NumChild = Children ? Children->NumSlot() : 0;
		for (int32 ChildIdx = 0; ChildIdx < NumChild; ++ChildIdx)
		{
			TSharedRef<SWidget> ChildWidget = Children->GetChildAt(ChildIdx);
			if (ChildWidget->GetTypeAsString().Compare(WidgetType) == 0 && !ProcessFunc(ChildWidget))
			{
				break;
			}
			FindRef(ChildWidget, WidgetType, FindRef);
		}
	};
	ProcessChildWidgets(RootWidget, WidgetType, ProcessChildWidgets);
}


