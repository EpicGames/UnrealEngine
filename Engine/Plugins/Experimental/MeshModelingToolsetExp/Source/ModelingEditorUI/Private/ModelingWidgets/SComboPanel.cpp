// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SComboPanel.h"

#include "Widgets/SOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"

#include "Internationalization/BreakIterator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateStyleMacros.h"

#include "ModelingWidgets/ModelingCustomizationUtil.h"

#define LOCTEXT_NAMESPACE "SComboPanel"


/**
 * Internal class used for icons in the ComboPanel STileView and in the ComboButton
 */
class SComboPanelIconTile : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SComboPanelIconTile)
		: _ThumbnailSize(102, 102)
		, _ExteriorPadding(2.0f)
		, _ImagePadding(FMargin(0))
		, _ThumbnailPadding(FMargin(5.0f, 0))
		, _bShowLabel(true)
		, _IsSelected(false)
		{
		}

		SLATE_ATTRIBUTE(const FSlateBrush*, Image)
		SLATE_ATTRIBUTE(FText, DisplayName)
		/** The size of the thumbnail */
		SLATE_ARGUMENT(FVector2D, ThumbnailSize)
		/** The size of the thumbnail */
		SLATE_ARGUMENT(float, ExteriorPadding)
		/** The size of the thumbnail */
		SLATE_ARGUMENT(FMargin, ImagePadding)
		/** The size of the image in the thumbnail. Used to make the thumbnail not fill the entire thumbnail area. If this is not set the Image brushes desired size is used */
		SLATE_ARGUMENT(TOptional<FVector2D>, ImageSize)
		SLATE_ARGUMENT(FMargin, ThumbnailPadding)
		SLATE_ARGUMENT(bool, bShowLabel)
		SLATE_ATTRIBUTE(bool, IsSelected)
	SLATE_END_ARGS()

public:
	/**
	 * Construct this widget
	 */
	void Construct( const FArguments& InArgs );

private:
	TAttribute<const FSlateBrush*> Image;
	TAttribute<FText> Text;
	TAttribute<bool> IsSelected;
};



void SComboPanelIconTile::Construct(const FArguments& InArgs)
{
	Image = InArgs._Image;
	Text = InArgs._DisplayName;
	IsSelected = InArgs._IsSelected;


	TSharedPtr<SWidget> LabelContent;
	if (InArgs._bShowLabel)
	{
		LabelContent = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(InArgs._ThumbnailPadding)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				.Padding(FMargin(3.0f, 3.0f))
				//.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground"))
				[
					SNew(STextBlock)
					//.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
					.Font( DEFAULT_FONT("Regular", 8) )
					.AutoWrapText(false)
					.LineBreakPolicy(FBreakIterator::CreateWordBreakIterator())
					.Text(InArgs._DisplayName)
					.ColorAndOpacity( FAppStyle::Get().GetSlateColor("Colors.Foreground") )
				]
			];
	}
	else
	{
		LabelContent = SNew(SSpacer);
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(InArgs._ExteriorPadding))
		.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
				// Thumbnail
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(InArgs._ThumbnailSize.X)
					.HeightOverride(InArgs._ThumbnailSize.Y)
					[
						SNew(SBorder)
						.Padding(InArgs._ImagePadding)
						.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Top)
						// TODO: Sort out the color transformation for BorderBackgroundColor to use FAppStyle
						// There is some color transformation beyond just SRGB that cause the colors to appear
						// darker than the actual color.
						//.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Recessed"))
						.BorderBackgroundColor(FLinearColor(0.16f, 0.16f, 0.16f))
						[
							SNew(SImage)
							.Image(Image)
							.DesiredSizeOverride(InArgs._ImageSize)
						]
					]
				]
			]
			+ SOverlay::Slot()
			[
				LabelContent->AsShared()
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.Image_Lambda
				(
					[this]()
					{
						const bool bSelected = IsSelected.Get();
						const bool bHovered = IsHovered();

						if (bSelected && bHovered)
						{
							static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
							return FAppStyle::Get().GetBrush(SelectedHover);
						}
						else if (bSelected)
						{
							static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
							return FAppStyle::Get().GetBrush(Selected);
						}
						else if (bHovered)
						{
							static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
							return FAppStyle::Get().GetBrush(Hovered);
						}

						return FStyleDefaults::GetNoBrush();
					}
				)
			]
		]
	];
}





void SComboPanel::Construct(const FArguments& InArgs)
{
	this->ComboButtonTileSize = InArgs._ComboButtonTileSize;
	this->FlyoutTileSize = InArgs._FlyoutTileSize;
	this->FlyoutSize = InArgs._FlyoutSize;
	this->Items = InArgs._ListItems;
	this->OnSelectionChanged = InArgs._OnSelectionChanged;

	MissingIcon = MakeShared<FSlateBrush>();

	// Check for any parameters that the coder forgot to specify.
	if ( Items.Num() == 0 )
	{
		UE::ModelingUI::SetCustomWidgetErrorString(LOCTEXT("MissingListItemsError", "Please specify a ListItems with at least one Item."), this->ChildSlot);
		return;
	}

	TileView = SNew(STileView<TSharedPtr<FComboPanelItem>>)
		.ListItemsSource( &Items )
		.OnGenerateTile(this, &SComboPanel::CreateFlyoutIconTile)
		.OnSelectionChanged(this, &SComboPanel::FlyoutSelectionChanged)
		.ClearSelectionOnClick(false)
		.ItemAlignment(EListItemAlignment::LeftAligned)
		.ItemWidth(InArgs._FlyoutTileSize.X)
		.ItemHeight(InArgs._FlyoutTileSize.Y)
		.SelectionMode(ESelectionMode::Single);

	TSharedPtr<SWidget> ComboButtonContent;
	if (InArgs._ComboDisplayType == EComboDisplayType::IconAndLabel)
	{
		ComboButtonContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinAspectRatio(1.0f)
				.MaxAspectRatio(1.0f)
				[
					SNew(SComboPanelIconTile)
					.Image_Lambda([this]() { return (SelectedItem->Icon != nullptr) ? SelectedItem->Icon : MissingIcon.Get(); })
					.DisplayName_Lambda([this]() { return SelectedItem->Name; })
					.bShowLabel(false)
					.ThumbnailSize(ComboButtonTileSize)
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4,0,0,0)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.AutoWrapText(false)
				.Text_Lambda([this]() { return SelectedItem->Name; })
			];
	}
	else
	{
		ComboButtonContent = SNew(SComboPanelIconTile)
			.Image_Lambda([this]() { return (SelectedItem->Icon != nullptr) ? SelectedItem->Icon : MissingIcon.Get(); })
			.DisplayName_Lambda([this]() { return SelectedItem->Name; })
			.ExteriorPadding(0.0f)
			.ImagePadding(FMargin(0,2,0,0))
			.ImageSize( FVector2D(ComboButtonTileSize.X-15, ComboButtonTileSize.Y-15) )
			.ThumbnailSize(ComboButtonTileSize);
	}

	ComboButton = SNew(SComboButton)
		//.ToolTipText(BrushTypeHandle->GetToolTipText())		// how to do this? SWidget has TooltipText...
		.HasDownArrow(false)
		.OnGetMenuContent( this, &SComboPanel::OnGetMenuContent )
		.OnMenuOpenChanged( this, &SComboPanel::OnMenuOpenChanged )
		//.IsEnabled(IsEnabledAttribute)
		.ContentPadding(FMargin(0))
		.ButtonContent()
		[
			ComboButtonContent->AsShared()
		];

	int InitialSelectionIndex = FMath::Clamp(InArgs._InitialSelectionIndex, 0, Items.Num());
	SelectedItem = Items[InitialSelectionIndex];
	TileView->SetSelection(SelectedItem);

	ChildSlot
	[
		ComboButton->AsShared()
	];
}

void SComboPanel::SetSelectionIndex(int InSelectionIndex)
{
	int SelectionIndex = FMath::Clamp(InSelectionIndex, 0, Items.Num());

	SelectedItem = Items[SelectionIndex];
	TileView->SetSelection(SelectedItem);
}

TSharedRef<SWidget> SComboPanel::OnGetMenuContent()
{
	bool bInShouldCloseWindowAfterMenuSelection = true;
	bool bCloseSelfOnly = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly);
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrushesHeader", "Brush Types"));
	{
		TSharedPtr<SWidget> MenuContent;
		MenuContent =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(TileViewContainer, SBox)
				.Padding(6.0f)
				.MinDesiredWidth( FlyoutSize.X )
			];

		TileViewContainer->SetContent(TileView->AsShared());

		MenuBuilder.AddWidget(MenuContent.ToSharedRef(), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SComboPanel::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		ComboButton->SetMenuContent(SNullWidget::NullWidget);
	}
}


TSharedRef<ITableRow> SComboPanel::CreateFlyoutIconTile(
	TSharedPtr<FComboPanelItem> Item,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow< TSharedPtr<FComboPanelItem> >, OwnerTable)
		.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
		.ToolTipText( Item->Name )
		.Padding(2.0f)
		[
			SNew(SComboPanelIconTile)
			.Image( (Item->Icon != nullptr) ? Item->Icon : MissingIcon.Get() )
			.DisplayName( Item->Name )
			.ThumbnailSize( FlyoutTileSize )
			.ImagePadding(FMargin(0,4,0,0))
			.ImageSize( FVector2D(FlyoutTileSize.X-25, FlyoutTileSize.Y-25) )
			.IsSelected_Lambda( [this, Item] { return Item.Get() == SelectedItem.Get(); })
		];
}


void SComboPanel::FlyoutSelectionChanged(
	TSharedPtr<FComboPanelItem> SelectedItemIn,
	ESelectInfo::Type SelectInfo)
{
	SelectedItem = SelectedItemIn;

	if (ComboButton.IsValid())
	{
		ComboButton->SetIsOpen(false);
	}

	if ( OnSelectionChanged.IsBound() )
	{
		OnSelectionChanged.ExecuteIfBound(SelectedItem);
	}	
}


#undef LOCTEXT_NAMESPACE