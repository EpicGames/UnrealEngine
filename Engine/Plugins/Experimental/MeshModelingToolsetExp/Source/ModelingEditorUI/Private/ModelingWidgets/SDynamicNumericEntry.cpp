// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SDynamicNumericEntry.h"

#include "Editor.h"
#include "Styling/AppStyle.h"
#include "PropertyHandle.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Math/Interval.h"

#define LOCTEXT_NAMESPACE "SDynamicNumericEntry"


void SDynamicNumericEntry::Construct(const FArguments& InArgs)
{
	this->Source = InArgs._Source;

	this->TypeInterface = MakeShared<TDefaultNumericTypeInterface<float>>();

	SAssignNew(NumericEntry, SNumericEntryBox<float>)
	.AllowSpin(true)
	.Value_Lambda([this]() { return this->Source->GetValue(); })
	//.Font(InArgs._Font)
	.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )		// standard details panel font
	.MinValue_Lambda([this]() { return this->Source->GetValueRange().Min; })
	.MaxValue_Lambda([this]() { return this->Source->GetValueRange().Max; })
	.MinSliderValue_Lambda([this]() { return this->Source->GetUIRange().Min; })
	.MaxSliderValue_Lambda([this]() { return this->Source->GetUIRange().Max; })
	//.SliderExponent(SliderExponent)
	//.Delta(Delta)
	.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
	.OnValueChanged(this, &SDynamicNumericEntry::OnValueChanged)
	.OnValueCommitted(this, &SDynamicNumericEntry::OnValueCommitted)
	//.OnUndeterminedValueCommitted(this, &SPropertyEditorNumeric<NumericType>::OnUndeterminedValueCommitted)
	.OnBeginSliderMovement(this, &SDynamicNumericEntry::OnBeginSliderMovement)
	.OnEndSliderMovement(this, &SDynamicNumericEntry::OnEndSliderMovement)
	.TypeInterface(this->TypeInterface)
	;

	this->TypeInterface->SetMaxFractionalDigits(InArgs._MaxNumFloatDigits);

	ChildSlot
	[
		NumericEntry->AsShared()
	];
}



TSharedPtr<SDynamicNumericEntry::FDataSource> SDynamicNumericEntry::MakeSimpleDataSource(
	TSharedPtr<IPropertyHandle> PropertyHandle, 
	TInterval<float> ValueRange, 
	TInterval<float> UIRange)
{
	TSharedPtr<SDynamicNumericEntry::FDataSource> NumericSource = MakeShared<SDynamicNumericEntry::FDataSource>();
	NumericSource->SetValue = [PropertyHandle](float NewSize, EPropertyValueSetFlags::Type Flags)
	{
		PropertyHandle->SetValue(NewSize, Flags);
	};
	NumericSource->GetValue = [PropertyHandle]() -> float
	{
		float Size;
		PropertyHandle->GetValue(Size);
		return Size;
	};
	NumericSource->GetValueRange = [ValueRange]() -> TInterval<float>
	{
		return ValueRange;
	};
	NumericSource->GetUIRange = [UIRange]() -> TInterval<float>
	{
		return UIRange;
	};

	return NumericSource;
}



void SDynamicNumericEntry::OnValueChanged( float NewValue )
{
	if( bIsUsingSlider )
	{
		// Value hasn't changed, so lets return now
		if (Source->GetValue() == NewValue)
		{ 
			return;
		}

		EPropertyValueSetFlags::Type Flags = (EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
		Source->SetValue(NewValue, Flags);

		//if (TypeInterface.IsValid() && !TypeInterface->FixedDisplayUnits.IsSet())
		//{
		//	TypeInterface->SetupFixedDisplay(NewValue);
		//}

	}
}

void SDynamicNumericEntry::OnValueCommitted( float NewValue, ETextCommit::Type CommitInfo )
{
	if (bIsUsingSlider || Source->GetValue() != NewValue)
	{
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags;
		Source->SetValue(NewValue, Flags);
		LastSliderCommittedValue = NewValue;
	}

	//if (TypeInterface.IsValid() && !TypeInterface->FixedDisplayUnits.IsSet())
	//{
	//	TypeInterface->SetupFixedDisplay(NewValue);
	//}

}


/**
* Called when the slider begins to move.  We create a transaction here to undo the property
*/
void SDynamicNumericEntry::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	LastSliderCommittedValue = Source->GetValue();

	//GEditor->BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetNumericPropertyTransaction", "Edit {0}"), PropertyEditor->GetDisplayName()), nullptr );
	GEditor->BeginTransaction(LOCTEXT("EditNumericEntry", "Edit Number"));
}


/**
* Called when the slider stops moving.  We end the previously created transaction
*/
void SDynamicNumericEntry::OnEndSliderMovement( float NewValue )
{
	bIsUsingSlider = false;

	// When the slider end, we may have not called SetValue(NewValue) without the InteractiveChange|NotTransactable flags.
	//That prevents some transaction and callback to be triggered like the NotifyHook.
	if (LastSliderCommittedValue != NewValue)
	{
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags;
		Source->SetValue(NewValue, Flags);
	}
	else
	{
		GEditor->EndTransaction();
	}

}





#undef LOCTEXT_NAMESPACE