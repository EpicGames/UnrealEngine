// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SMeshLayers.h"

#include "ModelingEditorUIStyle.h"
#include "ModelingWidgets/SculptLayersController.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SMeshLayers"

float SMeshLayerItem::LayersHorizontalPadding = 6.0f;;
float SMeshLayerItem::LayersVerticalPadding = 3.0f;

//
// FMeshLayerElement
//

TSharedRef<ITableRow> FMeshLayerElement::MakeListRowWidget(
	const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FMeshLayerElement> InMeshLayerElement,
	TSharedPtr<SMeshLayersList> InMeshLayersListWidget)
{
	return SNew(SMeshLayerItem, InOwnerTable)
		.InMeshLayerElement(InMeshLayerElement)
		.InMeshLayersList(InMeshLayersListWidget);
}

//
// SMeshLayerList
//

void SMeshLayersList::Construct(const FArguments& InArgs)
{
	Controller = InArgs._InController;
	
	SListView::FArguments SuperArgs;
	SuperArgs.ListItemsSource(&Layers);
	SuperArgs.SelectionMode(ESelectionMode::Single);
	SuperArgs.OnGenerateRow(this, &SMeshLayersList::MakeListRowWidget);
	SuperArgs.OnSelectionChanged(this, &SMeshLayersList::OnSelectionChanged);
	SuperArgs.OnMouseButtonClick(this, &SMeshLayersList::OnItemClicked);

	SListView::Construct(SuperArgs);

	bAllowReordering = InArgs._InAllowReordering;
	bAllowAddRemove = InArgs._InAllowAddRemove;
}

TSharedRef<ITableRow> SMeshLayersList::MakeListRowWidget(TSharedPtr<FMeshLayerElement> InLayerElement,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return InLayerElement->MakeListRowWidget(OwnerTable,InLayerElement.ToSharedRef(),SharedThis(this));
}

FReply SMeshLayersList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TArray<TSharedPtr<FMeshLayerElement>> SelectedLayers = GetSelectedItems();
	if (!SelectedLayers.IsEmpty() && InKeyEvent.GetKey() == EKeys::Delete)
	{
		DeleteMeshLayer(SelectedLayers.Last());
		return FReply::Handled();
	}
	
	return SListView::OnKeyDown(MyGeometry, InKeyEvent);
}

void SMeshLayersList::DeleteMeshLayer(TSharedPtr<FMeshLayerElement> LayerToDelete)
{
	if (!Controller.IsValid() || !LayerToDelete.IsValid())
	{
		return;
	}

	// remove the layer
	if (!Controller.Pin()->RemoveMeshLayer(LayerToDelete->GetIndexInStack()))
	{
		return;
	}

	// Note: The controller will indirectly trigger a RefreshAndRestore() call via RemoveMeshLayer
	// so we don't need to manually update the list post-removal.
}

void SMeshLayersList::RefreshAndRestore()
{
	IMeshLayersController* Ctrl = Controller.Pin().Get();
	if (!ensure(Ctrl))
	{
		return;
	}
	RequestListRefresh();
}

void SMeshLayersList::OnSelectionChanged(TSharedPtr<FMeshLayerElement> InLayer, ESelectInfo::Type SelectInfo) const
{
	// adds support for keyboard navigation of layer stack
	if (SelectInfo == ESelectInfo::OnNavigation || SelectInfo == ESelectInfo::Direct)
	{
		if (Controller.IsValid() && InLayer.IsValid())
		{
			Controller.Pin()->SetActiveLayer(InLayer->GetIndexInStack());
		}
		return;
	}

	// set first layer in stack as selected layer
	if (!InLayer.IsValid() && Controller.IsValid())
	{
		Controller.Pin()->SetActiveLayer(0);
	}
}

void SMeshLayersList::OnItemClicked(TSharedPtr<FMeshLayerElement> InLayer)
{
	// to rename an item, you have to select it first, then click on it again
	const bool ClickedOnSameItem = LastSelectedLayer.Pin().Get() == InLayer.Get();
	if (ClickedOnSameItem)
	{
		RegisterActiveTimer(0.f,
			FWidgetActiveTimerDelegate::CreateLambda([this](double, float)
			{
				RequestRenameSelectedLayer();
				return EActiveTimerReturnType::Stop;
			}));
	}

	LastSelectedLayer = InLayer;

	if (Controller.IsValid() && InLayer.IsValid())
	{
		Controller.Pin()->SetActiveLayer(InLayer->GetIndexInStack());
	}
}

void SMeshLayersList::RequestRenameSelectedLayer() const
{
	if (!LastSelectedLayer.IsValid())
	{
		return;
	}
	LastSelectedLayer.Pin()->OnRenameRequested.ExecuteIfBound();
}

FReply SMeshLayersList::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bAllowReordering)
	{
		return FReply::Unhandled();
	}
	const TArray<TSharedPtr<FMeshLayerElement>> CurrentSelectedItems = GetSelectedItems();
	if (CurrentSelectedItems.Num() != 1)
	{
		return FReply::Unhandled();
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedPtr<FMeshLayerElement> DraggedElement = CurrentSelectedItems[0];
		const TSharedRef<FMeshLayerStackDragDropOp> DragDropOp = FMeshLayerStackDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SMeshLayersList::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone,
	TSharedPtr<FMeshLayerElement> TargetLayer)
{
	TOptional<EItemDropZone> ReturnedDropZone;
	const TSharedPtr<FMeshLayerStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FMeshLayerStackDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return ReturnedDropZone;
	}

	const TSharedPtr<FMeshLayerElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	if (!DraggedElement)
	{
		return ReturnedDropZone;
	}

	// validate index to move to
	const int32 IndexToMoveTo = FMeshLayerStackDragDropOp::GetIndexToMoveTo(DraggedElement, TargetLayer, DropZone);

	// GetIndexToMoveTo returns index none for invalid moves
	if (IndexToMoveTo == INDEX_NONE)
	{
		return ReturnedDropZone;
	}
	
	ReturnedDropZone = DropZone;
	return ReturnedDropZone;
}

FReply SMeshLayersList::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FMeshLayerElement> TargetLayer)
{
	const TSharedPtr<FMeshLayerStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FMeshLayerStackDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const TSharedPtr<IMeshLayersController> Ctrl = Controller.Pin();
	if (!Ctrl.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<FMeshLayerElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	// stack index
	const int32 IndexToMoveTo = FMeshLayerStackDragDropOp::GetIndexToMoveTo(DraggedElement, TargetLayer, DropZone);
	if (IndexToMoveTo == INDEX_NONE)
	{
		// don't do anything if the drop location is invalid
		return FReply::Unhandled();
	}
	
	const int32 LayerToMoveIndex = DraggedElement->GetIndexInStack();
	// perform move in underlying property set
	Ctrl->MoveLayerInStack(LayerToMoveIndex, IndexToMoveTo);

	// also need to perform move in Layers (array of MeshElements) here
	const int Dir = IndexToMoveTo > LayerToMoveIndex ? 1 : -1;
	int32 CurrentLayerIndex = LayerToMoveIndex;
	// perform move by swapping with previous/next layer in stack until layer is in desired location
	while (CurrentLayerIndex != IndexToMoveTo)
	{
		Layers.Swap(CurrentLayerIndex, CurrentLayerIndex + Dir);
		// adjusting indices as we go 
		Layers[CurrentLayerIndex]->SetIndexInStack(CurrentLayerIndex);
		Layers[CurrentLayerIndex  + Dir]->SetIndexInStack(CurrentLayerIndex + Dir);
		CurrentLayerIndex += Dir;
	}

	// if we are moving the active layer, update our ActiveLayer to reflect new index
	const int32 ActiveLayer = Ctrl->GetActiveLayer();
	if (ActiveLayer == LayerToMoveIndex)
	{
		Ctrl->SetActiveLayer(IndexToMoveTo);
	}
	// if we are placing a layer before the ActiveLayer (that was previously after it), increment ActiveLayer's index
	else if (ActiveLayer >= IndexToMoveTo && LayerToMoveIndex > ActiveLayer)
	{
		Ctrl->SetActiveLayer(ActiveLayer + 1);
	}
	// if we are moving a layer that was before the ActiveLayer to be after it, decrement ActiveLayer index
	else if (ActiveLayer > LayerToMoveIndex && ActiveLayer <= IndexToMoveTo)
	{
		Ctrl->SetActiveLayer(ActiveLayer - 1);
	}
	Ctrl->RefreshLayersStackView();
	
	return FReply::Handled();
}

//
// SMeshLayerItem
//

void SMeshLayerItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	ListView = InArgs._InMeshLayersList;
	Element = InArgs._InMeshLayerElement;

	SHorizontalBox::FArguments BoxArgs;

	// if reordering is enabled, add drag icon to UI
	if (ListView.Pin()->GetAllowReordering())
	{
		// drag icon
		BoxArgs
		+ SHorizontalBox::Slot()
		.MaxWidth(18.f)
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(LayersHorizontalPadding, LayersVerticalPadding)
		[
			SNew(SImage)
			.Image(FModelingEditorUIStyle::Get().GetBrush("MeshLayers.DragSolver"))
		];
	}

	// enable Checkbox - TODO
	// Note the code for this checkbox is intentionally left here but disabled, as the underlying feature doesn't exist yet
	// but this has the desired UI look for when it does
#if 0
	BoxArgs
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
	[
		SNew(SBox)
		[
			SNew(SCheckBox)
			.Style(FModelingEditorUIStyle::Get().Get(), "MeshLayers.TransparentCheckBox")
			.Padding(FMargin(4, 2))
			.HAlign(HAlign_Center)
			.IsEnabled(false) // TODO
			.ToolTipText( LOCTEXT("EnabledTooltip", "Enables/disables the mesh layer") )
			.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
			{
				//IMeshLayersController* Ctrl = ListView.Pin()->Controller.Pin().Get();
				//Ctrl->AssetController->SetMeshLayerEnabled( Element.Pin()->GetIndexInStack(), bIsChecked);
			})
			.IsChecked_Lambda([this]()
			{
				//IMeshLayersController* Ctrl = ListView.Pin()->Controller.Pin().Get();
				//bool bEnabled = Ctrl->AssetController->GetMeshLayerEnabled(Element.Pin()->GetIndexInStack());
				//return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				return ECheckBoxState::Checked;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0))
				.AutoWidth()
				[
					SNew(SImage)
					.Image_Lambda([this]()
					{
						return IsLayerEnabled() ?
							FAppStyle::Get().GetBrush(TEXT("Level.VisibleIcon16x")) :
						FAppStyle::Get().GetBrush(TEXT("Level.NotVisibleIcon16x"));
					})
				]
			]
		]
	];
#endif

	// display Name
	BoxArgs
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
	[
		SAssignNew(EditNameWidget, SInlineEditableTextBlock)
		.Text_Lambda([this]()
		{
			const IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
			const FName LayerName = Ctrl->GetLayerName(Element.Pin()->GetIndexInStack());
			return FText::FromName(LayerName);
		})
		.OnVerifyTextChanged_Lambda([](const FText& InText, FText& OutErrorMessage)
		{
			static FString IllegalNameCharacters = "^<>:\"/\\|?*";
			return FName::IsValidXName(InText.ToString(), FString(IllegalNameCharacters), &OutErrorMessage);
		})
		.OnTextCommitted(this, &SMeshLayerItem::OnNameCommitted)
		.MultiLine(false)
	];
	
	// spacer
	BoxArgs
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Fill)
	.FillWidth(1.0f)
	[
		SNew(SSpacer)
		.Size(FVector2D(0.0f, 1.0f))
	];
	
	// spin Box
	BoxArgs
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
	[
		SNew(SSpinBox<double>)
		.Value_Lambda([this]() -> double
		{
			IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
			return Ctrl->GetLayerWeight(Element.Pin()->GetIndexInStack());
		})
		.OnBeginSliderMovement_Lambda([this]()
		{
			GEditor->BeginTransaction(LOCTEXT("MeshLayers_SetWeight_Slider", "Set Layer Weight"));
		})
		.OnValueChanged_Lambda([this](const float NewValue)
		{
			// in the case of text entry, no transaction has been created yet
			const bool bIsTextEntry = !GEditor->IsTransactionActive();
			if (bIsTextEntry)
			{
				GEditor->BeginTransaction(LOCTEXT("MeshLayers_SetWeight_ValueChange", "Set Layer Weight"));
			}
			IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
			Ctrl->SetLayerWeight(Element.Pin()->GetIndexInStack(), NewValue, EPropertyChangeType::Interactive);
			if (bIsTextEntry)
			{
				GEditor->EndTransaction();
			}
		})
		// only log the slider movement after its complete
		.OnEndSliderMovement_Lambda([this](float LastValue)
		{
			IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
			Ctrl->SetLayerWeight(Element.Pin()->GetIndexInStack(), LastValue, EPropertyChangeType::ValueSet);
			GEditor->EndTransaction();
		})
		.MinSliderValue(-1.0f)
		.MaxSliderValue(1.5f)
		.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
	];

	if (ListView.Pin()->GetAllowAddRemove())
	{
		// delete Button
		BoxArgs
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
		[
			SNew(SButton)
			.ContentPadding(FMargin(0,0))
			.ToolTipText(LOCTEXT("DeleteLayer", "Delete mesh layer and remove from stack."))
			.ButtonStyle(FModelingEditorUIStyle::Get().Get(), "MeshLayers.TransparentButton")
			.OnClicked_Lambda([this]() -> FReply
			{
				ListView.Pin()->DeleteMeshLayer(Element.Pin());
				const IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
				Ctrl->RefreshLayersStackView();
				return FReply::Handled();
			})
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	STableRow::Construct(
		STableRow::FArguments()
		.OnDragDetected(ListView.Pin().Get(), &SMeshLayersList::OnDragDetected)
		.OnCanAcceptDrop(ListView.Pin().Get(), &SMeshLayersList::OnCanAcceptDrop)
		.OnAcceptDrop(ListView.Pin().Get(), &SMeshLayersList::OnAcceptDrop)
		.ShowSelection(false)
		.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
		.Content()
		[
			SNew(SBorder)
			.BorderImage_Lambda([this]()
			{
				// display highlight on selected layer only
				if (Element.Pin()->GetIndexInStack() == ListView.Pin()->GetController().Pin()->GetActiveLayer())
				{
					return FModelingEditorUIStyle::Get().GetBrush("MeshLayers.SelectedLayerBorder");
				}
				return FModelingEditorUIStyle::Get().GetBrush("MeshLayers.LayerBorder");
			})
			.Padding(FMargin(LayersHorizontalPadding, LayersVerticalPadding))
			[
				SArgumentNew(BoxArgs, SHorizontalBox)
			]
		], OwnerTable);
	
	// bind the rename callback to enter text editing mode when item is double-clicked
	Element.Pin()->OnRenameRequested.BindSP(EditNameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
}

void SMeshLayerItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	GEditor->BeginTransaction(LOCTEXT("MeshLayers_ChangeName", "Change Mesh Layer Name"));
	const FName NewName = FName(InText.ToString());
	const IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
	Ctrl->SetLayerName(Element.Pin()->GetIndexInStack(), NewName);
	GEditor->EndTransaction();
}

FText SMeshLayerItem::GetName() const
{
	const IMeshLayersController* Ctrl = ListView.Pin()->GetController().Pin().Get();
	const FName LayerName = Ctrl->GetLayerName(Element.Pin()->GetIndexInStack());
	return FText::FromName(LayerName);
}

bool SMeshLayerItem::IsLayerEnabled() const
{
	// TODO
	
	return true;
}

//
// FMeshLayerStackDragDropOp
//


TSharedRef<FMeshLayerStackDragDropOp> FMeshLayerStackDragDropOp::New(TWeakPtr<FMeshLayerElement> InElement)
{
	TSharedRef<FMeshLayerStackDragDropOp> Operation = MakeShared<FMeshLayerStackDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FMeshLayerStackDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(FText::FromName(Element.Pin()->GetLayerList().Pin()->GetController().Pin()->
				GetLayerName(Element.Pin()->GetIndexInStack())))
		];
}

int32 FMeshLayerStackDragDropOp::GetIndexToMoveTo(
	const TSharedPtr<FMeshLayerElement>& InDraggedElement,
	const TSharedPtr<FMeshLayerElement>& InTargetElement,
	const EItemDropZone InDropZone)
{
	// disallow dropping on self
	const int32 DraggedItemIndex = InDraggedElement->GetIndexInStack();
	const int32 TargetItemIndex = InTargetElement->GetIndexInStack();
	if (DraggedItemIndex == TargetItemIndex)
	{
		return INDEX_NONE;
	}

	// disallow dropping in a place that would not change the order (like below the one above, or above the one below)
	int32 IndexToMoveDragItemTo = INDEX_NONE;
	bool bDraggingDown = DraggedItemIndex <= TargetItemIndex;
	switch (InDropZone)
	{
	case EItemDropZone::AboveItem:
		IndexToMoveDragItemTo = bDraggingDown ? TargetItemIndex - 1 : TargetItemIndex;
		break;
	case EItemDropZone::BelowItem:
		IndexToMoveDragItemTo = bDraggingDown ? TargetItemIndex : TargetItemIndex + 1;
		break;
	case EItemDropZone::OntoItem:
		IndexToMoveDragItemTo = TargetItemIndex;
		break;
	default:
		checkNoEntry();
	}
	
	if (DraggedItemIndex == IndexToMoveDragItemTo)
	{
		return INDEX_NONE;
	}

	return IndexToMoveDragItemTo;
}

#undef LOCTEXT_NAMESPACE
