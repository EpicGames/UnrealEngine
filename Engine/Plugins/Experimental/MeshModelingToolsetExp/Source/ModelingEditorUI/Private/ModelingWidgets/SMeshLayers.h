// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Views/SListView.h"

class SMeshLayersList;
class SInlineEditableTextBlock;
class IMeshLayersController;

/**
 * Represents a single mesh layer within the List
 */
class FMeshLayerElement
{
public:
	static TSharedRef<ITableRow> MakeListRowWidget(
		const TSharedRef<STableViewBase>& InOwnerTable,
		TSharedRef<FMeshLayerElement> InMeshLayerElement,
		TSharedPtr<SMeshLayersList> InMeshLayerListWidget);

	static TSharedRef<FMeshLayerElement> Make(
		const int32 InLayerIndex,
		const TSharedPtr<SMeshLayersList>& InLayerListWidget)
	{
		FMeshLayerElement* NewElement = new FMeshLayerElement(InLayerIndex, InLayerListWidget);
		return MakeShareable(NewElement);
	}
	
	int32 GetIndexInStack() const { return IndexInStack; };
	void SetIndexInStack(const int32 InIndex) { IndexInStack = InIndex;}
	
	TWeakPtr<SMeshLayersList> GetLayerList() const { return LayerListWidget; };
	
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;
private:

	int32 IndexInStack = INDEX_NONE;
	TWeakPtr<SMeshLayersList> LayerListWidget;
	
	// hidden constructors, always use Make above
	FMeshLayerElement() = default;
	FMeshLayerElement(
		const int32 InIndexInStack,
		const TSharedPtr<SMeshLayersList>& InLayerListWidget) :
		IndexInStack(InIndexInStack),
		LayerListWidget(InLayerListWidget){}
};

/**
 * Represents the list of mesh layers - works within the MeshLayersStack widget
 */
class SMeshLayersList : public SListView<TSharedPtr<FMeshLayerElement>>
{
public:
	SLATE_BEGIN_ARGS(SMeshLayersList)
		: _InAllowAddRemove(true)
		, _InAllowReordering(true){}
		SLATE_ARGUMENT(TWeakPtr<IMeshLayersController>, InController)
		// TODO put these props on the controller ?
		SLATE_ARGUMENT(bool, InAllowAddRemove)
		SLATE_ARGUMENT(bool, InAllowReordering)
	SLATE_END_ARGS()

	virtual ~SMeshLayersList() = default;

	void Construct(const FArguments& InArgs);

	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FMeshLayerElement> InLayerElement, const TSharedRef<STableViewBase>& OwnerTable);
	
	// drag and drop operations
	FReply OnDragDetected(
		const FGeometry& MyGeometry,
		const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> OnCanAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FMeshLayerElement> TargetLayer);
	FReply OnAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FMeshLayerElement> TargetLayer);
	// end drag and drop operations

	// renaming
	void OnSelectionChanged(TSharedPtr<FMeshLayerElement> InLayer, ESelectInfo::Type SelectInfo) const;
	void OnItemClicked(TSharedPtr<FMeshLayerElement> InLayer);
	void RequestRenameSelectedLayer() const;
	// END renaming
	
	// delete layer from stack
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	void DeleteMeshLayer(TSharedPtr<FMeshLayerElement> LayerToDelete);

	// must be called after refresh
	void RefreshAndRestore();
	
	TArray<TSharedPtr<FMeshLayerElement>>& GetLayers() { return Layers; }
	TWeakPtr<IMeshLayersController> GetController() { return Controller; }

	bool GetAllowReordering() const { return bAllowReordering;};
	bool GetAllowAddRemove() const { return bAllowAddRemove;}

private:
	bool bAllowReordering = true;
	bool bAllowAddRemove = true;
	
	// the layers contained in this list
	TArray<TSharedPtr<FMeshLayerElement>> Layers;
	
	TWeakPtr<IMeshLayersController> Controller;

	TWeakPtr<FMeshLayerElement> LastSelectedLayer;
};

/**
 * Builds a UI element representing a single mesh layer
 */
class SMeshLayerItem : public STableRow<TSharedPtr<FMeshLayerElement>>
{
public:
	SLATE_BEGIN_ARGS(SMeshLayerItem) {}
		SLATE_ARGUMENT(TWeakPtr<SMeshLayersList>, InMeshLayersList)
		SLATE_ARGUMENT(TWeakPtr<FMeshLayerElement>, InMeshLayerElement)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);

	// RENAMING op
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	FText GetName() const;
	// END renaming op

	static float LayersHorizontalPadding;
	static float LayersVerticalPadding;
	
private:

	bool IsLayerEnabled() const;
	
	TWeakPtr<SMeshLayersList> ListView;
	TWeakPtr<FMeshLayerElement> Element;
	TSharedPtr<SInlineEditableTextBlock> EditNameWidget;
};

/**
 * Support for drag and drop operations within the MeshLayers
 */
class FMeshLayerStackDragDropOp : public FDecoratedDragDropOp
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FMeshLayerStackDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FMeshLayerStackDragDropOp> New(TWeakPtr<FMeshLayerElement> InElement);
	
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	
	static int32 GetIndexToMoveTo(
		const TSharedPtr<FMeshLayerElement>& InDraggedElement,
		const TSharedPtr<FMeshLayerElement>& InTargetElement,
		const EItemDropZone InDropZone);
	
	TWeakPtr<FMeshLayerElement> Element;
};
