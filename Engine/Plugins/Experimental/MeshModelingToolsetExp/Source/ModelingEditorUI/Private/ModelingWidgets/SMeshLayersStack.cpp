// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SMeshLayersStack.h"

#include "ModelingWidgets/SculptLayersController.h"
#include "ModelingWidgets/SMeshLayers.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "SMeshLayersStack"

void SMeshLayersStack::Construct(
	const FArguments& InArgs,
	const TWeakPtr<IMeshLayersController>& InController)
{
	Controller = InController;
	Controller.Pin()->SetLayerStackView(SharedThis(this));

	SVerticalBox::FArguments BoxArgs;

	if (InArgs._InAllowAddRemove)
	{
		BoxArgs
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(6.0f, 4.0f)
					[
						SNew(SPositiveActionButton)
							.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
							.Text(LOCTEXT("AddNewMeshLayerLabel", "Add Layer"))
							.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new mesh layer"))
							.OnClicked_Lambda([this]()
							{
								IMeshLayersController* AssetController = Controller.Pin().Get();
								if (!AssetController)
								{
									return FReply::Unhandled();
								}

								// add a new layer to the stack
								AssetController->AddMeshLayer();
								RefreshStackView();
								return FReply::Handled();
							})
					]
				]
			]
		];
	}

	BoxArgs
	+SVerticalBox::Slot()
	.Padding(0.0f)
	.HAlign(HAlign_Fill)
	[
		SAssignNew( ListView, SMeshLayersList )
		.InController(Controller)
		.InAllowReordering(InArgs._InAllowReordering)
		.InAllowAddRemove(InArgs._InAllowAddRemove)
	];

	ChildSlot
	[
		SArgumentNew(BoxArgs, SVerticalBox)
	];

	RefreshStackView();
}

void SMeshLayersStack::RefreshStackView() const
{
	const TSharedPtr<IMeshLayersController> SLController = Controller.Pin();
	if (!SLController.IsValid())
	{
		return; 
	}

	// rebuild list of top level elements
	const int32 NumLayers = SLController->GetNumMeshLayers();
	
	// empty old elements
	ListView->GetLayers().Reset();

	// rebuild list of top level elements
	for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		// add op to main stack
		TSharedPtr<FMeshLayerElement> StackElement = FMeshLayerElement::Make(LayerIdx, ListView);
		ListView->GetLayers().Add(StackElement);
	}

	// refresh the list and restore the selection
	ListView->RefreshAndRestore();
}

#undef LOCTEXT_NAMESPACE