// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/STickDelegateWidget.h"

void STickDelegateWidget::Construct(const FArguments& InArgs)
{
	OnTickDelegate = InArgs._OnTickDelegate;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void STickDelegateWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	OnTickDelegate.ExecuteIfBound(AllottedGeometry, InCurrentTime, InDeltaTime);

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}