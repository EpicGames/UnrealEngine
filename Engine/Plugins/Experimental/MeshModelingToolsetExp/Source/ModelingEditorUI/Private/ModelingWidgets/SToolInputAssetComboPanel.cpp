// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SToolInputAssetComboPanel.h"

#include "CollectionManagerModule.h"
#include "ICollectionManager.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/SListView.h"
#include "SAssetView.h"
#include "SAssetDropTarget.h"

#include "Editor.h"

#include "ModelingWidgets/SToolInputAssetPicker.h"
#include "ModelingWidgets/ModelingCustomizationUtil.h"

#include "Styling/SlateStyleMacros.h"

#define LOCTEXT_NAMESPACE "SToolInputAssetComboPanel"


void SToolInputAssetComboPanel::Construct(const FArguments& InArgs)
{
	this->ComboButtonTileSize = InArgs._ComboButtonTileSize;
	this->FlyoutTileSize = InArgs._FlyoutTileSize;
	this->FlyoutSize = InArgs._FlyoutSize;
	this->Property = InArgs._Property;
	this->CollectionSets = InArgs._CollectionSets;
	this->OnSelectionChanged = InArgs._OnSelectionChanged;
	RecentAssetsProvider = InArgs._RecentAssetsProvider;
	this->ThumbnailDisplayMode = InArgs._ThumbnailDisplayMode;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (FNamedCollectionList& CollectionSet : CollectionSets)
	{
		// For backwards compatibility with older versions of FNamedCollectionList that have the CollectionNames member filled out.
		if (CollectionSet.Collections.IsEmpty() && !CollectionSet.CollectionNames.IsEmpty())
		{
			CollectionSet.Collections.Reserve(CollectionSet.CollectionNames.Num());
			for (const FCollectionNameType& CollectionName : CollectionSet.CollectionNames)
			{
				CollectionSet.Collections.Emplace(
					FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(),
					CollectionName);
			}
			CollectionSet.CollectionNames.Empty();
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// validate arguments
	if ( InArgs._AssetClassType == nullptr )
	{
		UE::ModelingUI::SetCustomWidgetErrorString(LOCTEXT("MissingAssetType", "Please specify an AssetClassType."), this->ChildSlot);
		return;
	}
	this->AssetClassType = InArgs._AssetClassType;

	FText UseTooltipText = Property.IsValid() ? Property->GetToolTipText() : InArgs._ToolTipText;

	// create our own thumbnail pool?
	ThumbnailPool = MakeShared<FAssetThumbnailPool>(20, 1.0 /** must be large or thumbnails will not render?? */ );
	ensure(ThumbnailPool.IsValid());

	// make an empty asset thumbnail
	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.bAllowRealTimeOnHovered = false;
	this->AssetThumbnail = MakeShared<FAssetThumbnail>(FAssetData(), ComboButtonTileSize.X, ComboButtonTileSize.Y, ThumbnailPool);

	TSharedPtr<SWidget> AssetDisplay = nullptr;
	
	if (ThumbnailDisplayMode == EThumbnailDisplayMode::AssetThumbnail)
	{
		AssetDisplay = SNew(SBox)
		.WidthOverride(ComboButtonTileSize.X)
		.HeightOverride(ComboButtonTileSize.Y)
		[
			SAssignNew(ThumbnailBorder, SBorder)
			.Padding(0)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnMouseDoubleClick(this, &SToolInputAssetComboPanel::OnAssetThumbnailDoubleClick)
			[
				SNew(SBox)
				.WidthOverride(ComboButtonTileSize.X)
				.HeightOverride(ComboButtonTileSize.Y)
				[
					AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig)
				]
			]
		];
	}
	else if (ThumbnailDisplayMode == EThumbnailDisplayMode::AssetName)
	{
		AssetDisplay = SNew(SBox)
		[
			SAssignNew(ThumbnailBorder, SBorder)
			.Padding(0)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnMouseDoubleClick(this, &SToolInputAssetComboPanel::OnAssetThumbnailDoubleClick)
			[
				SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return FText::FromName(AssetThumbnail->GetAssetData().AssetName);
					})
			]
		];
	}

	ComboButton = SNew(SComboButton)
		.ToolTipText(UseTooltipText)
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(SAssetDropTarget)
			.OnAreAssetsAcceptableForDropWithReason_Lambda( [this](TArrayView<FAssetData> InAssets, FText& OutReason) {
				UObject* AssetObject = InAssets[0].GetAsset();
				if ( (AssetObject != nullptr) && AssetObject->IsA(this->AssetClassType))
				{
					return true;
				}
				return false;
			})
			.OnAssetsDropped_Lambda( [this](const FDragDropEvent&, TArrayView<FAssetData> InAssets)
			{
				this->NewAssetSelected(InAssets[0].GetAsset());
			})
			[
				SNew(SBorder)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.Padding(FMargin(0.0f))
					.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						//.Padding(1)
						[
							SNew(SBox)
							.WidthOverride(75.f)
							[
								SNew(SVerticalBox)
								// Thumbnail or Label
								+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									AssetDisplay.ToSharedRef()
								]
							]
						]

					]
			]
		];

	EThumbnailLabel::Type TileThumbnailLabel = InArgs._AssetThumbnailLabel;
	bool bForceShowEngineContent = InArgs._bForceShowEngineContent;
	bool bForceShowPluginContent = InArgs._bForceShowPluginContent;
	EAssetViewType::Type AssetViewType = InArgs._AssetViewType;

	ComboButton->SetOnGetMenuContent(FOnGetContent::CreateLambda([this, TileThumbnailLabel,bForceShowEngineContent, bForceShowPluginContent, AssetViewType]()
	{
		// Configure filter for asset picker
		FAssetPickerConfig Config;
		Config.SelectionMode = ESelectionMode::Single;
		Config.Filter.bRecursiveClasses = true;
		Config.Filter.ClassPaths.Add( this->AssetClassType->GetClassPathName() );
		Config.Filter.bRecursivePaths = true;
		//Config.Filter.bIncludeOnlyOnDiskAssets = true;
		Config.InitialAssetViewType = AssetViewType;
		Config.bFocusSearchBoxWhenOpened = true;
		Config.bAllowNullSelection = true;
		Config.bAllowDragging = false;
		Config.ThumbnailLabel = TileThumbnailLabel;
		Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SToolInputAssetComboPanel::NewAssetSelected);
		Config.bForceShowEngineContent = bForceShowEngineContent;
		Config.bForceShowPluginContent = bForceShowPluginContent;

		// build asset picker UI
		TSharedRef<SToolInputAssetPicker> AssetPickerWidget = SNew( SToolInputAssetPicker )
			.IsEnabled( true )
			.AssetPickerConfig( Config );

		TSharedRef<SVerticalBox> PopupContent = SNew(SVerticalBox);

		int32 FilterButtonBarVertPadding = 2;
		UpdateRecentAssets();
		if ( RecentAssetsProvider.IsValid() && RecentAssetData.Num() > 0 )
		{
			TSharedRef<SListView<TSharedPtr<FRecentAssetInfo>>> RecentsListView = SNew(SListView<TSharedPtr<FRecentAssetInfo>>)
				.Orientation(Orient_Horizontal)
				.ListItemsSource(&RecentAssetData)
				.OnGenerateRow(this, &SToolInputAssetComboPanel::OnGenerateWidgetForRecentList)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FRecentAssetInfo> SelectedItem, ESelectInfo::Type SelectInfo)
				{
					NewAssetSelected(SelectedItem->AssetData);
				})
				.ClearSelectionOnClick(false)
				.SelectionMode(ESelectionMode::Single);

			PopupContent->AddSlot()
			.AutoHeight()
			[
				SNew(SBox)
				.Padding(6.0f)
				.HeightOverride(FlyoutTileSize.Y + 30.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RecentsHeaderText", "Recently Used"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						RecentsListView->AsShared()
					]
				]
			];
		}
		else
		{
			FilterButtonBarVertPadding = 10;
		}

		if (CollectionSets.Num() > 0)
		{
			PopupContent->AddSlot()
			.AutoHeight()
			.Padding(10, FilterButtonBarVertPadding, 4, 2)
			[
				MakeCollectionSetsButtonPanel(AssetPickerWidget)
			];
		}

		PopupContent->AddSlot()
		[
			SNew(SBox)
			.Padding(6.0f)
			.HeightOverride(FlyoutSize.Y)
			.WidthOverride(FlyoutSize.X)
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush("Menu.Background") )
				[
					AssetPickerWidget
				]
			]
		];

		return PopupContent;
	}));


	// set initial thumbnail
	RefreshThumbnailFromProperty();

	if (InArgs._InitiallySelectedAsset.IsValid())
	{
		NewAssetSelected(InArgs._InitiallySelectedAsset);
	}

	ChildSlot
		[
			ComboButton->AsShared()
		];

}


void SToolInputAssetComboPanel::RefreshThumbnailFromProperty()
{
	if (Property.IsValid())
	{
		FAssetData AssetData;
		if (Property->GetValue(AssetData) == FPropertyAccess::Result::Success)
		{
			AssetThumbnail->SetAsset(AssetData);
		}
	}
}

void SToolInputAssetComboPanel::RefreshThumbnail(const FAssetData& InAssetData) const
{
	if (InAssetData.IsValid())
	{
		AssetThumbnail->SetAsset(InAssetData);
	}
}

TSharedRef<SWidget> SToolInputAssetComboPanel::MakeCollectionSetsButtonPanel(TSharedRef<SToolInputAssetPicker> AssetPickerView)
{
	ActiveCollectionSetIndex = 0;

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
		
	for ( int32 Index = 0; Index <= CollectionSets.Num(); ++Index)
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(4,0)
		[
			SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.HAlign(HAlign_Center)
				.OnCheckStateChanged_Lambda([this, Index, AssetPickerView](ECheckBoxState NewState)
				{
					if ( NewState == ECheckBoxState::Checked )
					{
						if (Index == 0)
						{
							AssetPickerView->UpdateAssetSourceCollections(TArray<FCollectionRef>());
						}
						else
						{
							AssetPickerView->UpdateAssetSourceCollections(this->CollectionSets[Index - 1].Collections);
						}
						ActiveCollectionSetIndex = Index;
					}
				})
				.IsChecked_Lambda([this, Index]() -> ECheckBoxState
				{
					return (this->ActiveCollectionSetIndex == Index) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(4, 2))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
						.Text( (Index == 0) ? LOCTEXT("AllFilterLabel", "Show All") :  FText::FromString(CollectionSets[Index-1].Name) )
					]
				]
		];
	}

	return HorizontalBox;
}


void SToolInputAssetComboPanel::UpdateRecentAssets()
{
	if (RecentAssetsProvider.IsValid() == false)
	{
		return;
	}
	TArray<FAssetData> RecentAssets = RecentAssetsProvider->GetRecentAssetsList();
	int32 NumRecent = RecentAssets.Num();

	while (RecentAssetData.Num() < NumRecent)
	{
		FAssetThumbnailConfig ThumbnailConfig;
		ThumbnailConfig.bAllowRealTimeOnHovered = false;

		TSharedPtr<FRecentAssetInfo> NewInfo = MakeShared<FRecentAssetInfo>();
		NewInfo->Index = RecentAssetData.Num();
		RecentAssetData.Add(NewInfo);

		RecentThumbnails.Add(MakeShared<FAssetThumbnail>(FAssetData(), FlyoutTileSize.X, FlyoutTileSize.Y, ThumbnailPool));
		RecentThumbnailWidgets.Add(SNew(SBox)
			.WidthOverride(FlyoutTileSize.X)
			.HeightOverride(FlyoutTileSize.Y)
			[
				RecentThumbnails[NewInfo->Index]->MakeThumbnailWidget(ThumbnailConfig)
			]);
	}

	for (int32 k = 0; k < NumRecent; ++k)
	{
		RecentAssetData[k]->AssetData = RecentAssets[k];
		RecentThumbnails[k]->SetAsset(RecentAssets[k]);
	}

}


TSharedRef<ITableRow> SToolInputAssetComboPanel::OnGenerateWidgetForRecentList(TSharedPtr<FRecentAssetInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return  SNew(STableRow<TSharedPtr<FRecentAssetInfo>>, OwnerTable)
		//.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")		// do not like this style...
		//.ToolTipText( InItem->AssetData )		// todo - cannot be assigned to STableRow, needs to go on nested widget
		.Padding(2.0f)
		[
			// todo: maybe add a border around this?
			RecentThumbnailWidgets[InItem->Index]->AsShared()
		];
}


void SToolInputAssetComboPanel::NewAssetSelected(const FAssetData& AssetData)
{
	AssetThumbnail->SetAsset(AssetData);

	if (Property.IsValid())
	{
		Property->SetValue(AssetData);
	}

	if (RecentAssetsProvider.IsValid())
	{
		RecentAssetsProvider->NotifyNewAsset(AssetData);
	}

	OnSelectionChanged.ExecuteIfBound(AssetData);

	ComboButton->SetIsOpen(false);
}


FReply SToolInputAssetComboPanel::OnAssetThumbnailDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FAssetData& CurData = AssetThumbnail->GetAssetData();
	if (CurData.IsValid())
	{
		UObject* ObjectToEdit = CurData.GetAsset();
		if( ObjectToEdit )
		{
			GEditor->EditObject( ObjectToEdit );
		}
	}

	// might be open from first click?
	ComboButton->SetIsOpen(false);

	return FReply::Handled();
}




#undef LOCTEXT_NAMESPACE