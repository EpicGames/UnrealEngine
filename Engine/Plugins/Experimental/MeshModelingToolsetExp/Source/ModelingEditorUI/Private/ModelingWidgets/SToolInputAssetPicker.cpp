// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SToolInputAssetPicker.h"
#include "Algo/Transform.h"
#include "CollectionManagerModule.h"
#include "ICollectionManager.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "FrontendFilters.h"
#include "SAssetSearchBox.h"
#include "SAssetView.h"
#include "Framework/Commands/GenericCommands.h"
#include "Editor.h"
#include "PropertyHandle.h"
#include "AssetTextFilter.h"

#define LOCTEXT_NAMESPACE "SToolInputAssetPicker"

SToolInputAssetPicker::~SToolInputAssetPicker()
{
}

void SToolInputAssetPicker::Construct( const FArguments& InArgs )
{
	OnAssetsActivated = InArgs._AssetPickerConfig.OnAssetsActivated;
	OnAssetSelected = InArgs._AssetPickerConfig.OnAssetSelected;
	OnAssetEnterPressed = InArgs._AssetPickerConfig.OnAssetEnterPressed;
	bPendingFocusNextFrame = InArgs._AssetPickerConfig.bFocusSearchBoxWhenOpened;
	DefaultFilterMenuExpansion = InArgs._AssetPickerConfig.DefaultFilterMenuExpansion;

	if ( InArgs._AssetPickerConfig.bFocusSearchBoxWhenOpened )
	{
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SToolInputAssetPicker::SetFocusPostConstruct ) );
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	ChildSlot
	[
		VerticalBox
	];

	TAttribute< FText > HighlightText;
	EThumbnailLabel::Type ThumbnailLabel = InArgs._AssetPickerConfig.ThumbnailLabel;

	FrontendFilters = MakeShareable(new FAssetFilterCollectionType());

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (!InArgs._AssetPickerConfig.bAutohideSearchBar)
	{
		// Search box
		HighlightText = TAttribute< FText >(this, &SToolInputAssetPicker::GetHighlightedText);
		HorizontalBox->AddSlot()
		.FillWidth(1.0f)
		[
			SAssignNew( SearchBoxPtr, SAssetSearchBox )
			.HintText(NSLOCTEXT( "ContentBrowser", "SearchBoxHint", "Search Assets" ))
			.OnTextChanged( this, &SToolInputAssetPicker::OnSearchBoxChanged )
			.OnTextCommitted( this, &SToolInputAssetPicker::OnSearchBoxCommitted )
			.DelayChangeNotificationsWhileTyping( true )
			.OnKeyDownHandler( this, &SToolInputAssetPicker::HandleKeyDownFromSearchBox )
		];

	}
	else
	{
		HorizontalBox->AddSlot()
		.FillWidth(1.0)
		[
			SNew(SSpacer)
		];
	}

		
	VerticalBox->AddSlot()
	.AutoHeight()
	.Padding(8.f, 0.f, 8.f, 8.f)
	[
		HorizontalBox
	];

	// "None" button
	if (InArgs._AssetPickerConfig.bAllowNullSelection)
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
						.ButtonStyle( FAppStyle::Get(), "ContentBrowser.NoneButton" )
						.TextStyle( FAppStyle::Get(), "ContentBrowser.NoneButtonText" )
						.Text( LOCTEXT("NoneButtonText", "( None )") )
						.ToolTipText( LOCTEXT("NoneButtonTooltip", "Clears the asset selection.") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SToolInputAssetPicker::OnNoneButtonClicked)
				]

			// Trailing separator
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 4)
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
				]
		];
	}

	// Asset view
	
	TArray<FCollectionRef> CollectionsFilter = InArgs._AssetPickerConfig.CollectionsFilter;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (CollectionsFilter.IsEmpty() && !InArgs._AssetPickerConfig.Collections.IsEmpty())
	{
		// Fall back to the Collections field for backwards compatibility.
		CollectionsFilter.Reserve(InArgs._AssetPickerConfig.Collections.Num());
		Algo::Transform(
			InArgs._AssetPickerConfig.Collections,
			CollectionsFilter,
			[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CurrentContentSources = FAssetViewContentSources(InArgs._AssetPickerConfig.Filter.PackagePaths, MoveTemp(CollectionsFilter));
	CurrentBackendFilter = InArgs._AssetPickerConfig.Filter;
	CurrentBackendFilter.PackagePaths.Reset();

	// Make game-specific filter
	FOnShouldFilterAsset ShouldFilterAssetDelegate;
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(InArgs._AssetPickerConfig.AdditionalReferencingAssets);
		AssetReferenceFilterContext.AddReferencingAssetsFromPropertyHandle(InArgs._AssetPickerConfig.PropertyHandle);
		
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
		if (AssetReferenceFilter.IsValid())
		{
			FOnShouldFilterAsset ConfigFilter = InArgs._AssetPickerConfig.OnShouldFilterAsset;
			ShouldFilterAssetDelegate = FOnShouldFilterAsset::CreateLambda([ConfigFilter, AssetReferenceFilter](const FAssetData& AssetData) -> bool {
				if (!AssetReferenceFilter->PassesFilter(AssetData))
				{
					return true;
				}
				if (ConfigFilter.IsBound())
				{
					return ConfigFilter.Execute(AssetData);
				}
				return false;
			});
		}
		else
		{
			ShouldFilterAssetDelegate = InArgs._AssetPickerConfig.OnShouldFilterAsset;
		}
	}

	if (!InArgs._AssetPickerConfig.bAutohideSearchBar)
	{
		TextFilter = MakeShared<FAssetTextFilter>();
	}

	VerticalBox->AddSlot()
	.FillHeight(1.f)
	[
		SAssignNew(AssetViewPtr, SAssetView)
		.InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAssets)
		.SelectionMode( InArgs._AssetPickerConfig.SelectionMode )
		.OnShouldFilterAsset(ShouldFilterAssetDelegate)
		.OnItemSelectionChanged(this, &SToolInputAssetPicker::HandleItemSelectionChanged)
		.OnItemsActivated(this, &SToolInputAssetPicker::HandleItemsActivated)
		.OnIsAssetValidForCustomToolTip(InArgs._AssetPickerConfig.OnIsAssetValidForCustomToolTip)
		.OnGetCustomAssetToolTip(InArgs._AssetPickerConfig.OnGetCustomAssetToolTip)
		.OnVisualizeAssetToolTip(InArgs._AssetPickerConfig.OnVisualizeAssetToolTip)
		.OnAssetToolTipClosing(InArgs._AssetPickerConfig.OnAssetToolTipClosing)
		.FrontendFilters(FrontendFilters)
		.TextFilter(TextFilter)
		.InitialContentSources(CurrentContentSources)
		.InitialBackendFilter(CurrentBackendFilter)
		.InitialViewType(InArgs._AssetPickerConfig.InitialAssetViewType)
		.InitialAssetSelection(InArgs._AssetPickerConfig.InitialAssetSelection)
		.ShowBottomToolbar(InArgs._AssetPickerConfig.bShowBottomToolbar)
		.OnAssetTagWantsToBeDisplayed(InArgs._AssetPickerConfig.OnAssetTagWantsToBeDisplayed)
		.OnGetCustomSourceAssets(InArgs._AssetPickerConfig.OnGetCustomSourceAssets)
		.AllowDragging( InArgs._AssetPickerConfig.bAllowDragging )
		.CanShowClasses( InArgs._AssetPickerConfig.bCanShowClasses )
		.CanShowFolders( InArgs._AssetPickerConfig.bCanShowFolders )
		.CanShowReadOnlyFolders( InArgs._AssetPickerConfig.bCanShowReadOnlyFolders )
		.ShowPathInColumnView( InArgs._AssetPickerConfig.bShowPathInColumnView)
		.ShowTypeInColumnView( InArgs._AssetPickerConfig.bShowTypeInColumnView)
		.ShowViewOptions(false)  // We control this in the asset picker
		.SortByPathInColumnView( InArgs._AssetPickerConfig.bSortByPathInColumnView)
		.FilterRecursivelyWithBackendFilter( false )
		.CanShowRealTimeThumbnails( InArgs._AssetPickerConfig.bCanShowRealTimeThumbnails )
		.CanShowDevelopersFolder( InArgs._AssetPickerConfig.bCanShowDevelopersFolder )
		.ForceShowEngineContent( InArgs._AssetPickerConfig.bForceShowEngineContent )
		.ForceShowPluginContent( InArgs._AssetPickerConfig.bForceShowPluginContent )
		.HighlightedText( HighlightText )
		.ThumbnailLabel( ThumbnailLabel )
		.AssetShowWarningText( InArgs._AssetPickerConfig.AssetShowWarningText)
		.AllowFocusOnSync(false)	// Stop the asset view from stealing focus (we're in control of that)
		.HiddenColumnNames(InArgs._AssetPickerConfig.HiddenColumnNames)
		.CustomColumns(InArgs._AssetPickerConfig.CustomColumns)
		// custom stuff, should expose as parameters
		.InitialThumbnailSize(EThumbnailSize::Small)
		.ShowTypeInTileView(false)
	];

	if (AssetViewPtr.IsValid() && TextFilter)
	{
		bool bClassNamesProvided = (InArgs._AssetPickerConfig.Filter.ClassPaths.Num() != 1);
		TextFilter->SetIncludeClassName(bClassNamesProvided || AssetViewPtr->IsIncludingClassNames());
		TextFilter->SetIncludeAssetPath(AssetViewPtr->IsIncludingAssetPaths());
		TextFilter->SetIncludeCollectionNames(AssetViewPtr->IsIncludingCollectionNames());
	}

	AssetViewPtr->RequestSlowFullListRefresh();
}

void SToolInputAssetPicker::UpdateAssetSourceCollections(TArray<FCollectionRef> Collections)
{
	CurrentContentSources = FAssetViewContentSources(MoveTemp(Collections));
	if (AssetViewPtr.IsValid())
	{
		AssetViewPtr->SetContentSources(CurrentContentSources);
	}
}

void SToolInputAssetPicker::UpdateAssetSourceCollections(TArray<FCollectionNameType> Collections)
{
	TArray<FCollectionRef> CollectionsFilter;

	CollectionsFilter.Reserve(Collections.Num());
	Algo::Transform(
		Collections,
		CollectionsFilter,
		[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });

	UpdateAssetSourceCollections(MoveTemp(CollectionsFilter));
}

EActiveTimerReturnType SToolInputAssetPicker::SetFocusPostConstruct( double InCurrentTime, float InDeltaTime )
{
	if ( SearchBoxPtr.IsValid() )
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( SearchBoxPtr.ToSharedRef(), WidgetToFocusPath );
		FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
		WidgetToFocusPath.GetWindow()->SetWidgetToFocusOnActivate(SearchBoxPtr);

		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

FReply SToolInputAssetPicker::HandleKeyDownFromSearchBox(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Up/Right and Down/Left move thru the filtered list
	if (InKeyEvent.GetKey() == EKeys::Up || InKeyEvent.GetKey() == EKeys::Left)
	{
		AssetViewPtr->AdjustActiveSelection(-1);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Down || InKeyEvent.GetKey() == EKeys::Right)
	{
		AssetViewPtr->AdjustActiveSelection(+1);
	}

	return FReply::Unhandled();
}

FReply SToolInputAssetPicker::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<FContentBrowserItem> SelectionSet = AssetViewPtr->GetSelectedFileItems();
		if (SelectionSet.Num() == 1)
		{
			FAssetData ItemAssetData;
			SelectionSet[0].Legacy_TryGetAssetData(ItemAssetData);
			OnAssetSelected.ExecuteIfBound(ItemAssetData);
		}
		else
		{
			OnNoneButtonClicked();
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FText SToolInputAssetPicker::GetHighlightedText() const
{
	return TextFilter->GetRawFilterText();
}

void SToolInputAssetPicker::SetSearchBoxText(const FText& InSearchText)
{
	// Has anything changed? (need to test case as the operators are case-sensitive)
	if (!InSearchText.ToString().Equals(TextFilter->GetRawFilterText().ToString(), ESearchCase::CaseSensitive))
	{
		TextFilter->SetRawFilterText(InSearchText);
		if (InSearchText.IsEmpty())
		{
			AssetViewPtr->SetUserSearching(false);
		}
		else
		{
			AssetViewPtr->SetUserSearching(true);
		}
	}
}

void SToolInputAssetPicker::OnSearchBoxChanged(const FText& InSearchText)
{
	SetSearchBoxText( InSearchText );
}

void SToolInputAssetPicker::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
{
	SetSearchBoxText( InSearchText );

	if (CommitInfo == ETextCommit::OnEnter)
	{
		TArray<FContentBrowserItem> SelectionSet = AssetViewPtr->GetSelectedFileItems();
		if ( SelectionSet.Num() == 0 )
		{
			AssetViewPtr->AdjustActiveSelection(1);
			SelectionSet = AssetViewPtr->GetSelectedFileItems();
		}
		HandleItemsActivated(SelectionSet, EAssetTypeActivationMethod::Opened);
	}
}



FReply SToolInputAssetPicker::OnNoneButtonClicked()
{
	OnAssetSelected.ExecuteIfBound(FAssetData());
	if (AssetViewPtr.IsValid())
	{
		AssetViewPtr->ClearSelection(true);
	}
	return FReply::Handled();
}


void SToolInputAssetPicker::HandleItemSelectionChanged(const FContentBrowserItem& InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if (InSelectInfo != ESelectInfo::Direct)
	{
		FAssetData ItemAssetData;
		InSelectedItem.Legacy_TryGetAssetData(ItemAssetData);
		OnAssetSelected.ExecuteIfBound(ItemAssetData);
		
	}
}

void SToolInputAssetPicker::HandleItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod)
{
	FContentBrowserItem FirstActivatedFolder;

	TArray<FAssetData> ActivatedAssets;
	for (const FContentBrowserItem& ActivatedItem : ActivatedItems)
	{
		if (ActivatedItem.IsFile())
		{
			FAssetData ItemAssetData;
			if (ActivatedItem.Legacy_TryGetAssetData(ItemAssetData))
			{
				ActivatedAssets.Add(MoveTemp(ItemAssetData));
			}
		}

		if (ActivatedItem.IsFolder() && !FirstActivatedFolder.IsValid())
		{
			FirstActivatedFolder = ActivatedItem;
		}
	}

	if (ActivatedAssets.Num() == 0)
	{
		return;
	}

	if (ActivationMethod == EAssetTypeActivationMethod::Opened)
	{
		OnAssetEnterPressed.ExecuteIfBound(ActivatedAssets);
	}

	OnAssetsActivated.ExecuteIfBound( ActivatedAssets, ActivationMethod );
}



void SToolInputAssetPicker::RefreshAssetView(bool bRefreshSources)
{
	if (bRefreshSources)
	{
		AssetViewPtr->RequestSlowFullListRefresh();
	}
	else
	{
		AssetViewPtr->RequestQuickFrontendListRefresh();
	}
}





#undef LOCTEXT_NAMESPACE
