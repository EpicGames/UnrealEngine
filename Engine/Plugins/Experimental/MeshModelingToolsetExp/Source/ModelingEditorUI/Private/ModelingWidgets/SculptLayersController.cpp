// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SculptLayersController.h"

#include "ScopedTransaction.h"
#include "Properties/MeshSculptLayerProperties.h"
#include "ModelingWidgets/SMeshLayersStack.h"


DEFINE_LOG_CATEGORY(LogMeshLayers);

#define LOCTEXT_NAMESPACE "SculptLayersController"

FSculptLayersController::FSculptLayersController() { }

int32 FSculptLayersController::AddMeshLayer()
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return INDEX_NONE;
	}
	FScopedTransaction Transaction(LOCTEXT("AddSculptLayer_Label", "Add Sculpt Layer"));

	// add the layer
	Properties->AddLayer();
	return Properties->LayerWeights.Num() - 1;
}

bool FSculptLayersController::RemoveMeshLayer(const int32 InLayerIndex)
{
	// ensure properties are valid
	if (!ensure(Properties))
	{
		return false;
	}
	FScopedTransaction Transaction(LOCTEXT("RemoveSculptLayer_Label", "Remove Sculpt Layer"));
	
	// remove the layer and return if the remove was successful
	return Properties->RemoveLayerAtIndex(InLayerIndex);
}

void FSculptLayersController::SetActiveLayer(const int32 InLayerIndex) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return;
	}
	
	Properties->SetActiveLayer(InLayerIndex);
}

int32 FSculptLayersController::GetActiveLayer() const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return INDEX_NONE;
	}
	return Properties->ActiveLayer - Properties->GetNumLockedBaseLayers();
}

FName FSculptLayersController::GetLayerName(const int32 InLayerIndex) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return NAME_None;
	}
	
	return Properties->GetLayerName(InLayerIndex);
}

void FSculptLayersController::SetLayerName(const int32 InLayerIndex, const FName InName) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return;
	}
	
	return Properties->SetLayerName(InLayerIndex, InName);
}


double FSculptLayersController::GetLayerWeight(const int32 InLayerIndex) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return 0.0;
	}
	
	// ensure the provided index is valid
	if (!Properties->LayerWeights.IsValidIndex(InLayerIndex))
	{
		UE_LOGF(LogMeshLayers, Warning, "Sculpt Layer not found. Invalid index: %d.", InLayerIndex);
		return INDEX_NONE;
	}
	
	return Properties->LayerWeights[InLayerIndex];
}

void FSculptLayersController::SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("SetLayerWeight_Label", "Set Layer Weight"));

	Properties->SetLayerWeight(InLayerIndex, InWeight, ChangeType);
}

int32 FSculptLayersController::GetNumMeshLayers() const
{
	if (!ensure(Properties))
	{
		return INDEX_NONE;
	}
	return Properties->LayerWeights.Num();
}

bool FSculptLayersController::MoveLayerInStack(int32 InLayerToMoveIndex, int32 InTargetIndex)
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return false;
	}
	FScopedTransaction Transaction(LOCTEXT("ReorderSculptLayers_Label", "Reorder Sculpt Layers"));

	Properties->MoveLayer(InLayerToMoveIndex, InTargetIndex);

	return true;
}

void FSculptLayersController::RefreshLayersStackView() const
{
	if (LayersStackView.IsValid())
	{
		LayersStackView->RefreshStackView();
	}
}

#undef LOCTEXT_NAMESPACE
