// Copyright Epic Games, Inc. All Rights Reserved.

#include "STransformGizmoNumericalUIOverlay.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h" // UCombinedTransformGizmoContextObject
#include "ContextObjectStore.h"
#include "Framework/Commands/UIAction.h" // FUIAction
#include "Framework/Commands/UICommandInfo.h" // EUserInterfaceActionType
#include "Framework/MultiBox/MultiBoxBuilder.h" // FMenuBuilder
#include "InteractiveToolsContext.h"
#include "Misc/AxisDisplayInfo.h"
#include "ModelingWidgets/STickDelegateWidget.h"
#include "Transforms/TransformGizmoDataBinder.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h" // SHorizontalBox
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "STransformGizmoNumericalUIOverlay"

namespace STransformGizmoNumericalUIOverlayLocals
{
	// Used for placing the icon in the translate/rotate/scale rows
	float ICON_RIGHT_PADDING = 8;
}

void STransformGizmoNumericalUIOverlay::Construct(const FArguments& InArgs)
{
	bPositionRelativeToBottom = InArgs._bPositionRelativeToBottom;
	DefaultLeftPadding = InArgs._DefaultLeftPadding;
	DefaultVerticalPadding = InArgs._DefaultVerticalPadding;
	TranslationScrubSensitivity = InArgs._TranslationScrubSensitivity;

	// Set up the binding between our displayed values and the gizmo
	DataBinder = MakeShared<FTransformGizmoDataBinder>();
	DataBinder->InitializeBoundVectors(&DisplayTranslation, &DisplayEulerAngles, &DisplayScale);
	DataBinder->SetDefaultLocalReferenceTransform(InArgs._DefaultLocalReferenceTransform);
	DataBinder->SetTranslationConversionFunctions(InArgs._InternalToDisplayFunction, InArgs._DisplayToInternalFunction);
	
	// Rebuild our display whenever we start tracking a new gizmo
	DataBinder->OnTrackedGizmoChanged.AddLambda([this](UCombinedTransformGizmo* Gizmo) {
		if (Gizmo)
		{
			WidgetContents->SetContent(CreateWidgetForGizmo(Gizmo->GetGizmoElements()));
		}
	});

	ChildSlot
	[
		SAssignNew(DraggableBoxOverlay, UE::ToolWidgets::SDraggableBoxOverlay)
		.InitialAlignmentOffset({ DefaultLeftPadding, DefaultVerticalPadding })
		.VAlign(bPositionRelativeToBottom ? VAlign_Bottom : VAlign_Top)
		.Visibility_Lambda([this]()
		{
			return bIsEnabled && DataBinder->HasVisibleGizmo() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
		})
		[
			SNew(STickDelegateWidget)
			.OnTickDelegate(STickDelegateWidget::FOnTick::CreateSP(this, &STransformGizmoNumericalUIOverlay::OnTickWidget))
			[
				SAssignNew(WidgetContents, SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
				.Padding(8.f)
			]
		]
	]
	;

	// Don't block things under the overlay.
	SetVisibility(EVisibility::SelfHitTestInvisible);
}

STransformGizmoNumericalUIOverlay::~STransformGizmoNumericalUIOverlay()
{
	Reset();
	DataBinder = nullptr;
}

void STransformGizmoNumericalUIOverlay::Reset()
{
	if (DataBinder.IsValid())
	{
		DataBinder->Reset();
	}
}

bool STransformGizmoNumericalUIOverlay::BindToGizmoContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ToolsContext && ToolsContext->ContextObjectStore)
	{
		UCombinedTransformGizmoContextObject* ContextObject = ToolsContext->ContextObjectStore->FindContext<UCombinedTransformGizmoContextObject>();
		if (ContextObject)
		{
			DataBinder->BindToGizmoContextObject(ContextObject);
			return true;
		}
	}
	return ensure(false);
}

void STransformGizmoNumericalUIOverlay::OnTickWidget(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// We update the coordinate system each tick since context queries api doesn't tell us when the value changes.
	// In the editor, we could actually bind to a notification callback instead of ticking here.
	DataBinder->UpdateCoordinateSystem();
}

// Helper to reuse code when creating translation/rotation/scale UI pieces. We originally built these individually
// so that we could individually pick and choose components based on active gizmo components. However, we have since
// decided to disable unused components rather than removing them entirely. Otherwise, maybe we could have used
// SVectorInputBox. There didn't seem to be enough generality here to justify making a utility function, but perhaps
// we will change our mind about that.
SNumericEntryBox<double>::FArguments STransformGizmoNumericalUIOverlay::CreateComponentBaseArgs(FVector3d& TargetVector, int ComponentIndex, TAttribute<bool> IsEnabled)
{
	// Can be indexed via ComponentIndex:
	const EAxisList::Type Axes[3] {
		EAxisList::X,
		EAxisList::Y,
		EAxisList::Z
	};

	SNumericEntryBox<double>::FArguments Args;

	TWeakPtr<FTransformGizmoDataBinder> DataBinderWeakPtr = DataBinder;

	Args.AllowSpin(true)
		.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.Value(MakeAttributeLambda([&TargetVector, ComponentIndex]()->TOptional<double> { return TargetVector[ComponentIndex]; }))
		.OnValueChanged(SNumericEntryBox<double>::FOnValueChanged::CreateLambda([DataBinderWeakPtr, &TargetVector, ComponentIndex](double Value) {
			TargetVector[ComponentIndex] = Value;
			if (DataBinderWeakPtr.IsValid())
			{
				DataBinderWeakPtr.Pin()->UpdateAfterDataEdit();
			}
		}))
		// This callback doesn't seem to be that useful because we get OnValueChanged callbacks regardless of the input method.
		//.OnValueCommitted(SNumericEntryBox<double>::FOnValueCommitted::CreateLambda(this, [](double Value, ETextCommit::Type TextCommitType) {}))
		
		.OnBeginSliderMovement(FSimpleDelegate::CreateLambda([DataBinderWeakPtr]() {
			if (DataBinderWeakPtr.IsValid())
			{
				DataBinderWeakPtr.Pin()->BeginDataEditSequence();
			}
			}))
		.OnEndSliderMovement(SNumericEntryBox<double>::FOnValueChanged::CreateLambda([DataBinderWeakPtr](double Value) {
			if (DataBinderWeakPtr.IsValid())
			{
				DataBinderWeakPtr.Pin()->EndDataEditSequence();
			}
		}))
		.ToolTipText(MakeAttributeLambda([this, &TargetVector, Axis = Axes[ComponentIndex], ComponentIndex]
		{
			return FText::Format(LOCTEXT("Component_ToolTip", "{0}: {1}"), AxisDisplayInfo::GetAxisDisplayName(Axis), TargetVector[ComponentIndex]);
		}))
		.IsEnabled(IsEnabled)

		.LabelPadding(FMargin(3.f))
		.LabelLocation(SNumericEntryBox<double>::ELabelLocation::Inside)

		.Label()
		[
			SNumericEntryBox<double>::BuildNarrowColorLabel(AxisDisplayInfo::GetAxisColor(Axes[ComponentIndex]))
		]
		;

	return Args;
}

TSharedRef<SWidget> STransformGizmoNumericalUIOverlay::CreateTranslationComponent(int ComponentIndex, TAttribute<bool> IsEnabled)
{
	TSharedRef<SWidget> WidgetToReturn = SArgumentNew(CreateComponentBaseArgs(DisplayTranslation, ComponentIndex, IsEnabled), SNumericEntryBox<double>)

		// All four of these have to be explicitly unset for the spin range to be unlimited.
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.MinSliderValue(TOptional<double>())
		.MaxSliderValue(TOptional<double>())

		// Controls the linear scrolling behavior
		.LinearDeltaSensitivity(1)
		.Delta(MakeAttributeLambda([this]() { return TranslationScrubSensitivity; }))
		;

	return WidgetToReturn;
}

TSharedRef<SWidget> STransformGizmoNumericalUIOverlay::CreateRotationComponent(int ComponentIndex, TAttribute<bool> IsEnabled)
{
	TSharedRef<SWidget> WidgetToReturn = SArgumentNew(CreateComponentBaseArgs(DisplayEulerAngles, ComponentIndex, IsEnabled), SNumericEntryBox<double>)
		.MinSliderValue(-360)
		.MaxSliderValue(360)
		;

	return WidgetToReturn;
}

TSharedRef<SWidget> STransformGizmoNumericalUIOverlay::CreateScaleComponent(int ComponentIndex, TAttribute<bool> IsEnabled)
{
	TSharedRef<SWidget> WidgetToReturn = SArgumentNew(CreateComponentBaseArgs(DisplayScale, ComponentIndex, IsEnabled), SNumericEntryBox<double>)

		// All four of these have to be explicitly unset for the spin range to be unlimited.
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.MinSliderValue(TOptional<double>())
		.MaxSliderValue(TOptional<double>())

		// Controls the linear scrolling behavior
		.LinearDeltaSensitivity(1)
		.Delta(0.01)
		;

	return WidgetToReturn;
}

TSharedRef<SWidget> STransformGizmoNumericalUIOverlay::CreateWidgetForGizmo(ETransformGizmoSubElements GizmoElements)
{
	using namespace STransformGizmoNumericalUIOverlayLocals;

	// We'll add the translation/rotation/scale rows to this
	TSharedRef<SVerticalBox> Rows = SNew(SVerticalBox);

	TSharedRef<SBox> WidgetToReturn = SNew(SBox)
		.MinDesiredWidth(100.0f * 3.0f)
		.MaxDesiredWidth(100.0f * 3.0f)
		[
			Rows
		];

	// Create the delta mode toggle row.
	Rows->AddSlot()
		.Padding(FMargin(0.f, 0.f, 0.f, 6.f))
	[
		SNew(SHorizontalBox)

		// Label
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DeltaModeLabel", "Delta Mode:"))
			.ToolTipText(LOCTEXT("DeltaModeTooltip", "In Delta mode, the gizmo values are interpreted as changes relative to last gizmo position."))
		]

		// Actual on/off toggle
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
		[
			SNew(SCheckBox)
			//.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.Padding(FMargin(5, 2))
			.HAlign(HAlign_Center)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				if (NewState == ECheckBoxState::Checked)
				{
					DataBinder->ResetToDeltaMode();
				}
				else
				{
					DataBinder->SetToDestinationMode();
				}
			})
			.IsChecked_Lambda([this]() -> ECheckBoxState { return DataBinder->IsInDeltaMode() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0, 0.f, 2.0, 0.f))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("DeltaModeOnOff", "On/Off"))
					.ToolTipText(MakeAttributeLambda([this]() -> FText
					{
						if (!DataBinder->ShouldDestinationModeBeAllowed())
						{
							return LOCTEXT("GizmoOnlySupportsDeltaMode", "Current gizmo only supports delta mode.");
						}
						return LOCTEXT("DeltaModeToggleTooltip", "Toggle delta mode on or off");
					}))
				]
			]
			.IsEnabled_Lambda([this]() { 
				return DataBinder->ShouldDestinationModeBeAllowed();
			})
		]

		// Reset button for delta mode
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
		[
			SNew(SButton)
			.Text(LOCTEXT("ResetToDeltaModeButton", "Reset"))
			.ToolTipText(LOCTEXT("ResetToDeltaModeTooltip", "Reset the location relative to which delta values are measured to be current gizmo position."))
			.HAlign(HAlign_Center)
			.OnClicked_Lambda([this]() { 
				DataBinder->ResetToDeltaMode();
				return FReply::Handled(); 
			})
			.IsEnabled_Lambda([this]() { return DataBinder->IsInDeltaMode(); })
		]
	];

	// Create the translation row
	if ((GizmoElements & ETransformGizmoSubElements::TranslateAllAxes) != ETransformGizmoSubElements::None
		|| (GizmoElements & ETransformGizmoSubElements::TranslateAllPlanes) != ETransformGizmoSubElements::None
		|| (GizmoElements & ETransformGizmoSubElements::FreeTranslate) != ETransformGizmoSubElements::None)
	{
		TSharedRef<SHorizontalBox> Row = SNew(SHorizontalBox);
		Rows->AddSlot()
		[
			Row
		];

		// Translation icon
		Row->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, ICON_RIGHT_PADDING, 0.f))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("EditorViewport.TranslateMode"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

		bool bHaveXComponent = (GizmoElements & ETransformGizmoSubElements::TranslateAxisX) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneXY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneXZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeTranslate) != ETransformGizmoSubElements::None;
		bool bHaveYComponent = (GizmoElements & ETransformGizmoSubElements::TranslateAxisY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneXY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneYZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeTranslate) != ETransformGizmoSubElements::None;
		bool bHaveZComponent = (GizmoElements & ETransformGizmoSubElements::TranslateAxisZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneXZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::TranslatePlaneYZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeTranslate) != ETransformGizmoSubElements::None;

		Row->AddSlot()
		[
			CreateTranslationComponent(0, bHaveXComponent)
		];
		Row->AddSlot()
		[
			CreateTranslationComponent(1, bHaveYComponent)
		];
		Row->AddSlot()
		[
			CreateTranslationComponent(2, bHaveZComponent)
		];
	}

	// Add rotate row
	if ((GizmoElements & ETransformGizmoSubElements::RotateAllAxes) != ETransformGizmoSubElements::None
		|| (GizmoElements & ETransformGizmoSubElements::FreeRotate) != ETransformGizmoSubElements::None)
	{
		TSharedRef<SHorizontalBox> Row = SNew(SHorizontalBox);
		Rows->AddSlot()
		[
			Row
		];

		// Rotate icon
		Row->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, ICON_RIGHT_PADDING, 0.f))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("EditorViewport.RotateMode"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

		bool bHaveXComponent = (GizmoElements & ETransformGizmoSubElements::RotateAxisX) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeRotate) != ETransformGizmoSubElements::None;
		bool bHaveYComponent = (GizmoElements & ETransformGizmoSubElements::RotateAxisY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeRotate) != ETransformGizmoSubElements::None;
		bool bHaveZComponent = (GizmoElements & ETransformGizmoSubElements::RotateAxisZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::FreeRotate) != ETransformGizmoSubElements::None;

		Row->AddSlot()
		[
			CreateRotationComponent(0, bHaveXComponent)
		];
		Row->AddSlot()
		[
			CreateRotationComponent(1, bHaveYComponent)
		];
		Row->AddSlot()
		[
			CreateRotationComponent(2, bHaveZComponent)
		];
	}

	// Scale row
	if ((GizmoElements & ETransformGizmoSubElements::ScaleAllAxes) != ETransformGizmoSubElements::None
		|| (GizmoElements & ETransformGizmoSubElements::ScaleAllPlanes) != ETransformGizmoSubElements::None
		|| (GizmoElements & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None)
	{
		TSharedRef<SHorizontalBox> Row = SNew(SHorizontalBox);
		Rows->AddSlot()
		[
			Row
		];

		// Icon
		Row->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, ICON_RIGHT_PADDING, 0.f))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("EditorViewport.ScaleMode"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

		bool bHaveXComponent = (GizmoElements & ETransformGizmoSubElements::ScaleAxisX) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneXY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneXZ) != ETransformGizmoSubElements::None
			// Alternatively, create this component if the only scaling we have is uniform
			|| ((GizmoElements & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None
				&& (GizmoElements & ETransformGizmoSubElements::ScaleAllAxes) == ETransformGizmoSubElements::None
				&& (GizmoElements & ETransformGizmoSubElements::ScaleAllPlanes) == ETransformGizmoSubElements::None);
		bool bHaveYComponent = (GizmoElements & ETransformGizmoSubElements::ScaleAxisY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneXY) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneYZ) != ETransformGizmoSubElements::None;
		bool bHaveZComponent = (GizmoElements & ETransformGizmoSubElements::ScaleAxisZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneXZ) != ETransformGizmoSubElements::None
			|| (GizmoElements & ETransformGizmoSubElements::ScalePlaneYZ) != ETransformGizmoSubElements::None;

		Row->AddSlot()
		[
			CreateScaleComponent(0, bHaveXComponent)
		];
		Row->AddSlot()
		[
			CreateScaleComponent(1, bHaveYComponent)
		];
		Row->AddSlot()
		[
			CreateScaleComponent(2, bHaveZComponent)
		];
	}

	return WidgetToReturn;
}

void STransformGizmoNumericalUIOverlay::SetEnabled(bool bEnabledIn)
{
	bIsEnabled = bEnabledIn;

	// Note that currently, bIsEnabled just acts as a visibility toggle, meaning that the numerical UI
	// system will still receive updates when disabled. This makes it easy for the system to show the
	// correct values if it is re-enabled after being disabled.
	// If we decide that we don't want this to be the case, we'd need to add disabling support to 
	// FTransformGizmoDataBinder, which shouldn't be too hard.
}

void STransformGizmoNumericalUIOverlay::ResetPositionInViewport()
{
	if (DraggableBoxOverlay.IsValid())
	{
		DraggableBoxOverlay->SetBoxHorizontalAlignment(HAlign_Left);
		DraggableBoxOverlay->SetBoxVerticalAlignment(bPositionRelativeToBottom ? VAlign_Bottom : VAlign_Top);
		DraggableBoxOverlay->SetBoxAlignmentOffset({ DefaultLeftPadding, DefaultVerticalPadding });
	}
}

void STransformGizmoNumericalUIOverlay::MakeNumericalUISubMenu(FMenuBuilder& MenuBuilder)
{
	TWeakPtr<STransformGizmoNumericalUIOverlay> NumericalUI = StaticCastSharedRef<STransformGizmoNumericalUIOverlay>(AsShared());

	const FUIAction NumericalUIAction(
		FExecuteAction::CreateLambda([NumericalUI]
		{
			if (NumericalUI.IsValid())
			{
				TSharedPtr<STransformGizmoNumericalUIOverlay> NumericalUIPinned = NumericalUI.Pin();
				NumericalUIPinned->SetEnabled(!NumericalUIPinned->IsEnabled());
			}
		}),
		FCanExecuteAction::CreateLambda([NumericalUI]()
		{
			return NumericalUI.IsValid();
		}),
		FIsActionChecked::CreateLambda([NumericalUI]()
		{
			if (!NumericalUI.IsValid())
			{
				return false;
			}
			return NumericalUI.Pin()->IsEnabled();
		}));
	MenuBuilder.AddMenuEntry(LOCTEXT("TransformPanel_Enabled", "Enabled"), 
		LOCTEXT("TransformPanel_Enabled_Tooltip", "Show the Gizmo Transform Panel when a gizmo is visible."),
		FSlateIcon(), NumericalUIAction, NAME_None, EUserInterfaceActionType::ToggleButton);

	const FUIAction GizmoMode_ResetNumericalUIPosition(
		FExecuteAction::CreateLambda([NumericalUI]
		{
			if (NumericalUI.IsValid())
			{
				NumericalUI.Pin()->ResetPositionInViewport();
			}
		}),
		FCanExecuteAction::CreateLambda([NumericalUI]()
		{
			return NumericalUI.IsValid() && NumericalUI.Pin()->IsEnabled();
		}));
	MenuBuilder.AddMenuEntry(LOCTEXT("TransformPanel_ResetPosition", "Reset Panel Position"),
		LOCTEXT("TransformPanel_ResetPosition_Tooltip", "Reset the Gizmo Transform Panel position in the viewport."),
		FSlateIcon(), GizmoMode_ResetNumericalUIPosition, NAME_None, EUserInterfaceActionType::Button);
}

void STransformGizmoNumericalUIOverlay::SetCustomDisplayConversionFunctions(
	TFunction<FVector3d(const FVector3d& InternalValue)> InternalToDisplayConversionIn, 
	TFunction<FVector3d(const FVector3d& DisplayValue)> DisplayToInternalConversionIn, 
	double TranslationScrubSensitivityIn)
{
	if (ensure(DataBinder.IsValid()))
	{
		DataBinder->SetTranslationConversionFunctions(InternalToDisplayConversionIn, DisplayToInternalConversionIn);

		TranslationScrubSensitivity = TranslationScrubSensitivityIn;
	}
}

void STransformGizmoNumericalUIOverlay::SetDefaultLocalReferenceTransform(const TOptional<FTransform>& CustomReferenceTransform)
{
	if (ensure(DataBinder.IsValid()))
	{
		DataBinder->SetDefaultLocalReferenceTransform(CustomReferenceTransform);
	}
}

#undef LOCTEXT_NAMESPACE