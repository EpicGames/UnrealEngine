// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/UnrealType.h"

class UMeshSculptLayerProperties;
class SMeshLayersStack;

namespace UE::MeshPartition
{
	class UProjectMeshLayersModifier;
}

class IMeshLayersController
{
public:
	virtual ~IMeshLayersController() = default;
	
	/** Adds a new mesh layer to the bottom of the stack
	 * @return the stack index of the new layer. */
	virtual int32 AddMeshLayer() { return INDEX_NONE; };
	
	/** Remove the mesh layer at the given stack index.
	 * @param InLayerIndex the index of the layer to remove
	 * @return true if layer was found and removed, false otherwise */
	virtual bool RemoveMeshLayer(const int32 InLayerIndex) { return false; };

	/** Sets the layer with the given index as the Active one
	 * @param InLayerIndex the index of the layer to set as active */
	virtual void SetActiveLayer(const int32 InLayerIndex) const { };

	/** Gets the layer that is currently active
	 * @return index of the currently active layer */
	virtual int32 GetActiveLayer() const { return INDEX_NONE; };

	/** Set the name of the layer at the given index in the stack.
	 * @param InName the new name to use
	 * @param InLayerIndex the index of the layer to be renamed */
	virtual void SetLayerName(const int32 InLayerIndex, const FName InName) const { };

	/** Get the name of the layer at the given index in the stack.
	 * @param InLayerIndex the index of the layer to get the name of
	 * @return the name of the layer or None if the index was invalid */
	virtual FName GetLayerName(const int32 InLayerIndex) const { return FName(); };

	/** Sets the layer weight at the given index in the stack.
	 * @param InLayerIndex the index of the layer to set the weight of
	 * @param InWeight the new weight to set the layer to
	 * @param ChangeType the type of property change we want when setting the layer weight. */
	virtual void SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType) const { };
	
	/** Gets the layer weight at the given index in the stack.
	 * @param InLayerIndex the index of the layer to get the weight of
	 * @return the weight of the layer or none if the index was invalid */
	virtual double GetLayerWeight(const int32 InLayerIndex) const { return 0.0; };

	/** Get the number of Layers in the stack.
	 * @return int, the number of layers */
	virtual int32 GetNumMeshLayers() const = 0;
	
	/** Move the mesh layer at the given index to the target index.
	 * @param InLayerToMoveIndex the index of the layer to be moved
	 * @param InTargetIndex the index where the layer should be moved to */
	virtual bool MoveLayerInStack(int32 InLayerToMoveIndex, int32 InTargetIndex)  { return false; };
	
	/** Toggle a mesh layer on/off (visible/invisible)
	 * @param InLayerIndex the index of the layer to toggle the visibility of
	 * @param bIsEnabled if true, turns the layer on/visible, else off/invisible
	 * @return true if layer was found at index */
	//bool SetMeshLayerEnabled(int32 InLayerIndex, bool bIsEnabled) const};

	/** Get the enabled/visibility status of the given layer
	 * @param InLayerIndex the index of the layer to get the enabled/visibility state for
	 * @return true of the layer is enabled/visible, false if disabled/invisible or not found */
	//bool GetMeshLayerEnabled(int32 InLayerIndex) const { return false; };

	/** Refresh the view of the layer stack */
	virtual void RefreshLayersStackView() const { };

	/** Connect the properties to be displayed and modified */
	virtual void SetProperties(UMeshSculptLayerProperties* InProperties) { }
	/** Connect the modifier to be displayed and modified */
	virtual void SetProperties(UE::MeshPartition::UProjectMeshLayersModifier* InProperties) { }

	/** Retrieve the Stack View */
	virtual TSharedPtr<SMeshLayersStack> GetLayerStackView() { return LayersStackView; };
	/** Set the stack view */
	virtual void SetLayerStackView(const TSharedPtr<SMeshLayersStack>& InLayersStackView) { LayersStackView = InLayersStackView; };

protected:

	TSharedPtr<SMeshLayersStack> LayersStackView;
};

DECLARE_LOG_CATEGORY_EXTERN(LogMeshLayers, Warning, All);
