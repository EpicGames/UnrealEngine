// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "ModelingWidgets/SDynamicNumericEntry.h"

class SWidget;
class SBox;
class SCheckBox;

namespace UE
{
namespace ModelingUI
{


/**
 * Constants used for various standard Modeling UI gizmos/etc
 */
struct ModelingUIConstants
{
	// top/bottom padding for detail row widgets, to be consistent w/ standard details panel
	static constexpr int DetailRowVertPadding = 4;

	// minimum padding between label and widget, for tight packing scenarios
	static constexpr int LabelWidgetMinPadding = 4;

	// padding between multiple widgets in a horizontal row (eg for two widgets, this is the padding between them) (("widget" here is often label+widget))
	static constexpr int MultiWidgetRowHorzPadding = 10;
};





/**
 * Wrap a widget in a fixed-width (ie non-resizable) box 
 */
MODELINGEDITORUI_API TSharedRef<SBox> WrapInFixedWidthBox(TSharedRef<SWidget> SubWidget, int32 Width);


/**
* Make an HBox for a label/slider pair, with the label in a fixed-width box (ie no resizing)
* The Label is taken from the existing label of a UProperty Handle
*/
MODELINGEDITORUI_API TSharedRef<SHorizontalBox> MakeFixedWidthLabelSliderHBox(
	TSharedPtr<IPropertyHandle> LabelHandle,
	TSharedPtr<SDynamicNumericEntry::FDataSource> SliderDataSource,
	int32 LabelFixedWidth);

/**
* Make an HBox for a label/slider pair, with the label in a fixed-width box (ie no resizing)
* The Label is taken from the existing label of a UProperty Handle
*/
MODELINGEDITORUI_API TSharedRef<SHorizontalBox> MakeToggleSliderHBox(
	TSharedPtr<IPropertyHandle> BoolToggleHandle,
	FText ToggleLabelText,
	TSharedPtr<SDynamicNumericEntry::FDataSource> SliderDataSource,
	int32 ToggleFixedWidth);

 
/**
* Make an HBox for two widgets in a details row, with standard padding between them
*/
MODELINGEDITORUI_API TSharedRef<SHorizontalBox> MakeTwoWidgetDetailRowHBox(
	TSharedRef<SWidget> Widget1,
	TSharedRef<SWidget> Widget2,
	float FillWidth1 = 1.0f, float FillWidth2 = 1.0f);


/**
 * Make a Button that can be clicked to toggle the value of a bool UProperty.
 * When true, the button is highlighted
 */
MODELINGEDITORUI_API TSharedRef<SCheckBox> MakeBoolToggleButton(
	TSharedPtr<IPropertyHandle> BoolToggleHandle,
	FText ButtonLabelText,
	TFunction<void(bool)> ToggledCallback = [](bool) {},
	int HorzPadding = 4);


/**
 * Run ProcessFunc over each child widget of WidgetType under RootWidget.
 * @param RootWidget the root widget to search from.
 * @param WidgetType the type of widgets to consider
 * @param ProcessFunc the function to run on each instance of the widget. If return value is false, stop further processing.
 */
MODELINGEDITORUI_API void ProcessChildWidgetsByType(
	const TSharedRef<SWidget>& RootWidget,
	const FString& WidgetType,
	TFunction<bool(TSharedRef<SWidget>&)> ProcessFunc);


/**
 * Create an error-string widget and add it to the Slot argument, to give the (developer) feedback about slate programming errors
 * @param ErrorString error message
 * @param Slot Object that supports bracket[SWidget] operator, such as any Box Slot object (eg can be called w/ SCompoundWidget::ChildSlot as the Slot argument)
 */
template<typename SlotType>
void SetCustomWidgetErrorString(
	FText ErrorString,
	SlotType& Slot)
{
	Slot
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(ErrorString)
		];
}



}  // end namepace ModelingUI
}  // end namepace UE
