// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STileView.h"
#include "Styling/CoreStyle.h"

#define UE_API MODELINGEDITORUI_API

class SComboButton;

/**
* SComboPanel provides a ComboBox-type interface but instead of a drop-down
* list, a STileView panel ("Flyout") is shown with an icon for each selectable item.
* Various options for the ComboButton are available, eg a single large icon, icon with adjacent label, etc.
* 
* Client provides a TArray<TSharedPtr<FComboPanelItem>> via the .ListItems() argument, 
* where SComboPanel::FComboPanelItem contains the label, icon, and integer identifier for each selectable option.
* 
* 
*/
class SComboPanel : public SCompoundWidget
{
public:
	/** Item in the Combo Flyout (ie selectable option). */
	struct FComboPanelItem
	{
		/** Text label shown as overlay on the icon */
		FText Name;
		/** Icon */
		const FSlateBrush* Icon = nullptr;
		/** Integer identifier. Must be unique for each Item. */
		int Identifier = 0;
	};

	/** EComboDisplayType is used to specify the widgets in the ComboButton (ie non-expanded widget) */
	enum class EComboDisplayType
	{
		/** Display a single large icon with overlay label, the same as is shown in the flyout */
		LargeIcon,
		IconAndLabel
	};

	/** Event that fires when selection changes */
	DECLARE_DELEGATE_OneParam(FOnActiveItemChanged, TSharedPtr<FComboPanelItem>);
	typedef FOnActiveItemChanged FOnSelectionChanged;

public:

	SLATE_BEGIN_ARGS( SComboPanel )
		: _ComboButtonTileSize(50, 50)
		, _FlyoutTileSize(85, 85)
		, _FlyoutSize(600, 400)
		, _OnSelectionChanged()
		, _ComboDisplayType(EComboDisplayType::LargeIcon)
		, _InitialSelectionIndex(0)
		{
		}

		/** The size of the combo button icon tile */
		SLATE_ARGUMENT(FVector2D, ComboButtonTileSize)

		/** The size of the icon tiles in the flyout */
		SLATE_ARGUMENT(FVector2D, FlyoutTileSize)

		/** Size of the Flyout Panel */
		SLATE_ARGUMENT(FVector2D, FlyoutSize)

		/** List of selectable options */
		SLATE_ARGUMENT( TArray<TSharedPtr<FComboPanelItem>> , ListItems )

		/** Callback called when selection changes */
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )

		/** The text to display above the button list */
		SLATE_ATTRIBUTE( FText, FlyoutHeaderText )

		/** Type of widget layout for the ComboButton (default to LargeIcon) */
		SLATE_ARGUMENT(EComboDisplayType, ComboDisplayType)
		
		/** Index into ListItems of the initially-selected item, defaults to index 0 */
		SLATE_ARGUMENT(int, InitialSelectionIndex)

	SLATE_END_ARGS()


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	/**
	* Sets the currently selected index without user interaction.
	* 
	* @param InSelectionIndex The new selection index for the widget
	*/
	UE_API void SetSelectionIndex(int InSelectionIndex);

protected:
	FVector2D ComboButtonTileSize;
	FVector2D FlyoutTileSize;
	FVector2D FlyoutSize;

	/** Pointer to the array of data items that we are observing */
	TArray<TSharedPtr<FComboPanelItem>> Items;

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnSelectionChanged;

	// STileView functions
	UE_API virtual TSharedRef<ITableRow> CreateFlyoutIconTile(
		TSharedPtr<FComboPanelItem> Item,
		const TSharedRef<STableViewBase>& OwnerTable);
	UE_API virtual void FlyoutSelectionChanged(
		TSharedPtr<FComboPanelItem> SelectedItemIn,
		ESelectInfo::Type SelectInfo);


	// SComboButton functions
	UE_API TSharedRef<SWidget> OnGetMenuContent();
	UE_API void OnMenuOpenChanged(bool bOpen);

	/** Icon to use when Item does not have a valid icon */
	TSharedPtr<FSlateBrush> MissingIcon;

	TSharedPtr<FComboPanelItem> SelectedItem;

	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<STileView<TSharedPtr<FComboPanelItem>>> TileView;
	TSharedPtr<SBox> TileViewContainer;
};

#undef UE_API
