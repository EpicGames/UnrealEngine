// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Math/Interval.h"

#define UE_API MODELINGEDITORUI_API


/**
 * SNumericEntryBox wrapper that supports custom Get/SetValue and dynamic value/UI range
 */
class SDynamicNumericEntry : public SCompoundWidget
{
public:

	/**
	 * 
	 */
	struct FDataSource
	{
		TFunction<void(float, EPropertyValueSetFlags::Type)> SetValue;
		TFunction<float()> GetValue;
		TFunction<TInterval<float>()> GetValueRange;
		TFunction<TInterval<float>()> GetUIRange;
	};

public:

	SLATE_BEGIN_ARGS( SDynamicNumericEntry )
		: _MaxNumFloatDigits(2)
	{
	}

	SLATE_ARGUMENT( TSharedPtr<FDataSource>, Source )

	/** Maximum number of digits of precision that will be displayed for float slider */
	SLATE_ARGUMENT( int, MaxNumFloatDigits )

	SLATE_END_ARGS()


	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	UE_API void Construct( const FArguments& InArgs );

	static UE_API TSharedPtr<FDataSource> MakeSimpleDataSource(TSharedPtr<IPropertyHandle> PropertyHandle, TInterval<float> ValueRange, TInterval<float> UIRange);

protected:
	TSharedPtr<FDataSource> Source;
	TSharedPtr<SNumericEntryBox<float>> NumericEntry;
	TSharedPtr<TDefaultNumericTypeInterface<float>> TypeInterface;

	float LastSliderCommittedValue;
	UE_API void OnValueChanged(float NewValue);
	UE_API void OnValueCommitted(float NewValue, ETextCommit::Type CommitInfo);

	bool bIsUsingSlider = false;
	UE_API void OnBeginSliderMovement();
	UE_API void OnEndSliderMovement(float NewValue);
};

#undef UE_API
