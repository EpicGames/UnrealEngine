// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"

class SMeshLayersList;
class IMeshLayersController;

/**
 * top level view of a stack of mesh layers
 * shows the main menu for adding layers to the stack, and a list view of layers
*/
class SMeshLayersStack : public SCompoundWidget, public FEditorUndoClient
{
public:
	
	SLATE_BEGIN_ARGS(SMeshLayersStack)
		: _InAllowAddRemove(true)
		, _InAllowReordering(true){}
	SLATE_ARGUMENT(TWeakPtr<IMeshLayersController>, InController)
	SLATE_ARGUMENT(bool, InAllowAddRemove)
	SLATE_ARGUMENT(bool, InAllowReordering)
	SLATE_END_ARGS()

	MODELINGEDITORUI_API void Construct(const FArguments& InArgs, const TWeakPtr<IMeshLayersController>& InController);
	
	MODELINGEDITORUI_API void RefreshStackView() const;

private:
	TWeakPtr<IMeshLayersController> Controller;
	TSharedPtr<SMeshLayersList> ListView;
};
