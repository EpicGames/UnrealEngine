// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MODELINGEDITORUI_API

/**
 * Simple widget that calls its OnTickDelegate in its Tick function, to allow for custom
 * tick behavior via composition instead of having to create a class and override the method,
 * for instance when creating custom slate in-line somewhere.
 */
class STickDelegateWidget : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_ThreeParams(FOnTick, const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	SLATE_BEGIN_ARGS(STickDelegateWidget) {}
	SLATE_EVENT(FOnTick, OnTickDelegate)
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	FOnTick OnTickDelegate;
};

#undef UE_API
