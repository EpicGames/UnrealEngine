// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyHandle.h"
#include "AssetRegistry/AssetData.h"
#include "CollectionManagerTypes.h"
#include "AssetThumbnail.h"
#include "IContentBrowserSingleton.h"

#define UE_API MODELINGEDITORUI_API

class SComboButton;
class FAssetThumbnailPool;
class FAssetThumbnail;
class SBorder;
class SBox;
class ITableRow;
class STableViewBase;
class SToolInputAssetPicker;

enum class EThumbnailDisplayMode
{
	AssetThumbnail,
	AssetName
};

/**
* SToolInputAssetComboPanel provides a similar UI to SComboPanel but
* specifically for picking Assets. The standard widget is a SComboButton
* that displays a thumbnail of the selected Asset, and on click a flyout
* panel is shown that has an Asset Picker tile view, as well as (optionally)
* a list of recently-used Assets, and also Collection-based filters.
* 
* Drag-and-drop onto the SComboButton is also supported, and the "selected Asset"
* can be mapped to/from a PropertyHandle. However note that a PropertyHandle is *not* required,
* each time the selection is modified the OnSelectionChanged(AssetData) delegate will also fire.
* 
* Note that "No Selection" is valid option by default
*/
class SToolInputAssetComboPanel : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSelectedAssetChanged, const FAssetData& AssetData);

	/** List of Collections with associated Name, used to provide pickable Collection filters */
	struct FNamedCollectionList
	{
		FNamedCollectionList() = default;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		FNamedCollectionList(const FNamedCollectionList&) = default;
		FNamedCollectionList(FNamedCollectionList&&) = default;

		FNamedCollectionList& operator=(const FNamedCollectionList&) = default;
		FNamedCollectionList& operator=(FNamedCollectionList&&) = default;

		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FString Name;

		TArray<FCollectionRef> Collections;

		UE_DEPRECATED(5.6, "Use Collections instead.")
		TArray<FCollectionNameType> CollectionNames;
	};

	/** IRecentAssetsProvider allows the Client to specify a set of "recently-used" Assets which the SToolInputAssetComboPanel will try to update as the selected Asset changes */
	class IRecentAssetsProvider
	{
	public:
		virtual ~IRecentAssetsProvider() = default;
		// SToolInputAssetComboPanel calls this to get the recent-assets list each time the flyout is opened
		virtual TArray<FAssetData> GetRecentAssetsList() = 0;
		// SToolInputAssetComboPanel calls this whenever the selected asset changes
		virtual void NotifyNewAsset(const FAssetData& NewAsset) = 0;
	};

public:

	SLATE_BEGIN_ARGS( SToolInputAssetComboPanel )
		: _ComboButtonTileSize(50, 50)
		, _FlyoutTileSize(85, 85)
		, _FlyoutSize(600, 400)
		, _AssetClassType(0)
		, _OnSelectionChanged()
		, _InitiallySelectedAsset()
		, _AssetThumbnailLabel(EThumbnailLabel::NoLabel)
		, _bForceShowEngineContent(false)
		, _bForceShowPluginContent(false)
		, _AssetViewType(EAssetViewType::Tile)
		, _ThumbnailDisplayMode(EThumbnailDisplayMode::AssetThumbnail)
	{
		}

		/** The size of the combo button icon tile */
		SLATE_ARGUMENT(FVector2D, ComboButtonTileSize)

		/** The size of the icon tiles in the flyout */
		SLATE_ARGUMENT(FVector2D, FlyoutTileSize)

		/** Size of the Flyout Panel */
		SLATE_ARGUMENT(FVector2D, FlyoutSize)

		/** Target PropertyHandle, selected value will be written here (Note: not required, selected  */
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, Property )

		/** Tooltip for the ComboButton. If Property is defined, this will be ignored. */
		SLATE_ARGUMENT( FText, ToolTipText )

		/** UClass of Asset to pick. Required, and only one class is supported */
		SLATE_ARGUMENT( UClass*, AssetClassType )

		/** (Optional) external provider/tracker of Recently-Used Assets. If not provided, Recent Assets area will not be shown. */
		SLATE_ARGUMENT(TSharedPtr<IRecentAssetsProvider>, RecentAssetsProvider)

		/** (Optional) set of collection-lists, if provided, button bar will be shown with each CollectionSet as an option */
		SLATE_ARGUMENT(TArray<FNamedCollectionList>, CollectionSets)
			
		/** This delegate is executed each time the Selected Asset is modified */
		SLATE_EVENT( FOnSelectedAssetChanged, OnSelectionChanged )

		/** Sets the asset selected by the widget before any user made selection occurs. */
		SLATE_ARGUMENT( FAssetData, InitiallySelectedAsset)

		/** Sets the type of label used for the asset picker tiles */
		SLATE_ARGUMENT( EThumbnailLabel::Type, AssetThumbnailLabel)

		/** Indicates if engine content should always be shown */
		SLATE_ARGUMENT(bool, bForceShowEngineContent)

		/** Indicates if plugin content should always be shown */
		SLATE_ARGUMENT(bool, bForceShowPluginContent)

		/** Sets the type of display we want the asset view to take. */
		SLATE_ARGUMENT(EAssetViewType::Type, AssetViewType)

		/** Display mode of the asset thumbnail */
		SLATE_ARGUMENT(EThumbnailDisplayMode, ThumbnailDisplayMode)

	SLATE_END_ARGS()


	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	// refresh the thumbnail using the AssetComboPanel's Property
	UE_API void RefreshThumbnailFromProperty();
	// refresh the thumbnail using provided AssetData, for cases where Property may not be available
	UE_API void RefreshThumbnail(const FAssetData& InAssetData) const;

protected:

	FVector2D ComboButtonTileSize;
	FVector2D FlyoutTileSize;
	FVector2D FlyoutSize;
	TSharedPtr<IPropertyHandle> Property;
	UClass* AssetClassType = nullptr;
	EThumbnailDisplayMode ThumbnailDisplayMode;


	/** Delegate to invoke when selection changes. */
	FOnSelectedAssetChanged OnSelectionChanged;

	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<FAssetThumbnailPool> ThumbnailPool;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<SBorder> ThumbnailBorder;

	TSharedPtr<IRecentAssetsProvider> RecentAssetsProvider;

	struct FRecentAssetInfo
	{
		int Index;
		FAssetData AssetData;
	};
	TArray<TSharedPtr<FRecentAssetInfo>> RecentAssetData;

	TArray<TSharedPtr<FAssetThumbnail>> RecentThumbnails;
	TArray<TSharedPtr<SBox>> RecentThumbnailWidgets;

	UE_API void UpdateRecentAssets();

	UE_API virtual void NewAssetSelected(const FAssetData& AssetData);
	UE_API virtual FReply OnAssetThumbnailDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	UE_API TSharedRef<ITableRow> OnGenerateWidgetForRecentList(TSharedPtr<FRecentAssetInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	TArray<FNamedCollectionList> CollectionSets;
	int32 ActiveCollectionSetIndex = -1;
	UE_API TSharedRef<SWidget> MakeCollectionSetsButtonPanel(TSharedRef<SToolInputAssetPicker> AssetPickerView);
};

#undef UE_API
