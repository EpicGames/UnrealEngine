// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetViewContentSources.h"
#include "IContentBrowserSingleton.h"
#include "SourcesData.h"

#define UE_API MODELINGEDITORUI_API

class FAssetTextFilter;
class FUICommandList;
class SAssetSearchBox;
class SAssetView;
class SComboButton;
enum class ECheckBoxState : uint8;

/**
 * SToolInputAssetPicker is designed to support Asset picking in Modeling Tools, 
 * where the Assets in question are input parameters/options to Tools, eg such as
 * a brush alpha texture for use in a Painting/Sculpting Tool. 
 * 
 * Implementation is derived from SAssetPicker (private class in the ContentBrowser module).
 * However many optional features have been stripped out as they are not relevant
 * in the Modeling-Tool Parameters context. 
 * 
 * Most settings are provided via the FAssetPickerConfig, which is passed to an SAssetView internally.
 * 
 * Unless you are really certain you need to use this class directly, it's likely that 
 * you are looking for SToolInputAssetComboPanel, which provides a combobox/flyout-style
 * widget suitable for user interface panels.
 */
class SToolInputAssetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SToolInputAssetPicker ){}

		/** A struct containing details about how the asset picker should behave */
		SLATE_ARGUMENT(FAssetPickerConfig, AssetPickerConfig)

	SLATE_END_ARGS()

	UE_API virtual ~SToolInputAssetPicker();

	/** Constructs this widget with InArgs */
	UE_API void Construct( const FArguments& InArgs );

	// SWidget implementation
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	// End of SWidget implementation

	/** Return the associated AssetView */
	const TSharedPtr<SAssetView>& GetAssetView() const { return AssetViewPtr; }

	/**
	 * Update the set of input Assets to be only based on the given set of Collections
	 * (or all Assets, if the Collections list is empty)
	 */
	UE_API virtual void UpdateAssetSourceCollections(TArray<FCollectionRef> Collections);

	UE_DEPRECATED(5.6, "Use the FCollectionRef overload instead.")
	UE_API virtual void UpdateAssetSourceCollections(TArray<FCollectionNameType> Collections);

private:
	/** Focuses the search box post-construct */
	UE_API EActiveTimerReturnType SetFocusPostConstruct( double InCurrentTime, float InDeltaTime );

	/** Special case handling for SAssetSearchBox key commands */
	UE_API FReply HandleKeyDownFromSearchBox(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Called when the editable text needs to be set or cleared */
	UE_API void SetSearchBoxText(const FText& InSearchText);

	/** Called by the editable text control when the search text is changed by the user */
	UE_API void OnSearchBoxChanged(const FText& InSearchText);

	/** Called by the editable text control when the user commits a text change */
	UE_API void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

	/** Handler for when the "None" button is clicked */
	UE_API FReply OnNoneButtonClicked();

	/** Handle forwarding picking events. We wrap OnAssetSelected here to prevent 'Direct' selections being identified as user actions */
	UE_API void HandleItemSelectionChanged(const FContentBrowserItem& InSelectedItem, ESelectInfo::Type InSelectInfo);

	/** Handler for when the user double clicks, presses enter, or presses space on an asset */
	UE_API void HandleItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod);

	/** Forces a refresh */
	UE_API void RefreshAssetView(bool bRefreshSources);

	/** @return The text to highlight on the assets  */
	UE_API FText GetHighlightedText() const;

private:

	/** The list of FrontendFilters currently applied to the asset view */
	TSharedPtr<FAssetFilterCollectionType> FrontendFilters;

	/** The asset view widget */
	TSharedPtr<SAssetView> AssetViewPtr;

	/** The search box */
	TSharedPtr<SAssetSearchBox> SearchBoxPtr;

	/** Called to when an asset is selected or the none button is pressed */
	FOnAssetSelected OnAssetSelected;

	/** Called when enter is pressed while an asset is selected */
	FOnAssetEnterPressed OnAssetEnterPressed;

	/** Called when any number of assets are activated */
	FOnAssetsActivated OnAssetsActivated;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** Filters needed for filtering the assets */
	TSharedPtr< FAssetFilterCollectionType > FilterCollection;
	TSharedPtr<FAssetTextFilter> TextFilter;

	EAssetTypeCategories::Type DefaultFilterMenuExpansion;

	/** The sources data currently used by the picker */
	FAssetViewContentSources CurrentContentSources;

	/** Current filter we are using, needed reset asset view after we have custom filtered */
	FARFilter CurrentBackendFilter;
};

#undef UE_API
