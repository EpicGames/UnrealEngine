// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshLayersController.h"

#define UE_API MODELINGEDITORUI_API

class FSculptLayersController : public IMeshLayersController
{
public:
	 UE_API FSculptLayersController();

	// get access to the Properties.
	// Warning: do not make modifications to the asset directly, use the controller API
	UMeshSculptLayerProperties* GetProperties() const { return Properties;};
	virtual void SetProperties(UMeshSculptLayerProperties* InProperties) override
	{
		Properties = const_cast<UMeshSculptLayerProperties*>(InProperties);
	};

private:
	TObjectPtr<UMeshSculptLayerProperties> Properties = nullptr;

public:
	
	UE_API virtual int32 AddMeshLayer() override;
	UE_API virtual bool RemoveMeshLayer(const int32 InLayerIndex) override;
	
	UE_API virtual void SetActiveLayer(const int32 InLayerIndex) const override;
	UE_API virtual int32 GetActiveLayer() const override;
	
	UE_API virtual void SetLayerName(const int32 InLayerIndex, const FName InName) const override;
	UE_API virtual FName GetLayerName(const int32 InLayerIndex) const override;
	UE_API virtual void SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType) const override;
	UE_API virtual double GetLayerWeight(const int32 InLayerIndex) const override;
	
	UE_API virtual int32 GetNumMeshLayers() const override;
	UE_API virtual bool MoveLayerInStack(int32 InLayerToMoveIndex, int32 InTargetIndex) override;
	UE_API virtual void RefreshLayersStackView() const override;
};

#undef UE_API
