// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveGizmo.h" // ETransformGizmoSubElements
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MODELINGEDITORUI_API

class FTransformGizmoDataBinder;
class SBorder;
namespace UE::ToolWidgets { class SDraggableBoxOverlay; }
class SWidget;
class UInteractiveToolsContext;

/**
 * An overlay that contains a draggable panel that mirrors the values of a UCombinedTransformGizmo
 * and allows them to be settable by scrubbing the values or typing.
 *
 * To use it, just add the overlay to your viewport and call BindToGizmoContextObject (sometime after
 * TransformGizmoUtil::RegisterTransformGizmoContextObject has already been called). This will bind
 * any gizmos subsequently created through TransformGizmoUtil to the panel.
 *
 * MakeNumericalUISubMenu creates a submenu that can be slotted somewhere to enable/disable the UI
 * panel or to reset its position in the viewport.
 */
class STransformGizmoNumericalUIOverlay : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STransformGizmoNumericalUIOverlay) 
		: _bPositionRelativeToBottom(true)
		, _DefaultLeftPadding(15)
		, _DefaultVerticalPadding(15)
		, _TranslationScrubSensitivity(1)
	{}

	// When true (the default) the panel is positioned by measuring a distance from the bottom of
	// the viewport, and resizing the viewport will keep that distance the same.
	SLATE_ARGUMENT(bool, bPositionRelativeToBottom)
	// These values determine where the panel is reset when ResetPositionInViewport() is called.
	SLATE_ARGUMENT(float, DefaultLeftPadding)
	SLATE_ARGUMENT(float, DefaultVerticalPadding)

	// Usually do not need setting. See comment for SetCustomDisplayConversionFunctions
	SLATE_ARGUMENT(TFunction<FVector3d(const FVector3d& InternalValue)>, InternalToDisplayFunction)
	SLATE_ARGUMENT(TFunction<FVector3d(const FVector3d& DisplayValue)>, DisplayToInternalFunction)
	SLATE_ARGUMENT(double, TranslationScrubSensitivity)
	// Usually does not need setting. See comment for SetDefaultLocalReferenceTransform
	SLATE_ARGUMENT(TOptional<FTransform>, DefaultLocalReferenceTransform)

	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual ~STransformGizmoNumericalUIOverlay();

	/**
	 * Binding to the context object makes it so that the overlay gets attached to any subsequently created 
	 * transform gizmos.
	 */
	UE_API bool BindToGizmoContextObject(UInteractiveToolsContext* ToolsContext);

	virtual bool IsEnabled() { return bIsEnabled; }
	UE_API virtual void SetEnabled(bool bEnabledIn);

	/**
	 * Resets the position of the draggable UI box in the overlay to its default.
	 */
	UE_API virtual void ResetPositionInViewport();

	/**
	 * Creates a submenu to deal with the numerical UI, for placement in some toolbar by the mode/editor.
	 */
	UE_API virtual void MakeNumericalUISubMenu(FMenuBuilder& MenuBuilder);

	/**
	 * Clears all gizmo bindings
	 */
	UE_API virtual void Reset();

	/**
	 * Allows the positional (translation) copmonent to be displayed in different units than the gizmo world, for
	 * instance in a UV editor where the world coordinates are not actually 0 to 1.
	 */
	UE_API virtual void SetCustomDisplayConversionFunctions(
		TFunction<FVector3d(const FVector3d& InternalValue)> InternalToDisplayConversionIn,
		TFunction<FVector3d(const FVector3d& DisplayValue)> DisplayToInternalConversionIn,
		double TranslationScrubSensitivityIn = 1.0);

	/**
	 * After calling this function, any newly initialized gizmos will be given the provided reference
	 * transform as their custom reference transform. This mainly meant to allow the UI to be used in
	 * a 2D situation (e.g. UV editor) where the gizmos only have two axes. Normally, we only allow 
	 * delta mode in such a case, but the 2D editor can provide the used plane's reference transform 
	 * to allow the gizmos to be usable in destination mode as well.
	 */
	UE_API virtual void SetDefaultLocalReferenceTransform(const TOptional<FTransform>& CustomReferenceTransform);

protected:

	// Currently displayed values
	FVector3d DisplayTranslation;
	FVector3d DisplayEulerAngles;
	FVector3d DisplayScale;

	// Holds the functionality that binds the displayed values to the gizmo.
	TSharedPtr<FTransformGizmoDataBinder> DataBinder;

	TSharedPtr<UE::ToolWidgets::SDraggableBoxOverlay> DraggableBoxOverlay;

	// The part of the UI that changes when a different gizmo is chosen
	TSharedPtr<SBorder> WidgetContents;

	// The values to which the UI positioning is reset if asked
	float DefaultLeftPadding = 15.0f;
	float DefaultVerticalPadding = 75.0f;

	// Helpers for building the UI when changing gizmo
	UE_API TSharedRef<SWidget> CreateWidgetForGizmo(ETransformGizmoSubElements GizmoElements);
	UE_API SNumericEntryBox<double>::FArguments CreateComponentBaseArgs(FVector3d& TargetVector, int ComponentIndex, TAttribute<bool> IsEnabled);
	UE_API TSharedRef<SWidget> CreateTranslationComponent(int ComponentIndex, TAttribute<bool> IsEnabled);
	UE_API TSharedRef<SWidget> CreateRotationComponent(int ComponentIndex, TAttribute<bool> IsEnabled);
	UE_API TSharedRef<SWidget> CreateScaleComponent(int ComponentIndex, TAttribute<bool> IsEnabled);

	// We have to respond to coordinate system changes for the gizmo, and to do that we need to be able to poll
	// on tick, since we don't get notifications about it.
	EToolContextCoordinateSystem LastCoordinateSystem;
	UE_API void OnTickWidget(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	// Modifier for scrubbing translation, if we're using custom display functions.
	double TranslationScrubSensitivity = 1;

	bool bIsEnabled = true;
	bool bPositionRelativeToBottom = true;
};

#undef UE_API
