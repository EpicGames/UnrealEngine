// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingWidgets/SDraggableBox.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h" // FDragDropEvent
#include "Styling/SlateTypes.h" // FWindowStyle
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h" // SVerticalBox
#include "Widgets/SWindow.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SDraggableBoxOverlay::Construct(const FArguments& InArgs)
{
	// Depending on whether we're positioning relative to the bottom or top, we'll update the margin of the
	// draggable box differently.
	TAttribute<FMargin> DraggableBoxPaddingAttribute;
	SDraggableBox::FOnDragComplete OnDragCompleteFunction;

	if (InArgs._bPositionRelativeToBottom)
	{
		DraggableBoxPaddingAttribute = MakeAttributeLambda([this]()
		{
			return FMargin(DraggableBoxPaddingHorizontal, 0, 0, DraggableBoxPaddingVertical);
		});
		OnDragCompleteFunction = SDraggableBox::FOnDragComplete::CreateLambda([this](const FVector2D& ScreenSpacePosition)
		{
			FVector2D BottomLeftScreenSpace(ScreenSpacePosition.X, ScreenSpacePosition.Y + DraggableBox->GetTickSpaceGeometry().GetAbsoluteSize().Y);
			FVector2D BottomLeftLocal = ContainingBox->GetTickSpaceGeometry().AbsoluteToLocal(BottomLeftScreenSpace);

			DraggableBoxPaddingHorizontal = BottomLeftLocal.X;
			DraggableBoxPaddingVertical = ContainingBox->GetTickSpaceGeometry().GetLocalSize().Y - BottomLeftLocal.Y;
		});
	}
	else
	{
		DraggableBoxPaddingAttribute = MakeAttributeLambda([this]()
		{
			return FMargin(DraggableBoxPaddingHorizontal, DraggableBoxPaddingVertical, 0, 0);
		});
		OnDragCompleteFunction = SDraggableBox::FOnDragComplete::CreateLambda([this](const FVector2D& ScreenSpacePosition)
		{
			FVector2D Position = ContainingBox->GetTickSpaceGeometry().AbsoluteToLocal(ScreenSpacePosition);
			DraggableBoxPaddingHorizontal = Position.X;
			DraggableBoxPaddingVertical = Position.Y;
		});
	}

	ChildSlot
	[
		SAssignNew(ContainingBox, SVerticalBox)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SVerticalBox::Slot()

		.HAlign(HAlign_Left)
		.VAlign(InArgs._bPositionRelativeToBottom ? VAlign_Bottom : VAlign_Top)
		.Padding(DraggableBoxPaddingAttribute)
		[
			// The draggable box itself
			SAssignNew(DraggableBox, SDraggableBox)
			.OnDragComplete(OnDragCompleteFunction)
			[
				InArgs._Content.Widget
			]
		]
	]
	;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SDraggableBoxOverlay::SetBoxPosition(float HorizontalPosition, float VerticalPosition)
{
	DraggableBoxPaddingHorizontal = HorizontalPosition;
	DraggableBoxPaddingVertical = VerticalPosition;
}

// Draggable box:

void SDraggableBox::Construct(const FArguments& InArgs)
{
	OnDragComplete = InArgs._OnDragComplete;

	ChildSlot
	[
		SAssignNew(InnerWidget, SBackgroundBlur)
		.BlurStrength(2)
		.Padding(0.0f)
		.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
		[
			SNew(SBorder)
			.Padding(0.0f)
			[
				InArgs._Content.Widget
			]
		]
	];
}

FReply SDraggableBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	// Need to remember where within the box we grabbed. We do this here instead of OnDragDetected because 
	// our mouse can potentially travel some distance before OnDragDetected fires.
	ScreenSpaceOffsetOfGrab = MouseEvent.GetScreenSpacePosition() - MyGeometry.GetAbsolutePosition();

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FReply SDraggableBox::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// The drag/drop operation moves a transparent copy of our contents, so we should hide ourselves to make it
	// look like the contents are moving.
	SetVisibility(EVisibility::Collapsed);

	// This will be called when the drag/drop operation is done
	FOnDragComplete OnUIDropped = FOnDragComplete::CreateLambda([this] (const FVector2D& ScreenSpacePosition) {
		OnDragComplete.ExecuteIfBound(ScreenSpacePosition);
		SetVisibility(EVisibility::SelfHitTestInvisible);
	});

	TSharedRef<FDraggableBoxUIDragOperation> DragDropOperation =
		FDraggableBoxUIDragOperation::New(
			// Note that it is important to give the inner widget here, since SDraggableBox itself is invisible now
			InnerWidget.ToSharedRef(),
			ScreenSpaceOffsetOfGrab,
			GetDesiredSize(),
			OnUIDropped
		);

	return FReply::Handled().BeginDragDrop(DragDropOperation);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Drag operation:

void FDraggableBoxUIDragOperation::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FVector2D ScreenSpaceDropLocation = MouseEvent.GetScreenSpacePosition() - DecoratorOffsetFromCursor;

	OnDragComplete.ExecuteIfBound(ScreenSpaceDropLocation);

	// Call super
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);

	UIBeingDragged.Reset();
}

// These are mostly copied from FInViewportUIDragOperation:
void FDraggableBoxUIDragOperation::OnDragged(const FDragDropEvent& DragDropEvent)
{
	// The tab is being dragged. Move the the decorator window to match the cursor position.
	FVector2D TargetPosition = DragDropEvent.GetScreenSpacePosition() - DecoratorOffsetFromCursor;
	CursorDecoratorWindow->UpdateMorphTargetShape(FSlateRect(TargetPosition.X, TargetPosition.Y, TargetPosition.X + LastContentSize.X, TargetPosition.Y + LastContentSize.Y));
	CursorDecoratorWindow->MoveWindowTo(TargetPosition);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedRef<FDraggableBoxUIDragOperation> FDraggableBoxUIDragOperation::New(const TSharedRef<SWidget>& InUIToBeDragged, 
	const FVector2D InDecoratorOffset, const FVector2D& OwnerAreaSize, SDraggableBox::FOnDragComplete& OnDragComplete)
{
	const TSharedRef<FDraggableBoxUIDragOperation> Operation = MakeShareable(new FDraggableBoxUIDragOperation(InUIToBeDragged, InDecoratorOffset, OwnerAreaSize, OnDragComplete));
	return Operation;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDraggableBoxUIDragOperation::FDraggableBoxUIDragOperation(const TSharedRef<class SWidget>& InUIToBeDragged, 
	const FVector2D InDecoratorOffset, const FVector2D& OwnerAreaSize, SDraggableBox::FOnDragComplete& InOnDragComplete)
	: UIBeingDragged(InUIToBeDragged)
	, DecoratorOffsetFromCursor(InDecoratorOffset)
	, LastContentSize(OwnerAreaSize)
	, OnDragComplete(InOnDragComplete)
{
	// Create the decorator window that we will use during this drag and drop to make the user feel like
	// they are actually dragging a piece of UI.
	const bool bShowImmediately = true;
	CursorDecoratorWindow = FSlateApplication::Get().AddWindow(SWindow::MakeStyledCursorDecorator(FAppStyle::Get().GetWidgetStyle<FWindowStyle>("InViewportDecoratorWindow")), bShowImmediately);
	CursorDecoratorWindow->SetOpacity(0.45f);
	CursorDecoratorWindow->SetContent
	(
		InUIToBeDragged
	);

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS