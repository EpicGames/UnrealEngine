// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Class that can be used to place a draggable box into a viewport or some other large widget as an
 * overlay. Just place the widget that you want to be draggable as the contents of SDraggableBoxOverlay.
 */
class UE_DEPRECATED(5.6, "Use UE::ToolWidgets::SDraggableBoxOverlay instead.") SDraggableBoxOverlay : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDraggableBoxOverlay) {}

	// When true, the positioning of the box is relative to the bottom of the widget rather than the top,
	// so the VerticalPosition in SetBoxPosition is interpreted as distance from the bottom of the
	// containing widget. The choice here depends on how your widget is generally positioned, for instance
	// if it is near the bottom, you want this to be true so that making the window smaller does not
	// immediately clip your box.
	SLATE_ARGUMENT(bool, bPositionRelativeToBottom)

	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	MODELINGUI_API void Construct(const FArguments& InArgs);

	/**
	 * Sets the box position in the overlay. Horizontal position is distance from left, and vertical position
	 * is distance from bottom or from top depending on whether bPositionRelativeToBottom == true.
	 */
	MODELINGUI_API void SetBoxPosition(float HorizontalPosition, float VerticalPosition);

protected:

	TSharedPtr<SWidget> DraggableBox;
	TSharedPtr<SWidget> ContainingBox;

	float DraggableBoxPaddingHorizontal = 0;
	float DraggableBoxPaddingVertical = 0;
};

/**
 * A widget for the draggable box itself, which requires its parent to handle its positioning in
 * response to the drag.
 * 
 * Users probably shouldn't use this class directly; rather, they should use SDraggableBoxOverlay, 
 * which will put its contents into a draggable box and properly handle the dragging without the
 * user having to set it up.
 */
class UE_DEPRECATED(5.6, "SDraggableBox is now private. Use UE::ToolWidgets::SDraggableBoxOverlay instead.") SDraggableBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnDragComplete, const FVector2D& /*ScreenSpacePosition*/);

	SLATE_BEGIN_ARGS(SDraggableBox) {}
	SLATE_EVENT(FOnDragComplete, OnDragComplete)
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	MODELINGUI_API void Construct(const FArguments& InArgs);

	// SWidget overrides necessary for box to be draggable
	MODELINGUI_API FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	MODELINGUI_API FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

protected:
	TSharedPtr<SWidget> InnerWidget;
	FOnDragComplete OnDragComplete;

	// Remembers the point in the box that we grabbed (on click, rather than when the
	// drag was confirmed)
	FVector2D ScreenSpaceOffsetOfGrab;
};


/**
 * A drag/drop operation used by SDraggableBox, largely modeled on FInViewportUIDragOperation, except
 * instead of requiring the drop location to be able to handle that particular class to trigger the
 * OnDragComplete delegate, it just triggers on any drop. This makes it possible to use it in any
 * viewports, not just the level editor.
 */
class UE_DEPRECATED(5.6, "FDraggableBoxUIDragOperation is now private. Use UE::ToolWidgets::SDraggableBoxOverlay instead.") FDraggableBoxUIDragOperation : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FDraggableBoxUIDragOperation, FDragDropOperation)

	// FDragDropOperation
	MODELINGUI_API virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	MODELINGUI_API virtual void OnDragged(const FDragDropEvent& DragDropEvent) override;

	/**
	 * Create this Drag and Drop Content
	 *
	 * @param InUIToBeDragged	  The UI being dragged
	 * @param InDecoratorOffset   Where within the UI we grabbed, so we're not dragging by the upper left of the UI.
	 * @param OwnerAreaSize       Size of the DockArea at the time when we start dragging.
	 * @param OnDragComplete      Delegate to call when dropped. Gets passed in the screen space location of the top left corner.
	 *
	 * @return a new FDockingDragOperation
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MODELINGUI_API static TSharedRef<FDraggableBoxUIDragOperation> New(const TSharedRef<class SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffset, 
		const FVector2D& OwnerAreaSize, SDraggableBox::FOnDragComplete& OnDragComplete);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FDraggableBoxUIDragOperation() {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	
	/** The constructor is protected, so that this class can only be instantiated as a shared pointer. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MODELINGUI_API FDraggableBoxUIDragOperation(const TSharedRef<class SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffsetFromCursor, 
		const FVector2D& OwnerAreaSize, SDraggableBox::FOnDragComplete& OnDragCompletee);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Shared pointer to our contents, which are a weak pointer.
	 */
	TSharedPtr<class SWidget> UIBeingDragged;

	// Screen space offset of grab location from the top left corner of the rectangle being dragged.
	FVector2D DecoratorOffsetFromCursor;

	// Size of rectangle being dragged
	FVector2D LastContentSize;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SDraggableBox::FOnDragComplete OnDragComplete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
